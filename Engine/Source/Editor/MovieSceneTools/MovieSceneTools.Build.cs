// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MovieSceneTools : ModuleRules
{
	public MovieSceneTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(
            new string[] {
				System.IO.Path.Combine(GetModuleDirectory("UnrealEd"), "Private"), //compatibility for FBX importer
			}
        );

		OverridePackageType = PackageOverrideType.EngineDeveloper;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"LevelSequence",
				"MovieSceneCapture",
				"UnrealEd",
				"Sequencer",
                "EditorWidgets",
				"SequencerCore",
				"Constraints"
            }
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "ActorPickerMode",
				"AnimationBlueprintLibrary",
				"AppFramework",
				"CinematicCamera",
				"ClassViewer",
				"DataLayerEditor",
                "CurveEditor",
                "DesktopPlatform",
                "Json",
                "EditorFramework",
				"EditorSubsystem",
				"JsonUtilities",
                "LiveLinkInterface",
                "MessageLog",
				"MovieScene",
				"MovieSceneTracks",
				"BlueprintGraph",
				"Kismet",
				"KismetCompiler",
                "GraphEditor",
                "ContentBrowser",
				"Slate",
				"SlateCore",
				"SceneOutliner",
                "EditorStyle",
				"PropertyEditor",
                "MaterialEditor",
				"RenderCore",
				"RHI",
				"SequenceRecorder",
				"SequencerCore",
				"TimeManagement",
				"ToolWidgets",
                "AnimationCore",
				"TimeManagement",
                "XmlParser",
				"ToolMenus",
				"AnimGraphRuntime"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
                "AssetRegistry",
				"AssetTools",
                "Settings",
                "MainFrame",
                "Analytics",
            }
        );

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
                "AssetRegistry",
				"AssetTools",
			    "MainFrame",
			}
		);

        CircularlyReferencedDependentModules.AddRange(
            new string[] {
                "Sequencer",
            }
        );

        AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");
    }
}
