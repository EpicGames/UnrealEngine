// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedLevelSequenceCapture.h"
#include "MovieScene.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Slate/SceneViewport.h"
#include "Misc/CommandLine.h"
#include "LevelSequenceActor.h"
#include "JsonObjectConverter.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "MovieSceneTranslatorEDL.h"
#include "FCPXML/FCPXMLMovieSceneTranslator.h"
#include "EngineUtils.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "TimerManager.h"
#include "GameFramework/PlayerController.h"
#include "Camera/CameraComponent.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneToolHelpers.h"
#include "Protocols/AudioCaptureProtocol.h"
#include "ShaderCompiler.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedLevelSequenceCapture)

const FName UAutomatedLevelSequenceCapture::AutomatedLevelSequenceCaptureUIName = FName(TEXT("AutomatedLevelSequenceCaptureUIInstance"));

struct FMovieSceneTimeController_FrameStep : FMovieSceneTimeController
{
	FMovieSceneTimeController_FrameStep()
		: StartTimeOffset(0)
	{}

	virtual void OnTick(float DeltaSeconds, float InPlayRate) override
	{}

	virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime)
	{
		DeltaTime = FFrameTime(0);
		StartTimeOffset = FFrameTime(0);
	} 

	virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override
	{
		TOptional<FQualifiedFrameTime> StartTimeIfPlaying = GetPlaybackStartTime();
		if (!StartTimeIfPlaying.IsSet())
		{
			UE_LOGF(LogMovieSceneCapture, VeryVerbose, "FMovieSceneTimeController_FrameStep::OnRequestCurrentTime = Paused at frame %d subframe %f", InCurrentTime.Time.FrameNumber.Value, InCurrentTime.Time.GetSubFrame());
			return InCurrentTime.Time;
		}
		else
		{
			// Scale the delta time (should be one frame) by this frame's play rate, and add it to the current time offset
			if (InPlayRate == 1.f)
			{
				StartTimeOffset += DeltaTime;
			}
			else
			{
				StartTimeOffset += DeltaTime * InPlayRate;
			}

			DeltaTime = FFrameTime(0);

			FFrameTime NewTime = StartTimeIfPlaying->ConvertTo(InCurrentTime.Rate) + StartTimeOffset;
			UE_LOGF(LogMovieSceneCapture, VeryVerbose, "FMovieSceneTimeController_FrameStep::OnRequestCurrentTime = Playing at frame %d subframe %f", NewTime.FrameNumber.Value, NewTime.GetSubFrame());
			return NewTime;
		}
	}

	FFrameTime DeltaTime;
	FFrameTime StartTimeOffset;
};

UMovieScene* GetMovieScene(TWeakObjectPtr<ALevelSequenceActor> LevelSequenceActor)
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();
	if (!Actor)
	{
		return nullptr;
	}

	ULevelSequence* LevelSequence = Actor->GetSequence();
	if (!LevelSequence)
	{
		return nullptr;
	}

	return LevelSequence->GetMovieScene();
}

UMovieSceneCinematicShotTrack* GetCinematicShotTrack(TWeakObjectPtr<ALevelSequenceActor> LevelSequenceActor)
{
	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return nullptr;
	}

	return MovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
}

UMovieSceneMotionVectorSimulationSystem* FindMotionVectorSimulation(ALevelSequenceActor* LevelSequenceActor)
{
	return static_cast<IMovieScenePlayer*>(LevelSequenceActor->GetSequencePlayer())->GetEvaluationTemplate().GetEntitySystemLinker()->FindSystem<UMovieSceneMotionVectorSimulationSystem>();
}

UAutomatedLevelSequenceCapture::UAutomatedLevelSequenceCapture(const FObjectInitializer& Init)
	: Super(Init)
{
#if WITH_EDITORONLY_DATA == 0
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		checkf(false, TEXT("Automated level sequence captures can only be used in editor builds."));
	}
#else
	bUseCustomStartFrame = false;
	CustomStartFrame = 0;
	bUseCustomEndFrame = false;
	CustomEndFrame = 1;
	WarmUpFrameCount = 0;
	DelayBeforeWarmUp = 0.0f;
	DelayBeforeShotWarmUp = 0.0f;
	DelayEveryFrame = 0.0f;
	bWriteEditDecisionList = true;
	bWriteFinalCutProXML = true;

	RemainingWarmUpFrames = 0;

	NumShots = 0;
	ShotIndex = -1;

	BurnInOptions = Init.CreateDefaultSubobject<ULevelSequenceBurnInOptions>(this, MovieSceneCaptureUIName);
#endif
}

#if WITH_EDITORONLY_DATA
void UAutomatedLevelSequenceCapture::AddFormatMappings(TMap<FString, FStringFormatArg>& OutFormatMappings, const FFrameMetrics& FrameMetrics) const
{
	OutFormatMappings.Add(TEXT("sequence"), CachedState.RootName);

	OutFormatMappings.Add(TEXT("shot"), CachedState.CurrentShotName);
	OutFormatMappings.Add(TEXT("shot_frame"), FString::Printf(TEXT("%0*d"), Settings.ZeroPadFrameNumbers, CachedState.CurrentShotLocalTime.Time.FrameNumber.Value));

	if (CachedState.CameraComponent.IsValid())
	{
		AActor* OuterActor = Cast<AActor>(CachedState.CameraComponent.Get()->GetOuter());
		if (OuterActor)
		{
			OutFormatMappings.Add(TEXT("camera"), OuterActor->GetActorLabel());
		}
	}
}

void UAutomatedLevelSequenceCapture::Initialize(TSharedPtr<FSceneViewport> InViewport, int32 PIEInstance)
{
	TimeController = MakeShared<FMovieSceneTimeController_FrameStep>();
	Viewport = InViewport;

	// Apply command-line overrides from parent class first. This needs to be called before setting up the capture strategy with the desired frame rate.
	Super::Initialize(InViewport);

	// Apply command-line overrides
	{
		FString LevelSequenceAssetPath;
		if (FParse::Value(FCommandLine::Get(), TEXT("-LevelSequence="), LevelSequenceAssetPath))
		{
			LevelSequenceAsset.SetPath(LevelSequenceAssetPath);
		}

		FString ShotNameOverride;
		if (FParse::Value(FCommandLine::Get(), TEXT("-Shot="), ShotNameOverride))
		{
			ShotName = ShotNameOverride;
		}

		int32 StartFrameOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieStartFrame=" ), StartFrameOverride ) )
		{
			bUseCustomStartFrame = true;
			CustomStartFrame = StartFrameOverride;
		}

		int32 EndFrameOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieEndFrame=" ), EndFrameOverride ) )
		{
			bUseCustomEndFrame = true;
			CustomEndFrame = EndFrameOverride;
		}

		int32 WarmUpFrameCountOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieWarmUpFrames=" ), WarmUpFrameCountOverride ) )
		{
			WarmUpFrameCount = WarmUpFrameCountOverride;
		}

		float DelayBeforeWarmUpOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieDelayBeforeWarmUp=" ), DelayBeforeWarmUpOverride ) )
		{
			DelayBeforeWarmUp = DelayBeforeWarmUpOverride;
		}

		float DelayBeforeShotWarmUpOverride;
		if( FParse::Value( FCommandLine::Get(), TEXT( "-MovieDelayBeforeShotWarmUp=" ), DelayBeforeShotWarmUpOverride ) )
		{
			DelayBeforeShotWarmUp = DelayBeforeShotWarmUpOverride;
		}

		float DelayEveryFrameOverride;
		if (FParse::Value(FCommandLine::Get(), TEXT("-MovieDelayEveryFrame="), DelayEveryFrameOverride))
		{
			DelayEveryFrame = DelayEveryFrameOverride;
		}

		bool bWriteEditDecisionListOverride;
		if (FParse::Bool(FCommandLine::Get(), TEXT("-WriteEditDecisionList="), bWriteEditDecisionListOverride))
		{
			bWriteEditDecisionList = bWriteEditDecisionListOverride;
		}

		bool bWriteFinalCutProXMLOverride;
		if (FParse::Bool(FCommandLine::Get(), TEXT("-WriteFinalCutProXML="), bWriteFinalCutProXMLOverride))
		{
			bWriteFinalCutProXML = bWriteFinalCutProXMLOverride;
		}
	}

	if (Settings.bUsePathTracer)
	{
		float PathTracerSamplePerPixel = float(Settings.FrameRate.AsSeconds(Settings.PathTracerSamplePerPixel));

		if (DelayEveryFrame != PathTracerSamplePerPixel)
		{
			UE_LOGF(LogMovieSceneCapture, Log, "Delay every frame overridden by path tracer sample per pixel: %f", PathTracerSamplePerPixel);

			DelayEveryFrame = PathTracerSamplePerPixel;
		}
	}

	ALevelSequenceActor* Actor = LevelSequenceActor.Get();

	// If we don't have a valid actor, attempt to find a level sequence actor in the world that references this asset
	if( Actor == nullptr )
	{
		if( LevelSequenceAsset.IsValid() )
		{
			ULevelSequence* Asset = Cast<ULevelSequence>( LevelSequenceAsset.TryLoad() );
			if( Asset != nullptr )
			{
				for( auto It = TActorIterator<ALevelSequenceActor>( InViewport->GetClient()->GetWorld() ); It; ++It )
				{
					if( It->GetSequence() == Asset )
					{
						// Found it!
						Actor = *It;
						this->LevelSequenceActor = Actor;

						break;
					}
				}
			}
		}
	}

	if (!Actor)
	{
		ULevelSequence* Asset = Cast<ULevelSequence>(LevelSequenceAsset.TryLoad());
		if (Asset)
		{
			// Spawn a new actor
			Actor = InViewport->GetClient()->GetWorld()->SpawnActor<ALevelSequenceActor>();
			Actor->SetSequence(Asset);
		
			LevelSequenceActor = Actor;
		}
		else
		{
			//FPlatformMisc::RequestExit(FMovieSceneCaptureExitCodes::AssetNotFound);
		}
	}

	if (Actor && !ShotName.IsEmpty())
	{
		UMovieScene* MovieScene = GetMovieScene(Actor);

		UMovieSceneCinematicShotTrack* CinematicShotTrack = GetCinematicShotTrack(Actor);

		if (CinematicShotTrack && MovieScene)
		{
			FFrameRate TickResolution = MovieScene->GetTickResolution();
			FFrameRate DisplayRate = MovieScene->GetDisplayRate();

			UMovieSceneCinematicShotSection* CinematicShotSection = nullptr;
			for (UMovieSceneSection* Section : CinematicShotTrack->GetAllSections())
			{
				if (UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section))
				{
					if (ShotSection->GetShotDisplayName() == ShotName)
					{
						CinematicShotSection = ShotSection;
						break;
					}
				}
			}

			if (CinematicShotSection)
			{
				bUseCustomStartFrame = true;
				bUseCustomEndFrame = true;

				FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(CinematicShotSection->GetRange());
				FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(CinematicShotSection->GetRange());

				CustomStartFrame = FFrameRate::TransformTime(StartFrame, TickResolution, DisplayRate).CeilToFrame();
				CustomEndFrame = FFrameRate::TransformTime(EndFrame, TickResolution, DisplayRate).CeilToFrame();
				
				UE_LOGF(LogMovieSceneCapture, Log, "Found shot '%ls' to capture. Setting custom start and end frames to %d and %d.", *ShotName, CustomStartFrame.Value, CustomEndFrame.Value);
			}
			else
			{
				UE_LOGF(LogMovieSceneCapture, Error, "Could not find named shot '%ls' to capture", *ShotName);
			}
		}
	}

	ExportEDL();
	ExportFCPXML();

	if (Actor)
	{
		// Ensure it doesn't loop (-1 is indefinite)
		Actor->PlaybackSettings.LoopCount.Value = 0;
		Actor->GetSequencePlayer()->SetTimeController(TimeController);
		Actor->PlaybackSettings.bPauseAtEnd = true;
		Actor->PlaybackSettings.bAutoPlay = false;

		if (BurnInOptions)
		{
			Actor->BurnInOptions = BurnInOptions;

			bool bUseBurnIn = false;
			if( FParse::Bool( FCommandLine::Get(), TEXT( "-UseBurnIn=" ), bUseBurnIn ) )
			{
				Actor->BurnInOptions->bUseBurnIn = bUseBurnIn;
			}
		}

		// Make sure we're not playing yet, and have a fully up to date player based on the above settings (in case AutoPlay was called from BeginPlay)
		if( Actor->GetSequencePlayer() != nullptr )
		{
			if (Actor->GetSequencePlayer()->IsPlaying())
			{
				Actor->GetSequencePlayer()->Stop();
			}
			Actor->InitializePlayer();
		}

		if (InitializeShots())
		{
			FFrameNumber StartTime, EndTime;
			SetupShot(StartTime, EndTime);
		}
		Actor->RefreshBurnIn();
	}
	else
	{
		UE_LOGF(LogMovieSceneCapture, Error, "Could not find or create a Level Sequence Actor for this capture. Capturing will fail.");
	}

	CaptureState = ELevelSequenceCaptureState::Setup;
	CaptureStrategy = MakeShareable(new FFixedTimeStepCaptureStrategy(Settings.GetFrameRate()));
	CaptureStrategy->OnInitialize();

	// Cache these off. We override them for audio passes and that overrides the settings members directly
	// so we need to be able to restore them at the end
	CachedWarmUpFrameCount = WarmUpFrameCount;
	CachedDelayBeforeWarmUp = DelayBeforeWarmUp;
	CachedDelayBeforeShotWarmUp = DelayBeforeShotWarmUp;
}

bool UAutomatedLevelSequenceCapture::InitializeShots()
{
	NumShots = 0;
	ShotIndex = -1;
	CachedShotStates.Empty();

	if (Settings.HandleFrames <= 0)
	{
		return false;
	}

	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return false;
	}

	UMovieSceneCinematicShotTrack* CinematicShotTrack = GetCinematicShotTrack(LevelSequenceActor);
	if (!CinematicShotTrack)
	{
		return false;
	}

	NumShots = CinematicShotTrack->GetAllSections().Num();
	ShotIndex = 0;
	CachedPlaybackRange = MovieScene->GetPlaybackRange();

	// Compute handle frames in tick resolution space since that is what the section ranges are defined in
	FFrameNumber HandleFramesResolutionSpace = ConvertFrameTime(Settings.HandleFrames, Settings.GetFrameRate(), MovieScene->GetTickResolution()).FloorToFrame();

	CinematicShotTrack->SortSections();

	for (int32 SectionIndex = 0; SectionIndex < CinematicShotTrack->GetAllSections().Num(); ++SectionIndex)
	{
		UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(CinematicShotTrack->GetAllSections()[SectionIndex]);
		UMovieScene* ShotMovieScene = ShotSection->GetSequence() ? ShotSection->GetSequence()->GetMovieScene() : nullptr;

		if (ShotMovieScene != nullptr)
		{
			// Expand the inner shot section range by the handle size, multiplied by the difference between the outer and inner tick resolutions (and factoring in the time scale)
			const float OuterToInnerRateDilation = (MovieScene->GetTickResolution() == ShotMovieScene->GetTickResolution()) ? 1.f : (ShotMovieScene->GetTickResolution() / MovieScene->GetTickResolution()).AsDecimal();
			// coderot: this is not considering timewarping
			const float OuterToInnerScale = OuterToInnerRateDilation;

			CachedShotStates.Add(FCinematicShotCache(ShotSection->IsActive(), ShotSection->IsLocked(), ShotSection->GetRange(), ShotMovieScene ? ShotMovieScene->GetPlaybackRange() : TRange<FFrameNumber>::Empty()));

			if (ShotMovieScene)
			{
				TRange<FFrameNumber> NewPlaybackRange = UE::MovieScene::ExpandRange(ShotMovieScene->GetPlaybackRange(), FFrameNumber(FMath::FloorToInt(HandleFramesResolutionSpace.Value * OuterToInnerScale)));
				ShotMovieScene->SetPlaybackRange(NewPlaybackRange, false);
			}

			ShotSection->SetIsLocked(false);
			ShotSection->SetIsActive(false);

			ShotSection->SetRange(UE::MovieScene::ExpandRange(ShotSection->GetRange(), HandleFramesResolutionSpace));
		}
	}
	return NumShots > 0;
}

void UAutomatedLevelSequenceCapture::RestoreShots()
{
	if (Settings.HandleFrames <= 0)
	{
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return;
	}

	UMovieSceneCinematicShotTrack* CinematicShotTrack = GetCinematicShotTrack(LevelSequenceActor);
	if (!CinematicShotTrack)
	{
		return;
	}

	MovieScene->SetPlaybackRange(CachedPlaybackRange, false);

	for (int32 SectionIndex = 0; SectionIndex < CinematicShotTrack->GetAllSections().Num(); ++SectionIndex)
	{
		UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(CinematicShotTrack->GetAllSections()[SectionIndex]);
		UMovieScene* ShotMovieScene = ShotSection->GetSequence() ? ShotSection->GetSequence()->GetMovieScene() : nullptr;
		if (ShotMovieScene)
		{
			ShotMovieScene->SetPlaybackRange(CachedShotStates[SectionIndex].MovieSceneRange, false);
		}
		ShotSection->SetIsActive(CachedShotStates[SectionIndex].bActive);
		ShotSection->SetRange(CachedShotStates[SectionIndex].ShotRange);
		ShotSection->SetIsLocked(CachedShotStates[SectionIndex].bLocked);
	}
}

bool UAutomatedLevelSequenceCapture::SetupShot(FFrameNumber& StartTime, FFrameNumber& EndTime)
{
	if (Settings.HandleFrames <= 0)
	{
		return false;
	}

	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return false;
	}

	UMovieSceneCinematicShotTrack* CinematicShotTrack = GetCinematicShotTrack(LevelSequenceActor);
	if (!CinematicShotTrack)
	{
		return false;
	}

	if (ShotIndex > CinematicShotTrack->GetAllSections().Num()-1)
	{
		return false;
	}

	// Only render shots that are active
	for (; ShotIndex < CinematicShotTrack->GetAllSections().Num(); )
	{
		if (CachedShotStates[ShotIndex].bActive)
		{
			break;
		}

		++ShotIndex;
	}

	// Disable all shots unless it's the current one being rendered
	for (int32 SectionIndex = 0; SectionIndex < CinematicShotTrack->GetAllSections().Num(); ++SectionIndex)
	{
		UMovieSceneSection* ShotSection = CinematicShotTrack->GetAllSections()[SectionIndex];

		ShotSection->SetIsActive(SectionIndex == ShotIndex);
		ShotSection->MarkAsChanged();

		if (SectionIndex == ShotIndex)
		{
			// We intersect with the CachedPlaybackRange instead of copying the playback range from the shot to handle the case where
			// the playback range intersected the middle of the shot before we started manipulating ranges. We manually expand the root
			// Movie Sequence's playback range by the number of handle frames to allow handle frames to work as expected on first/last shot.
			FFrameNumber HandleFramesResolutionSpace = ConvertFrameTime(Settings.HandleFrames, Settings.GetFrameRate(), MovieScene->GetTickResolution()).FloorToFrame();
			TRange<FFrameNumber> ExtendedCachedPlaybackRange = UE::MovieScene::ExpandRange(CachedPlaybackRange, HandleFramesResolutionSpace);

			TRange<FFrameNumber> TotalRange = TRange<FFrameNumber>::Intersection(ShotSection->GetRange(), ExtendedCachedPlaybackRange);

			StartTime = TotalRange.IsEmpty() ? FFrameNumber(0) : UE::MovieScene::DiscreteInclusiveLower(TotalRange);
			EndTime   = TotalRange.IsEmpty() ? FFrameNumber(0) : UE::MovieScene::DiscreteExclusiveUpper(TotalRange);

			MovieScene->SetPlaybackRange(StartTime, (EndTime - StartTime).Value, false);
			MovieScene->MarkAsChanged();
		}
	}

	return true;
}

void UAutomatedLevelSequenceCapture::SetupFrameRange()
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();
	if( Actor )
	{
		ULevelSequence* LevelSequence = Actor->GetSequence();
		if( LevelSequence != nullptr )
		{
			UMovieScene* MovieScene = LevelSequence->GetMovieScene();
			if( MovieScene != nullptr )
			{
				FFrameRate           SourceFrameRate = MovieScene->GetTickResolution();
				TRange<FFrameNumber> SequenceRange   = MovieScene->GetPlaybackRange();

				FFrameNumber PlaybackStartFrame = ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(SequenceRange), SourceFrameRate, Settings.GetFrameRate()).CeilToFrame();
				FFrameNumber PlaybackEndFrame   = ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(SequenceRange), SourceFrameRate, Settings.GetFrameRate()).CeilToFrame();

				if( bUseCustomStartFrame )
				{
					PlaybackStartFrame = CustomStartFrame;
				}

				if( !Settings.bUseRelativeFrameNumbers )
				{
				 	// NOTE: The frame number will be an offset from the first frame that we start capturing on, not the frame
				 	// that we start playback at (in the case of WarmUpFrameCount being non-zero).  So we'll cache out frame
				 	// number offset before adjusting for the warm up frames.
				 	this->FrameNumberOffset = PlaybackStartFrame.Value;
				}

				if( bUseCustomEndFrame )
				{
				 	PlaybackEndFrame = CustomEndFrame;
				}

				// This is a fun hack... Due to the fragility of this code (which this makes more fragile admittedly...) the original audio implementation
				// just ran the entire process twice, starting all the way back at the Setup loop so that everything would be re-initialized like it was
				// for the video capture. Unfortunately, it looks like when we switch from a Fixed Timestep clock to the Platform clock (needed for audio
				// as audio is realtime only) this allows the sequence player to get out of sync with the warmup frame counter. Audio recording doesn't start
				// until the warmup time has passed, but because the sequence is playing at realtime it can start playing the part of the sequence you wanted
				// the audio for before the recorder ever kicks in!
				// To minimize changes, we're just going to override any warmup/delay times to zero here in the event that this is an audio pass, so that
				// they should be more in sync.
				if (bIsAudioCapturePass)
				{
					WarmUpFrameCount = 0;
					DelayBeforeWarmUp = 0.f;
					DelayBeforeShotWarmUp = 0.f; 
				}

				RemainingWarmUpFrames = FMath::Max( WarmUpFrameCount, 0 );
				if( RemainingWarmUpFrames > 0 )
				{
				 	// We were asked to playback additional frames before we start capturing
				 	PlaybackStartFrame -= RemainingWarmUpFrames;
				}

				if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = FindMotionVectorSimulation(Actor))
				{
					MotionVectorSim->PreserveSimulatedMotion(true);
				}

				// Override the movie scene's playback range
				Actor->GetSequencePlayer()->SetFrameRate(Settings.GetFrameRate());
				Actor->GetSequencePlayer()->SetFrameRange(PlaybackStartFrame.Value, (PlaybackEndFrame - PlaybackStartFrame).Value);
				Actor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(FFrameTime(PlaybackStartFrame), EUpdatePositionMethod::Jump));

				Actor->GetSequencePlayer()->SetSnapshotOffsetFrames(WarmUpFrameCount);
			}
		}
	}
}

void UAutomatedLevelSequenceCapture::EnableCinematicMode()
{
	if (!GetSettings().bCinematicMode)
	{
		return;
	}

	// iterate through the controller list and set cinematic mode if necessary
	bool bNeedsCinematicMode = !GetSettings().bAllowMovement || !GetSettings().bAllowTurning || !GetSettings().bShowPlayer || !GetSettings().bShowHUD;
	if (!bNeedsCinematicMode)
	{
		return;
	}

	if (Viewport.IsValid())
	{
		for (FConstPlayerControllerIterator Iterator = Viewport.Pin()->GetClient()->GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (PC && PC->IsLocalController())
			{
				PC->SetCinematicMode(true, !GetSettings().bShowPlayer, !GetSettings().bShowHUD, !GetSettings().bAllowMovement, !GetSettings().bAllowTurning);
			}
		}
	}
}

void UAutomatedLevelSequenceCapture::OnTick(float DeltaSeconds)
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();

	if (!Actor || !Actor->GetSequencePlayer())
	{
		return;
	}

	// Block till level streaming is completed. This would cause hitches under normal gameplay, but because we already run at slower-than-real-time
	// it doesn't matter for movie captures. This solves situations where games have systems that pull in sublevels via level streaming that cannot
	// be normally controlled via the Sequencer Level Visibility track. If all levels are already loaded, blocking will have no effect.
	if (Actor->GetWorld())
	{
		Actor->GetWorld()->BlockTillLevelStreamingCompleted();
	}

	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Waiting for %d shaders to finish compiling...", GFrameCounter, GShaderCompilingManager->GetNumRemainingJobs());
		GShaderCompilingManager->FinishAllCompilation();
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Done waiting for shaders to finish.", GFrameCounter);
	}

	if (GDistanceFieldAsyncQueue && GDistanceFieldAsyncQueue->GetNumOutstandingTasks() > 0)
	{
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Waiting for %d Mesh Distance Fields to finish building...", GFrameCounter, GDistanceFieldAsyncQueue->GetNumOutstandingTasks());
		GDistanceFieldAsyncQueue->BlockUntilAllBuildsComplete();
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Done waiting for Mesh Distance Fields to build.", GFrameCounter);
	}

	if (GCardRepresentationAsyncQueue && GCardRepresentationAsyncQueue->GetNumOutstandingTasks() > 0)
	{
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Waiting for %d Mesh Cards to finish building...", GFrameCounter, GCardRepresentationAsyncQueue->GetNumOutstandingTasks());
		GCardRepresentationAsyncQueue->BlockUntilAllBuildsComplete();
		UE_LOGF(LogMovieSceneCapture, Log, "[%lld] Done waiting for Mesh Cards to build.", GFrameCounter);
	}

	// Setup the automated capture
	if (CaptureState == ELevelSequenceCaptureState::Setup)
	{
		SetupFrameRange();

		EnableCinematicMode();
		
		// Bind to the event so we know when to capture a frame
		if (!bIsAudioCapturePass)
		{
			OnPlayerUpdatedBinding = Actor->GetSequencePlayer()->OnSequenceUpdated().AddUObject( this, &UAutomatedLevelSequenceCapture::SequenceUpdated );
		}

		StartWarmup();

		if (DelayBeforeWarmUp + DelayBeforeShotWarmUp + DelayEveryFrame > 0)
		{
			CaptureState = ELevelSequenceCaptureState::DelayBeforeWarmUp;

			Actor->GetWorld()->GetTimerManager().SetTimer(DelayTimer, FTimerDelegate::CreateUObject(this, &UAutomatedLevelSequenceCapture::DelayBeforeWarmupFinished), DelayBeforeWarmUp + DelayBeforeShotWarmUp + DelayEveryFrame, false);
		}
		else
		{
			DelayBeforeWarmupFinished();
		}
	}

	// Then we'll just wait a little bit.  We'll delay the specified number of seconds before capturing to allow any
	// textures to stream in or post processing effects to settle.
	if( CaptureState == ELevelSequenceCaptureState::DelayBeforeWarmUp )
	{
		// Do nothing, just hold at the current frame. This assumes that the current frame isn't changing by any other mechanisms.
	}
	else if( CaptureState == ELevelSequenceCaptureState::ReadyToWarmUp )
	{
		Actor->GetSequencePlayer()->Play();
		// Start warming up
		CaptureState = ELevelSequenceCaptureState::WarmingUp;
	}

	// Count down our warm up frames.
	if( CaptureState == ELevelSequenceCaptureState::WarmingUp)
	{
		// The post increment is important - it ensures we capture the very first frame if there are no warm up frames,
		// but correctly skip n frames if there are n warmup frames
		if (RemainingWarmUpFrames-- == 0)
		{
			// Start capturing - this will capture the *next* update from sequencer
			CaptureState = ELevelSequenceCaptureState::FinishedWarmUp;
			UpdateFrameState();
			StartCapture();
		}
		else
		{
			// Move onto the next frame
			TimeController->DeltaTime = FFrameTime(1);
		}
	}

	if( bCapturing && !Actor->GetSequencePlayer()->IsPlaying() && CaptureState != ELevelSequenceCaptureState::Paused )
	{
		++ShotIndex;

		FFrameNumber StartTime, EndTime;
		if (SetupShot(StartTime, EndTime))
		{
			UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);

			FFrameNumber StartTimePlayRateSpace = ConvertFrameTime(StartTime, MovieScene->GetTickResolution(), Settings.GetFrameRate()).CeilToFrame();
			FFrameNumber EndTimePlayRateSpace   = ConvertFrameTime(EndTime,   MovieScene->GetTickResolution(), Settings.GetFrameRate()).CeilToFrame();

			Actor->GetSequencePlayer()->SetFrameRange(StartTimePlayRateSpace.Value, (EndTimePlayRateSpace - StartTimePlayRateSpace).Value);
			Actor->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(FFrameTime(StartTimePlayRateSpace), EUpdatePositionMethod::Jump));
			Actor->GetSequencePlayer()->Play();

			// We need to re-register to the binding when we start each shot. When a shot reaches the last frame it unregisters the binding so that
			// any subsequent seeking doesn't accidentally render extra frames. SetupShot doesn't get called until after the first time we finish
			// rendering a shot so this doesn't register the delegate twice on the first go.
			OnPlayerUpdatedBinding = Actor->GetSequencePlayer()->OnSequenceUpdated().AddUObject(this, &UAutomatedLevelSequenceCapture::SequenceUpdated);

			CaptureState = ELevelSequenceCaptureState::FinishedWarmUp;
			UpdateFrameState();
		}
		else
		{
			// This is called when the sequence finishes playing and we've reached the end of all shots within the sequence.
			// We only render the audio pass if they have specified an audio capture protocol, so we allow this early out 
			// when there is no audio, or when we have finished the audio pass.
			if (IsAudioPassIfNeeded() && CaptureState != ELevelSequenceCaptureState::Setup)
			{
				// If they don't want to render audio, or they have rendered an audio pass, we finish and finalize the data.
				Actor->GetSequencePlayer()->OnSequenceUpdated().Remove( OnPlayerUpdatedBinding );
				FinalizeWhenReady();

				// Restore our cached variables since these are the actual one represented in the in-engine UI
				WarmUpFrameCount = CachedWarmUpFrameCount;
				DelayBeforeWarmUp = CachedDelayBeforeWarmUp;
				DelayBeforeShotWarmUp = CachedDelayBeforeShotWarmUp;
			}
			else
			{
				// Reset us to use the platform clock for controlling the playback rate of the sequence. The audio system
				// uses the platform clock for timings as well.
				Actor->GetSequencePlayer()->SetTimeController(MakeShared<FMovieSceneTimeController_PlatformClock>());
				CaptureState = ELevelSequenceCaptureState::Setup;
				
				// We'll now repeat the whole process including warmups and delays. The audio capture will pause recording while we are delayed.
				// This creates an audio discrepancy during the transition point (if there is shot warmup) but it allows a complex scenes to spend
				// enough time loading that it doesn't cause an audio desync.
				bIsAudioCapturePass = true;
				bCapturing = false;
			}
		}
	}
}

void UAutomatedLevelSequenceCapture::DelayBeforeWarmupFinished()
{
	// Wait a frame to go by after we've set the fixed time step, so that the animation starts
	// playback at a consistent time
	CaptureState = ELevelSequenceCaptureState::ReadyToWarmUp;
}

void UAutomatedLevelSequenceCapture::PauseFinished()
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();
	Actor->GetSequencePlayer()->Play();

	CaptureState = ELevelSequenceCaptureState::FinishedWarmUp;

	if (bIsAudioCapturePass)
	{
		UE_LOGF(LogMovieSceneCapture, Log, "WarmUp pause finished. Resuming the capture of audio.");
	}
	else
	{
		UE_LOGF(LogMovieSceneCapture, Log, "WarmUp pause finished. Resuming the capture of images.");
	}
}

void UAutomatedLevelSequenceCapture::SequenceUpdated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime)
{
	if (bCapturing)
	{
		FLevelSequencePlayerSnapshot PreviousState = CachedState;

		UpdateFrameState();

		ALevelSequenceActor* Actor = LevelSequenceActor.Get();
		if (Actor && Actor->GetSequencePlayer())
		{
			// If this is a new shot, set the state to shot warm up and pause on this frame until warmed up			
			const bool bHasMultipleShots = PreviousState.CurrentShotName != PreviousState.RootName;
			const bool bNewShot = bHasMultipleShots && PreviousState.ShotID != CachedState.ShotID;
			const bool bNewFrame = PreviousTime != CurrentTime;

			const bool bDelayingBeforeShotWarmUp = (bNewShot && DelayBeforeShotWarmUp > 0);
			const bool bDelayingEveryFrame = (bNewFrame && DelayEveryFrame > 0);

			if (Actor->GetSequencePlayer()->IsPlaying() && ( bDelayingBeforeShotWarmUp || bDelayingEveryFrame ))
			{
				if (bIsAudioCapturePass)
				{
					UE_LOGF(LogMovieSceneCapture, Log, "Entering WarmUp pause, pausing audio capture.");
					if (AudioCaptureProtocol)
					{
						AudioCaptureProtocol->WarmUp();
					}
				}
				else
				{
					UE_LOGF(LogMovieSceneCapture, Log, "Entering WarmUp pause, pausing image capture.");
					if (ImageCaptureProtocol)
					{
						ImageCaptureProtocol->WarmUp();
					}
				}
				
				CaptureState = ELevelSequenceCaptureState::Paused;

				if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = FindMotionVectorSimulation(Actor))
				{
					MotionVectorSim->PreserveSimulatedMotion(true);
				}

				Actor->GetWorld()->GetTimerManager().SetTimer(DelayTimer, FTimerDelegate::CreateUObject(this, &UAutomatedLevelSequenceCapture::PauseFinished), DelayBeforeShotWarmUp + DelayEveryFrame, false);
				Actor->GetSequencePlayer()->Pause();
			}
			else if (CaptureState == ELevelSequenceCaptureState::FinishedWarmUp)
			{
				// If we were preserving simulated motion, now's the time to stop that since we've captured the frame that was being simulated
				if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = FindMotionVectorSimulation(Actor))
				{
					MotionVectorSim->PreserveSimulatedMotion(false);
				}

				// These are called each frame to allow the state machine inside the protocol to transition back to capturing
				// after paused if needed. This is needed for things like the avi writer who spin up an avi writer per shot (if needed)
				// so that we can capture the movies into individual avi files per shot due to the format text.
				if (bIsAudioCapturePass)
				{
					if (AudioCaptureProtocol)
					{
						AudioCaptureProtocol->StartCapture();
					}
				}
				else
				{
					if (ImageCaptureProtocol)
					{
						ImageCaptureProtocol->StartCapture();
					}
				}

				bool bOnLastFrame = (CurrentTime.FrameNumber >= Actor->GetSequencePlayer()->GetStartTime().Time.FrameNumber + Actor->GetSequencePlayer()->GetFrameDuration() - 1);
				bool bLastShot = NumShots == 0 ? true : ShotIndex == NumShots - 1;

				CaptureThisFrame((CurrentTime - PreviousTime) / Settings.GetFrameRate());
				CachedMetrics.PreviousFrame = CurrentTime.FrameNumber.Value;

				// Our callback can be called multiple times for a given frame due to how Level Sequences evaluate.
				// For example, frame 161 is evaluated and an image is written. This isn't considered the end of the sequence
				// as technically the Level Sequence can be evaluated up to 161.9999994, so on the next Update loop it tries to
				// evaluate frame 162 (due to our fixed timestep controller). This then puts it over the limit so it forces a 
				// reevaluation of 161 before calling Stop/Pause. This then invokes this callback a second time for frame 161
				// and we end up with two instances of 161! To solve this, when we reach the last frame of each shot we stop listening
				// to updates. If there's a new shot it will re-register the delegate once it is set up.
				if (bOnLastFrame)
				{
					if (bLastShot && IsAudioPassIfNeeded())
					{
						FinalizeWhenReady();
					}
					Actor->GetSequencePlayer()->OnSequenceUpdated().Remove(OnPlayerUpdatedBinding);
				}


				// Move onto the next frame now that we've captured this one
				TimeController->DeltaTime = FFrameTime(1);
			}
		}
	}
}

void UAutomatedLevelSequenceCapture::UpdateFrameState()
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();

	if (Actor && Actor->GetSequencePlayer())
	{
		Actor->GetSequencePlayer()->TakeFrameSnapshot(CachedState);
	}
}

void UAutomatedLevelSequenceCapture::LoadFromConfig()
{
	UMovieSceneCapture::LoadFromConfig();

	BurnInOptions->LoadConfig();
	BurnInOptions->ResetSettings();
	if (BurnInOptions->Settings)
	{
		BurnInOptions->Settings->LoadConfig();
	}
}

void UAutomatedLevelSequenceCapture::SaveToConfig()
{
	FFrameNumber CurrentStartFrame = CustomStartFrame;
	FFrameNumber CurrentEndFrame = CustomEndFrame;
	bool bRestoreFrameOverrides = RestoreFrameOverrides();

	BurnInOptions->SaveConfig();
	if (BurnInOptions->Settings)
	{
		BurnInOptions->Settings->SaveConfig();
	}

	UMovieSceneCapture::SaveToConfig();

	if (bRestoreFrameOverrides)
	{
		SetFrameOverrides(CurrentStartFrame, CurrentEndFrame);
	}
}

void UAutomatedLevelSequenceCapture::Close()
{
	Super::Close();
	CachedState = FLevelSequencePlayerSnapshot();
	RestoreShots();
}

bool UAutomatedLevelSequenceCapture::RestoreFrameOverrides()
{
	bool bAnySet = CachedStartFrame.IsSet() || CachedEndFrame.IsSet() || bCachedUseCustomStartFrame.IsSet() || bCachedUseCustomEndFrame.IsSet();
	if (CachedStartFrame.IsSet())
	{
		CustomStartFrame = CachedStartFrame.GetValue();
		CachedStartFrame.Reset();
	}

	if (CachedEndFrame.IsSet())
	{
		CustomEndFrame = CachedEndFrame.GetValue();
		CachedEndFrame.Reset();
	}

	if (bCachedUseCustomStartFrame.IsSet())
	{
		bUseCustomStartFrame = bCachedUseCustomStartFrame.GetValue();
		bCachedUseCustomStartFrame.Reset();
	}

	if (bCachedUseCustomEndFrame.IsSet())
	{
		bUseCustomEndFrame = bCachedUseCustomEndFrame.GetValue();
		bCachedUseCustomEndFrame.Reset();
	}

	return bAnySet;
}

void UAutomatedLevelSequenceCapture::SetFrameOverrides(FFrameNumber InStartFrame, FFrameNumber InEndFrame)
{
	CachedStartFrame = CustomStartFrame;
	CachedEndFrame = CustomEndFrame;
	bCachedUseCustomStartFrame = bUseCustomStartFrame;
	bCachedUseCustomEndFrame = bUseCustomEndFrame;

	CustomStartFrame = InStartFrame;
	CustomEndFrame = InEndFrame;
	bUseCustomStartFrame = true;
	bUseCustomEndFrame = true;
}

void UAutomatedLevelSequenceCapture::SerializeAdditionalJson(FJsonObject& Object)
{
	TSharedRef<FJsonObject> OptionsContainer = MakeShareable(new FJsonObject);
	if (FJsonObjectConverter::UStructToJsonObject(BurnInOptions->GetClass(), BurnInOptions, OptionsContainer, 0, 0))
	{
		Object.SetField(TEXT("BurnInOptions"), MakeShareable(new FJsonValueObject(OptionsContainer)));
	}

	if (BurnInOptions->Settings)
	{
		TSharedRef<FJsonObject> SettingsDataObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(BurnInOptions->Settings->GetClass(), BurnInOptions->Settings, SettingsDataObject, 0, 0))
		{
			Object.SetField(TEXT("BurnInOptionsInitSettings"), MakeShareable(new FJsonValueObject(SettingsDataObject)));
		}
	}
}

void UAutomatedLevelSequenceCapture::DeserializeAdditionalJson(const FJsonObject& Object)
{
	if (!BurnInOptions)
	{
		BurnInOptions = NewObject<ULevelSequenceBurnInOptions>(this, "BurnInOptions");
	}

	TSharedPtr<FJsonValue> OptionsContainer = Object.TryGetField(TEXT("BurnInOptions"));
	if (OptionsContainer.IsValid())
	{
		FJsonObjectConverter::JsonAttributesToUStruct(OptionsContainer->AsObject()->Values, BurnInOptions->GetClass(), BurnInOptions, 0, 0);
	}

	BurnInOptions->ResetSettings();
	if (BurnInOptions->Settings)
	{
		TSharedPtr<FJsonValue> SettingsDataObject = Object.TryGetField(TEXT("BurnInOptionsInitSettings"));
		if (SettingsDataObject.IsValid())
		{
			FJsonObjectConverter::JsonAttributesToUStruct(SettingsDataObject->AsObject()->Values, BurnInOptions->Settings->GetClass(), BurnInOptions->Settings, 0, 0);
		}
	}
}

void UAutomatedLevelSequenceCapture::ExportEDL()
{
	if (!bWriteEditDecisionList)
	{
		return;
	}
	
	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return;
	}

	UMovieSceneCinematicShotTrack* ShotTrack = MovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (!ShotTrack)
	{
		return;
	}

	FString SaveFilename = 	Settings.OutputDirectory.Path / MovieScene->GetOuter()->GetName();
	int32 HandleFrames = Settings.HandleFrames;
	FString MovieExtension = Settings.MovieExtension;

	MovieSceneTranslatorEDL::ExportEDL(MovieScene, Settings.GetFrameRate(), SaveFilename, HandleFrames, MovieExtension);
}

double UAutomatedLevelSequenceCapture::GetEstimatedCaptureDurationSeconds() const
{
	ALevelSequenceActor* Actor = LevelSequenceActor.Get();

	if (Actor)
	{
		TRange<FFrameNumber> PlaybackRange = Actor->GetSequence()->GetMovieScene()->GetPlaybackRange();
		int32 MovieSceneDurationFrameCount = UE::MovieScene::DiscreteSize(PlaybackRange);
		
		return Actor->GetSequence()->GetMovieScene()->GetTickResolution().AsSeconds(FFrameTime(FFrameNumber(MovieSceneDurationFrameCount)));
	}

	return 0.0;
}

void UAutomatedLevelSequenceCapture::ExportFCPXML()
{
	if (!bWriteFinalCutProXML)
	{
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(LevelSequenceActor);
	if (!MovieScene)
	{
		return;
	}

	UMovieSceneCinematicShotTrack* ShotTrack = MovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (!ShotTrack)
	{
		return;
	}

	FString SaveFilename = Settings.OutputDirectory.Path / MovieScene->GetOuter()->GetName() + TEXT(".xml");
	FString FilenameFormat = Settings.OutputFormat;
	int32 HandleFrames = Settings.HandleFrames;
	FFrameRate FrameRate = Settings.GetFrameRate();
	uint32 ResX = Settings.Resolution.ResX;
	uint32 ResY = Settings.Resolution.ResY;
	FString MovieExtension = Settings.MovieExtension;

	FFCPXMLExporter *Exporter = new FFCPXMLExporter;

	TSharedRef<FMovieSceneTranslatorContext> ExportContext(new FMovieSceneTranslatorContext);
	ExportContext->Init();

	bool bSuccess = Exporter->Export(MovieScene, FilenameFormat, FrameRate, ResX, ResY, HandleFrames, SaveFilename, ExportContext, MovieExtension);

	// Log any messages in context
	MovieSceneToolHelpers::MovieSceneTranslatorLogMessages(Exporter, ExportContext, false);

	delete Exporter;
}


#endif
