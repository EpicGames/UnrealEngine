// Copyright Epic Games, Inc. All Rights Reserved.

#include "Baker/AnimSequenceBaker.h"
#include "ConstraintsManager.h"
#include "ConstraintChannel.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "GameFramework/Actor.h"
#include "IMovieSceneLinkedAnimTrackProvider.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "Misc/FrameRate.h"
#include "Components/SkeletalMeshComponent.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"


int32 UE::AIE::FBakeTimeRange::CalculateIndex(const FFrameNumber& InCurrentFrame) const
{
	if (InCurrentFrame > StartFrame && InCurrentFrame <= EndFrame)
	{
		return ((InCurrentFrame.Value - StartFrame.Value) / (FrameStep.Value));
	}
	else if (InCurrentFrame <= StartFrame)
	{
		return 0;
	}
	else if (InCurrentFrame > EndFrame)
	{
		return NumFrames - 1;
	}
	return INDEX_NONE;
}

namespace UE::Sequencer
{ 
	

TSharedPtr<ISequencerBakeRecorder> FAnimSequenceBakeRecorder::CreateRecorderAndAddToBaker(TSharedPtr<FSequencerBaker>& InBaker, FGuid InBindingID, UAnimSequence* InAnimSequence,
	FLevelSequenceAnimSequenceLinkItem& InItem)
{

	TSharedPtr<ISequencerBakeRecorder> Recorder = MakeShared<UE::Sequencer::FAnimSequenceBakeRecorder>(InBaker, InBindingID, InAnimSequence, InItem);

	UE::AIE::ISequencerBaker::FRecorderOptions RecorderOptions;
	RecorderOptions.WarmupFrames = InItem.WarmUpFrames;
	RecorderOptions.DelayBeforeStart = InItem.DelayBeforeStart;

	InBaker->AddRecorder(Recorder, RecorderOptions);
	return Recorder;
}

uint32 FAnimSequenceBakeRecorder::GetHash()
{
	return  GetTypeHash(BindingID) + GetTypeHash(WeakAnimSequence);
}

void FAnimSequenceBakeRecorder::BakeStarted(const UE::AIE::FBakeTimeRange& InRange)
{
	bProviderHandledBake = false;

	if (TSharedPtr<FSequencerBaker> Baker = WeakBaker.Pin())
	{
		// Give a linked-anim provider the chance to perform the bake itself when it
		// owns a richer evaluation model than skeletal-mesh-component sampling.
		if (UAnimSequence* AnimSequence = WeakAnimSequence.Get())
		{
			if (IMovieSceneLinkedAnimTrackProvider* Provider = FCommonAnimationTrackEditor::FindProviderForBinding(Baker->GetWeakSequencer(), BindingID))
			{
				if (Provider->TryBakeLinkedAnimSequence(Baker->GetWeakSequencer(), BindingID, AnimSequence, LinkItem))
				{
					bProviderHandledBake = true;
				}
			}
		}

		if (bProviderHandledBake)
		{
			return;
		}

		if (LinkItem.bUseCustomTimeRange)
		{
			if (ISequencer* Sequencer = Baker->GetWeakSequencer().Pin().Get())
			{
				TRange<FFrameNumber> NewRange{ 0 };

				const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
				const FFrameRate DisplayResolution = Sequencer->GetFocusedDisplayRate();
				const FFrameNumber StartFrameInTick = FFrameRate::TransformTime(FFrameTime(LinkItem.CustomStartFrame), DisplayResolution, TickResolution).FloorToFrame();
				FFrameNumber EndFrameInTick = FFrameRate::TransformTime(FFrameTime(LinkItem.CustomEndFrame), DisplayResolution, TickResolution).CeilToFrame();
				if (EndFrameInTick < StartFrameInTick)
				{
					EndFrameInTick = StartFrameInTick;
				}
				StartIndex = InRange.CalculateIndex(StartFrameInTick);
				EndIndex = InRange.CalculateIndex(EndFrameInTick);
			}
			else
			{
				StartIndex = 0;
				EndIndex = InRange.CalculateIndex(InRange.EndFrame);
			}
		}
		else
		{
			StartIndex = 0;
			EndIndex = InRange.CalculateIndex(InRange.EndFrame);
		}

		InitRecorder(InRange);

		if (ISequencer* Sequencer = Baker->GetWeakSequencer().Pin().Get())
		{
			if (USkeletalMeshComponent* SkelMeshComp = WeakComponent.Get())
			{
				TArray< USkeletalMeshComponent*> SkelMeshComps;
				if (LinkItem.bEvaluateAllSkeletalMeshComponents)
				{
					AActor* Actor = SkelMeshComp->GetTypedOuter<AActor>();
					if (Actor)
					{
						Actor->GetComponents(SkelMeshComps, false);
					}
				}
				else
				{
					SkelMeshComps.Add(SkelMeshComp);
				}
				Baker->AddSkelMeshCompsToTick(SkelMeshComps);
			}
		}
	}
}

void FAnimSequenceBakeRecorder::InitRecorder(const UE::AIE::FBakeTimeRange& InRange)
{
	if (BindingID.IsValid() && WeakAnimSequence.Get() && WeakBaker.IsValid())
	{
		WeakComponent = FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(BindingID, WeakBaker.Pin()->GetWeakSequencer().Pin());
		if (WeakComponent.IsValid())
		{

			FAnimationRecordingSettings RecordingSettings;
			RecordingSettings.SampleFrameRate = InRange.DisplayRate;
			RecordingSettings.Interpolation = LinkItem.Interpolation;
			RecordingSettings.InterpMode = LinkItem.CurveInterpolation;
			if (LinkItem.CurveInterpolation == ERichCurveInterpMode::RCIM_Constant ||
				LinkItem.CurveInterpolation == ERichCurveInterpMode::RCIM_None)
			{
				RecordingSettings.TangentMode = ERichCurveTangentMode::RCTM_None;
			}
			else
			{
				RecordingSettings.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			RecordingSettings.Length = 0;
			RecordingSettings.bRemoveRootAnimation = false;
			RecordingSettings.bCheckDeltaTimeAtBeginning = false;
			RecordingSettings.bRecordInWorldSpace = false;
			RecordingSettings.bTransactRecording = false;
			RecordingSettings.bSetRetargetSourceAsset = LinkItem.bSetRetargetSourceAsset;
			RecordingSettings.bRecordTransforms = LinkItem.bExportTransforms;
			RecordingSettings.bRecordMorphTargets = LinkItem.bExportMorphTargets;
			RecordingSettings.bRecordAttributeCurves = LinkItem.bExportAttributeCurves;
			RecordingSettings.bRecordMaterialCurves = LinkItem.bExportMaterialCurves;
			RecordingSettings.bSkipCurvesWithZeroValue = LinkItem.bSkipCurvesWithZeroValue;
			RecordingSettings.IncludeAnimationNames = LinkItem.IncludeAnimationNames;
			RecordingSettings.ExcludeAnimationNames = LinkItem.ExcludeAnimationNames;
			RecordingSettings.bRemoveExcludedCurves = LinkItem.bRemoveExcludedCurves;
			USkeletalMeshComponent* SkelMeshComp = WeakComponent.Get();
			UAnimSequence* AnimSequence = WeakAnimSequence.Get();
			AnimationRecorder.Init(SkelMeshComp, AnimSequence, nullptr, RecordingSettings);
		}
	}
}

void FAnimSequenceBakeRecorder::BakeFrame(const UE::AIE::FBakeTimeIndex& InIndex)
{
	if (bProviderHandledBake)
	{
		return;
	}

	if (InIndex.Index >= StartIndex && InIndex.Index <= EndIndex)
	{
		if (InIndex.Index == StartIndex)
		{
			AnimationRecorder.BeginRecording();
		}
		else
		{
			AnimationRecorder.Update(InIndex.DeltaTime);
		}
	}
}

void FAnimSequenceBakeRecorder::BakeCancelled()
{
	if (bProviderHandledBake)
	{
		return;
	}
	AnimationRecorder.FinishRecording();
}

void FAnimSequenceBakeRecorder::BakeFinished()
{
	if (!bProviderHandledBake)
	{
		AnimationRecorder.FinishRecording(true);
	}

	if (TSharedPtr<FSequencerBaker> Baker = WeakBaker.Pin())
	{
		if (TSharedPtr<ISequencer> SequencerPtr = Baker->GetWeakSequencer().Pin())
		{
			FCommonAnimationTrackEditor::UpdateLinkedAnimSectionRange(SequencerPtr, BindingID);
		}
	}
}

bool FAnimSequenceBakeRecorder::HasSequencerBinding(FGuid InGuid)
{
	return (InGuid == BindingID);
}

bool FAnimSequenceBakeRecorder::IsolateBakeResult(bool bIsolate) 
{
	bool bWasChanged = false;
	if (TSharedPtr<FSequencerBaker> Baker = WeakBaker.Pin())
	{
		if (TSharedPtr<ISequencer> SequencerPtr = Baker->GetWeakSequencer().Pin())
		{
			if (UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
			{
				if (FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingID))
				{
					if (FCommonAnimationTrackEditor::IsolateLinkedAnimTrack(SequencerPtr, BindingID, bIsolate))
					{
						bWasChanged = true;
					}
				}
			}
		}
	}
	return bWasChanged;
}

void FAnimSequenceBakeRecorder::SetRecordingEnabled(bool bInEnabled) 
{
	bRecordingEnabled = bInEnabled;
}
static FMovieSceneBinding GetBindingFromTrack(UMovieScene* InMovieScene, UMovieSceneTrack* Track)
{
	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)InMovieScene)->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		for (UMovieSceneTrack* PossibleTrack : Binding.GetTracks())
		{
			if (PossibleTrack == Track)
			{
				return Binding;
			}
		}
	}
	FMovieSceneBinding EmptyBinding;
	return EmptyBinding;
}

void FAnimSequenceBakeRecorder::AddDependencies(UMovieSceneTrack* InTrack, const TSharedPtr<ISequencer>& SequencerPtr)
{
	if(Dependencies.Contains(InTrack) == false)
	{ 
		Dependencies.Add(InTrack);
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			if (IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(Section))
			{
				for (FConstraintAndActiveChannel& ConstraintChannel : ConstrainedSection->GetConstraintsChannels())
				{
					if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
					{
						const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();
						ITransformConstraintChannelInterface* ParentInterface = InterfaceRegistry.FindConstraintChannelInterface(TransformConstraint->ParentTRSHandle->GetClass());
						if (ParentInterface)
						{
							if (UMovieSceneSection* ParentSection = ParentInterface->GetHandleSection(TransformConstraint->ParentTRSHandle, SequencerPtr->AsShared()))
							{
								if (UMovieSceneTrack* ParentTrack = ParentSection->GetTypedOuter<UMovieSceneTrack>())
								{
									if (UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
									{
										FMovieSceneBinding Binding = GetBindingFromTrack(MovieScene, ParentTrack);
										if (Binding.GetObjectGuid().IsValid())
										{
											for (UMovieSceneTrack* PossibleTrack : Binding.GetTracks())
											{
												if (PossibleTrack)
												{
													AddDependencies(PossibleTrack, SequencerPtr);
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

void FAnimSequenceBakeRecorder::ReadyToBake()
{
	if (TSharedPtr<FSequencerBaker> Baker = WeakBaker.Pin())
	{
		if (TSharedPtr<ISequencer> SequencerPtr = Baker->GetWeakSequencer().Pin())
		{
			if (UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
			{
				if (FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingID))
				{
					for (UMovieSceneTrack* PossibleTrack : Binding->GetTracks())
					{
						if (PossibleTrack)
						{
							AddDependencies(PossibleTrack, SequencerPtr);
						}
					}
				}
			}
		}
		TSharedPtr<ISequencerBakeRecorder> Shared = SharedThis(this);
		TArray<TWeakObjectPtr<UMovieSceneSignedObject>> DependenciesArray = Dependencies.Array();
		Baker->AddSignedObjectsToTrack(DependenciesArray, Shared);
	}
}

void FAnimSequenceBakeRecorder::RemovedFromBake() 
{
	if (TSharedPtr<FSequencerBaker> Baker = WeakBaker.Pin())
	{
		TSharedPtr<ISequencerBakeRecorder> Shared = SharedThis(this);
		TArray<TWeakObjectPtr<UMovieSceneSignedObject>> DependenciesArray = Dependencies.Array();
		Baker->RemoveSignedObjectsFromTrack(DependenciesArray, Shared);
	}
}

}

