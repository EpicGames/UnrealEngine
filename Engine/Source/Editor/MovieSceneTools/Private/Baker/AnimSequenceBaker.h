// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationRecorder.h"
#include "Animation/AnimSequence.h"
#include "Baker/ISequencerBaker.h"
#include "Baker/SequencerBaker.h"
#include "Components/SkeletalMeshComponent.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "Misc/Guid.h"

#define UE_API MOVIESCENETOOLS_API

namespace UE::Sequencer
{ 

/**
*  A Bake Recorder that creates linked anim sequences
*/

class FAnimSequenceBakeRecorder
	: public ISequencerBakeRecorder, public TSharedFromThis<FAnimSequenceBakeRecorder>
{
public:
	FAnimSequenceBakeRecorder()
		: WeakBaker(nullptr)
		, WeakComponent(nullptr)
		, WeakAnimSequence(nullptr) 
	{};
	FAnimSequenceBakeRecorder(TSharedPtr<FSequencerBaker>& InBaker, FGuid InBindingID, UAnimSequence* InAnimSequencer, 
		FLevelSequenceAnimSequenceLinkItem& InItem)
		: WeakBaker(InBaker)
		, BindingID(InBindingID)
		, WeakComponent(nullptr)
		, WeakAnimSequence(InAnimSequencer)
		, LinkItem(InItem)
	{};
	virtual ~FAnimSequenceBakeRecorder() {};

	//ISequencerBakeRecorder
	virtual void BakeStarted(const UE::AIE::FBakeTimeRange& InRange) override;
	virtual void BakeFrame(const UE::AIE::FBakeTimeIndex& InIndex) override;
	virtual void BakeCancelled() override;
	virtual void BakeFinished() override;
	virtual bool HasSequencerBinding(FGuid InGuid) override;
	virtual bool IsolateBakeResult(bool bIsolate) override;
	virtual void SetRecordingEnabled(bool bInEnabled) override;
	virtual bool GetRecordingEnabled() override
	{
		return bRecordingEnabled;
	}
	virtual uint32 GetHash() override;
	virtual void ReadyToBake() override;
	virtual void RemovedFromBake() override;

	UE_API static TSharedPtr<ISequencerBakeRecorder> CreateRecorderAndAddToBaker(TSharedPtr<FSequencerBaker>& InBaker, FGuid InBindingID, UAnimSequence* InAnimSequence,
		FLevelSequenceAnimSequenceLinkItem& InItem);
private:

	//may want to turn it off
	bool bRecordingEnabled = true;

	// True when a linked-anim track provider performed the bake itself
	// (e.g. the animation mixer composites root motion onto the root bone via its
	// dedicated bake helper). When true the recorder skips its per-frame skeletal
	// mesh sampling path so we don't overwrite the provider's output.
	bool bProviderHandledBake = false;

	TWeakPtr<FSequencerBaker> WeakBaker;
	//id in sequencer of skelmeshcomp we are baking
	FGuid BindingID; 
	TWeakObjectPtr<USkeletalMeshComponent> WeakComponent; 
	TWeakObjectPtr<UAnimSequence> WeakAnimSequence;

	//list of tracks we are dependent upon
	TSet<TWeakObjectPtr<UMovieSceneSignedObject>> Dependencies; 

	int32 StartIndex = INDEX_NONE;
	int32 EndIndex = INDEX_NONE;

	FLevelSequenceAnimSequenceLinkItem LinkItem; // containts baking options

	FAnimRecorderInstance AnimationRecorder;

private:
	void InitRecorder(const UE::AIE::FBakeTimeRange& InRange);
	//add dependencies recursively
	void AddDependencies(UMovieSceneTrack* InTrack, const TSharedPtr<ISequencer>& SequencerPtr);

};

}

#undef UE_API


