// Copyright Epic Games, Inc. All Rights Reserved.

#include "Baker/SequencerBaker.h"
#include "Baker/AnimSequenceBaker.h"
#include "ConstraintsManager.h"
#include "Editor.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"
#include "ISequencer.h"
#include "LevelEditorViewport.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolsModule.h"
#include "Transform/AnimationEvaluation.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"


bool UE::AIE::FBakeTimeRange::UpdateIfNeeded(const TRange<FFrameNumber>& InRange, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution)
{
	if (InRange != IncomingRange || InDisplayRate != DisplayRate || InTickResolution != TickResolution)
	{
		IncomingRange = InRange;
		TickResolution = InTickResolution;
		DisplayRate = InDisplayRate;
		FrameStep = FFrameRate::TransformTime(FFrameNumber(1), InDisplayRate,
			InTickResolution).RoundToFrame();
		if (FrameStep.Value <= 0)
		{
			FrameStep.Value = 1;
		}
		StartFrame = InRange.GetLowerBoundValue();
		EndFrame = InRange.GetUpperBoundValue();
		//make sure end frame falls on frame step 
		NumFrames = (EndFrame.Value - StartFrame.Value) / (FrameStep.Value) + 1;
		EndFrame.Value = StartFrame.Value + (NumFrames * FrameStep.Value) - 1;
		return true;
	}
	return false;
}

namespace UE::Sequencer
{

FSequencerBaker::FSequencerBaker() 
	: bNeedsBaking(false)
	, bIsBaking(false)
{	
}

UE::AIE::FBakeTimeRange FSequencerBaker::GetTimeRange()
{
	return BakeInterval.TimeRange;
}

bool FSequencerBaker::IsBakeRunning(TOptional<float>& OutPercentageDone)
{
	return bIsBaking && !BakeInterval.IsDoneCalculating();
}

void FSequencerBaker::CancelBake()
{
	BakeCancelled();
}

void FSequencerBaker::Initialize(TSharedPtr<ISequencer>& InSequencer)
{
	Release();
	if (InSequencer)
	{
		WeakSequencer = InSequencer;
		OnPreNavigateToSequenceChangedHandle = InSequencer->OnPreNavigateToSequence().AddRaw(this, &FSequencerBaker::OnPreNavigateToSequenceChanged);
		OnActivateSequenceChangedHandle = InSequencer->OnActivateSequence().AddRaw(this, &FSequencerBaker::OnActivateSequenceChanged);
		OnMovieSceneBindingsChangedHandle = InSequencer->OnMovieSceneBindingsChanged().AddRaw(this, &FSequencerBaker::OnMovieSceneBindingsChanged);
		BakeInterval.TimeRange = UE::AIE::FBakeTimeRange();
		BakeInterval.SetFromSequencer(InSequencer.Get());
		bNeedsBaking = true;
		OnActivateSequenceChanged(InSequencer->GetFocusedTemplateID());
	}
}

void FSequencerBaker::Release()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		BakeCancelled();
		if (OnPreNavigateToSequenceChangedHandle.IsValid())
		{
			Sequencer->OnPreNavigateToSequence().Remove(OnPreNavigateToSequenceChangedHandle);
			OnPreNavigateToSequenceChangedHandle.Reset();
		}
		if (OnActivateSequenceChangedHandle.IsValid())
		{
			Sequencer->OnActivateSequence().Remove(OnActivateSequenceChangedHandle);
			OnActivateSequenceChangedHandle.Reset();
		}
		if (OnMovieSceneBindingsChangedHandle.IsValid())
		{
			Sequencer->OnMovieSceneBindingsChanged().Remove(OnMovieSceneBindingsChangedHandle);
			OnMovieSceneBindingsChangedHandle.Reset();
		}
		for (TPair<uint32, TSharedPtr < ISequencerBakeRecorder>>& Pair : Recorders)
		{
			if (Pair.Value.IsValid())
			{
				Pair.Value.Reset();
			}
		}
		Recorders.Reset();
	}
	WeakSequencer.Reset();
}

void FSequencerBaker::OnMovieSceneBindingsChanged()
{
	//MZ todo, need to do what we did for anim layers and find bad bindings and replace them with good ones
}

void FSequencerBaker::OnPreNavigateToSequenceChanged()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (bIsBaking)
		{
			BakeFramesTillDone();
		}
		TArray<TSharedPtr<ISequencerBakeRecorder>> RecordersAsArray;
		Recorders.GenerateValueArray(RecordersAsArray);
		IsolateBakedTracks(RecordersAsArray, true);
	}
}

void FSequencerBaker::SetUpBakeRecorders()
{
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink && LevelAnimLink->AnimSequenceLinks.Num() > 0)
			{
				for (int32 Index = LevelAnimLink->AnimSequenceLinks.Num() - 1; Index >= 0; --Index)
				{
					FLevelSequenceAnimSequenceLinkItem& Item = LevelAnimLink->AnimSequenceLinks[Index];
					if (Item.bAutoBake)
					{
						UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
						if (AnimSequence == nullptr)
						{
							LevelAnimLink->AnimSequenceLinks.RemoveAt(Index);
							continue;
						}

						TSharedPtr<FSequencerBaker> Shared = SharedThis(this);
						FAnimSequenceBakeRecorder::CreateRecorderAndAddToBaker(Shared, Item.SkelTrackGuid, AnimSequence, Item);
					}
				}
			}
		}
	}
}

void FSequencerBaker::OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID) 
{
	if (!GIsTransacting)//is also called from undo from sequencer!!!!
	{
		if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			BakeInterval.TimeRange = UE::AIE::FBakeTimeRange();
			BakeInterval.SetFromSequencer(Sequencer.Get());
			BakeCancelled();
			BakingRecorders.Reset();
			GEditor->GetTimerManager()->SetTimerForNextTick([this, WeakThis = this->AsWeak()]()
			{
					if (WeakThis.IsValid())
					{
						if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
						{
							SetUpBakeRecorders();
						}
					}
			});
		}
	}
}

bool FSequencerBaker::CheckForChanges()
{
	bool bHasChange = false;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
		{
			bHasChange = LastValidMovieSceneGuid != Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
			LastValidMovieSceneGuid = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
		}
	}
	return bHasChange;
}

void FSequencerBaker::Tick(float DeltaTime)
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (BakeInterval.SetFromSequencer(Sequencer.Get()))
		{
			bNeedsBaking = true;
			AddAllRecordersToBake();
			return;
		}
	}

	if (CheckForChanges())
	{
		for (TPair<uint32, TSharedPtr < ISequencerBakeRecorder>>& Pair : Recorders)
		{
			if (Pair.Value.IsValid())
			{
				Pair.Value->ReadyToBake();
			}
		}
		TArray<TSharedPtr<ISequencerBakeRecorder>> ChangedRecorders = SignedObjectsToTrack.UpdateSignatures();
		if (ChangedRecorders.Num() > 0)
		{
			BakingRecorders.Append(ChangedRecorders);
			bNeedsBaking = true;
		}
	}
}

void FSequencerBaker::BakeStarted()
{
	if (TSharedPtr<ISequencer> SequencePtr = WeakSequencer.Pin())
	{
		BakeInterval.SetFromSequencer(SequencePtr.Get());

		//cache out important objects
		SkeletalMeshCompsTick.Reset();
		LastBakedRecorders.Reset();

		FMovieSceneSequenceTransform RootToLocalTransform = SequencePtr->GetFocusedMovieSceneSequenceTransform();
		LocalToRootTransform = RootToLocalTransform.Inverse();
		World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;

		if (bIsBaking)
		{
			BakeCancelled(); // cancel the bake if we were baking
		}

		if (BakingRecorders.Num() > 0)
		{
			bNeedsBaking = false;
			bIsBaking = true;
			if (SequencePtr->GetFocusedMovieSceneSequence() && SequencePtr->GetFocusedMovieSceneSequence()->GetMovieScene())
			{
				UMovieScene* MovieScene = SequencePtr->GetFocusedMovieSceneSequence()->GetMovieScene();
				for (TSharedPtr < ISequencerBakeRecorder>& Recorder : BakingRecorders)
				{
					if (Recorder.IsValid())
					{
						Recorder->BakeStarted(BakeInterval.TimeRange);
					}
				}
				for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
				{
					if (BakeHelper)
					{
						BakeHelper->StartBaking(MovieScene);
					}
				}
			}
			const float DeltaTime =  1.0 / BakeInterval.TimeRange.DisplayRate.AsDecimal();;
			//if we have warmup frames
			if (BakeInterval.WarmupFrames > 0)
			{
				for (int32 Index = BakeInterval.WarmupFrames.Value; Index > 0; --Index)
				{
					FFrameNumber FrameNumber = BakeInterval.TimeRange.StartFrame - FFrameNumber(Index);
					TickFrameInternal(FrameNumber, DeltaTime);
				}
			}
			//if we have delay frames run them first at the LocalStartFrame - ExportOptions->WarmupFrames;
			if (BakeInterval.DelayBeforeStart > 0)
			{
				for (int32 Index = 0; Index < BakeInterval.DelayBeforeStart.Value; ++Index)
				{
					TickFrameInternal(BakeInterval.TimeRange.StartFrame, DeltaTime);
				}
			}
		}
	}
	else
	{
		BakeCancelled();
	}
}

void FSequencerBaker::AddAllRecordersToBake()
{
	for (TPair<uint32, TSharedPtr < ISequencerBakeRecorder>>& Pair : Recorders)
	{
		if (Pair.Value.IsValid())
		{
			BakingRecorders.Add(Pair.Value);
		}
	}
}

void FSequencerBaker::BakeFramesTillDone()
{
	while (bIsBaking && BakingRecorders.Num() > 0)
	{
		int32 NextFrame = BakeInterval.NextFrameToCalculate();
		if (NextFrame != INDEX_NONE)
		{
			UE::AIE::FBakeTimeIndex BakeTimeIndex;
			BakeTimeIndex.Index = NextFrame;
			BakeTimeIndex.FrameNumber = BakeInterval.TimeRange.StartFrame + (BakeInterval.TimeRange.FrameStep * BakeTimeIndex.Index);
			BakeTimeIndex.DeltaTime = 1.0 / BakeInterval.TimeRange.DisplayRate.AsDecimal();
			TickFrameInternal(BakeTimeIndex.FrameNumber, BakeTimeIndex.DeltaTime);
			for (TSharedPtr < ISequencerBakeRecorder>& Recorder : BakingRecorders)
			{
				if (Recorder.IsValid())
				{
					Recorder->BakeFrame(BakeTimeIndex);
				}
			}
			BakeInterval.SetFrameCalculated(NextFrame);
		}
		else
		{
			BakeFinished(); //sets bIsBaking == false
		}
	}
}

void FSequencerBaker::BakeFullRange()
{
	if (TSharedPtr<ISequencer> SequencePtr = WeakSequencer.Pin())
	{
		if (bIsBaking)
		{
			CancelBake();
		}

		if (BakingRecorders.Num() == 0)
		{
			return;
		}

		BakeStarted();
		BakeFramesTillDone();

		BakingRecorders.Reset();
		SequencePtr->ForceEvaluate();
	}
}

bool FSequencerBaker::TickFrameInternal(const FFrameNumber& FrameNumber, float DeltaTime)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer && Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
	{

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		//Begin records a frame so need to set things up first
		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->PreEvaluation(MovieScene, FrameNumber);
			}
		}
		FFrameTime GlobalTime(FrameNumber);
		GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime); //player evals in root time so need to go back to it.

		FMovieSceneContext MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, BakeInterval.TimeRange.TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
		Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);

		if (World)
		{
			const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			Controller.EvaluateAllConstraints();
		}

		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->PostEvaluation(MovieScene, FrameNumber);
			}
		}
		//actors to tick? todo
		// 
		// Update space bases so new animation position has an effect.
		for (TWeakObjectPtr<USkeletalMeshComponent>& SkelMeshComp : SkeletalMeshCompsTick)
		{
			if (SkelMeshComp.IsValid())
			{
				//SkelMeshComp->UpdateLODStatus(); 

				SkelMeshComp->SetForcedLOD(1); //we need to do this every tick since a restore state may happen while evaluating and bummp us out ot a different LOD
				SkelMeshComp->TickAnimation(DeltaTime, false);

				SkelMeshComp->RefreshBoneTransforms();
				SkelMeshComp->RefreshFollowerComponents();
				SkelMeshComp->UpdateComponentToWorld();
				SkelMeshComp->FinalizeBoneTransform();
				SkelMeshComp->MarkRenderTransformDirty();
				SkelMeshComp->MarkRenderDynamicDataDirty();
			}
		}
		return true;
	}
	return false;
}

void FSequencerBaker::BakeFrame()
{
	if (bIsBaking && BakingRecorders.Num() > 0)
	{
		int32 NextFrame = BakeInterval.NextFrameToCalculate();
		if (NextFrame != INDEX_NONE)
		{
			UE::AIE::FBakeTimeIndex BakeTimeIndex;
			BakeTimeIndex.Index = NextFrame;
			BakeTimeIndex.FrameNumber = BakeInterval.TimeRange.StartFrame + (BakeInterval.TimeRange.FrameStep * BakeTimeIndex.Index);
			BakeTimeIndex.DeltaTime = 1.0 / BakeInterval.TimeRange.DisplayRate.AsDecimal();
			TickFrameInternal(BakeTimeIndex.FrameNumber, BakeTimeIndex.DeltaTime);
			for (TSharedPtr < ISequencerBakeRecorder>& Recorder : BakingRecorders)
			{
				if (Recorder.IsValid())
				{
					Recorder->BakeFrame(BakeTimeIndex);
				}
			}
			BakeInterval.SetFrameCalculated(NextFrame);
		}
		else
		{
			BakeFinished();
		}
	}
	else //check if baking
	{
		if (bIsBaking)
		{
			BakeCancelled();
		}
	}
}

void FSequencerBaker::BakeCancelled()
{
	if (bIsBaking)
	{
		bIsBaking = false;
		BakeInterval.Clear();
		for (TSharedPtr < ISequencerBakeRecorder>& Recorder : BakingRecorders)
		{
			if (Recorder.IsValid())
			{
				Recorder->BakeCancelled();
			}
		}
		SkeletalMeshCompsTick.Reset();
	}
}

void FSequencerBaker::BakeFinished()
{
	//if (bIsBaking && Recorders.Num() > 0)
	if (bIsBaking)
	{
		bIsBaking = false;
		bNeedsBaking = false;
		BakeInterval.Clear();
		for (TSharedPtr < ISequencerBakeRecorder>& Recorder : BakingRecorders)
		{
			if (Recorder.IsValid())
			{
				Recorder->BakeFinished();
				LastBakedRecorders.Add(Recorder);
			}
		}
		BakingRecorders.Reset();
		SkeletalMeshCompsTick.Reset();
	}
}

void FSequencerBaker::IsolateBakedTracks(TArray<TSharedPtr<ISequencerBakeRecorder>>& InRecorders, bool bIsolate)
{
	for (TSharedPtr < ISequencerBakeRecorder>& Recorder : InRecorders)
	{
		if (Recorder.IsValid())
		{
			Recorder->IsolateBakeResult(bIsolate);
		}
	}
}

void FSequencerBaker::AddSkelMeshCompsToTick(TArray<USkeletalMeshComponent*>& InSkelMeshCompArray)
{
	for (USkeletalMeshComponent* SkelmeshComp : InSkelMeshCompArray)
	{
		if (SkelmeshComp)
		{
			SkeletalMeshCompsTick.Add(SkelmeshComp);
		}
	}
}

void FSequencerBaker::IgnoreLastChange()
{
	SignedObjectsToTrack.UpdateSignatures();
}

void FSequencerBaker::AddSignedObjectsToTrack(TArray<TWeakObjectPtr<UMovieSceneSignedObject>>& InSignedObjects, TSharedPtr<ISequencerBakeRecorder>& InOwner)
{
	if (InOwner.IsValid())
	{
		for (TWeakObjectPtr< UMovieSceneSignedObject>& SignedObject : InSignedObjects)
		{
			if (SignedObject.IsValid())
			{
				SignedObjectsToTrack.AddObject(SignedObject.Get(), InOwner);
			}
		}
	}
}

void FSequencerBaker::RemoveSignedObjectsFromTrack(TArray<TWeakObjectPtr<UMovieSceneSignedObject>>& InSignedObjects, TSharedPtr<ISequencerBakeRecorder>& InOwner)
{
	if (InOwner.IsValid())
	{
		for (TWeakObjectPtr< UMovieSceneSignedObject>& SignedObject : InSignedObjects)
		{
			if (SignedObject.IsValid())
			{
				SignedObjectsToTrack.RemoveObject(SignedObject.Get(), InOwner);
			}
		}
	}
}

void FSequencerBaker::AddExtraRecordersToBake(const TArray<TSharedPtr<ISequencerBakeRecorder>>& InExtraRecorders)
{
	for (const TSharedPtr< ISequencerBakeRecorder>& Extra : InExtraRecorders)
	{
		if (Extra.IsValid())
		{
			BakingRecorders.Add(Extra);
			bNeedsBaking = true;
		}
	}
}

bool FSequencerBaker::AddRecorderToBake(TSharedPtr<ISequencerBakeRecorder>& InRecorder)
{
	if (Recorders.Contains(InRecorder->GetHash()) && BakingRecorders.Contains(InRecorder) == false)
	{
		BakingRecorders.Add(InRecorder);
		bNeedsBaking = true;
		return true;
	}
	return false;
}

void FSequencerBaker::AddRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder,const UE::AIE::ISequencerBaker::FRecorderOptions& Options)
{
	if (InRecorder.IsValid())
	{
		Recorders.Add(InRecorder->GetHash(),InRecorder);
		InRecorder->ReadyToBake();
		//use largest specified delay and warmup
		if (Options.DelayBeforeStart > BakeInterval.DelayBeforeStart)
		{
			BakeInterval.DelayBeforeStart = Options.DelayBeforeStart;
		}
		if (Options.WarmupFrames > BakeInterval.WarmupFrames)
		{
			BakeInterval.WarmupFrames = Options.WarmupFrames;
		}
		OnAddBakeRecorder.Broadcast(InRecorder.Get());
	}
}

void FSequencerBaker::RemoveRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder)
{
	if (InRecorder.IsValid())
	{
		Recorders.Remove(InRecorder->GetHash());
		OnRemoveBakeRecorder.Broadcast(InRecorder.Get());
	}
	if (BakingRecorders.Contains(InRecorder))
	{
		if (InRecorder.IsValid())
		{
			InRecorder->BakeCancelled();
			InRecorder->RemovedFromBake();
		}
		BakingRecorders.Remove(InRecorder);
		if (BakingRecorders.Num() == 0)
		{
			CancelBake();
		}
	}
}

void FSequencerBaker::IsolateRecorderWithGuid(FGuid InGuid, bool bIsolate)
{
	for (TPair<uint32, TSharedPtr < ISequencerBakeRecorder>>& Pair : Recorders)
	{
		if (Pair.Value.IsValid())
		{
			if (Pair.Value->HasSequencerBinding(InGuid))
			{
				Pair.Value->IsolateBakeResult(bIsolate);
			}
		}
	}
}

bool FSequencerBaker::FBakeInterval::SetFromSequencer(ISequencer* InSequencer)
{	
	if (InSequencer && InSequencer->GetFocusedMovieSceneSequence() && InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = InSequencer->GetFocusedDisplayRate();
		TOptional<TRange<FFrameNumber>> OptionalRange = InSequencer->GetSubSequenceRange();
		TRange<FFrameNumber> Range = OptionalRange.IsSet() ? OptionalRange.GetValue() : InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
		if (TimeRange.UpdateIfNeeded(Range, DisplayRate, TickResolution))
		{
			CalculatedFrames.Reset();
			CalculatedFrames.Init(false, TimeRange.NumFrames);
			return true;
		}
	}
	return false;
}

void FSequencerBaker::FBakeInterval::Clear()
{
	CalculatedFrames.Init(false, TimeRange.NumFrames);
}

bool FSequencerBaker::FBakeInterval::IsFrameCalcuated(int32 Index)
{
	if (Index >= 0 && Index < TimeRange.NumFrames)
	{
		return CalculatedFrames[Index];
	}
	return false;
}

void FSequencerBaker::FBakeInterval::SetFrameCalculated(int32 Index)
{
	if (Index >= 0 && Index < TimeRange.NumFrames)
	{
		CalculatedFrames[Index] = true;
	}
}

int32 FSequencerBaker::FBakeInterval::NextFrameToCalculate()
{
	return CalculatedFrames.Find(false);
}

bool FSequencerBaker::FBakeInterval::IsDoneCalculating() const
{
	return (TimeRange.NumFrames > 0 && CalculatedFrames.IsEmpty() == false &&
		CalculatedFrames.Find(false) == INDEX_NONE);
}

static TArray<bool> GetDisabledArray(UMovieSceneTrack* Track)
{
	TArray<bool> IsDisabled;
	if (Track)
	{
		if (Track->GetMaxRowIndex() == 0)
		{
			IsDisabled.Add(Track->IsEvalDisabled());
		}
		else
		{
			for (int32 RowIndex = 0; RowIndex <= Track->GetMaxRowIndex(); ++RowIndex)
			{
				IsDisabled.Add(Track->IsRowEvalDisabled(RowIndex));
			}
		}
	}
	return IsDisabled;
}

void FSequencerBaker::FSignedObjectChanges::AddObject(UMovieSceneSignedObject* InObject, TSharedPtr<ISequencerBakeRecorder>& Owner)
{
	if (InObject)
	{
		FGuid Signature = InObject->GetSignature();
		TArray<bool> DisabledArray;
		if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(InObject))
		{
			DisabledArray = GetDisabledArray(Track);
		}
		if (FSignedObjectState* AlreadyHere = ObjectSignatures.Find(InObject))
		{
			//added track with new owner, clear guid so it get's baked
			if(AlreadyHere->Owners.Contains(Owner) == false)
			{
				AlreadyHere->Guid.Invalidate();
				AlreadyHere->Owners.Add(Owner);
			}
		}
		else //track was never added, clear guid so it get's baked.
		{
			FSignedObjectState State;
			State.Guid.Invalidate();
			State.DisabledArray = DisabledArray;
			State.Owners.Add(Owner);
			ObjectSignatures.Add(InObject, State);
		}
	}
}

void FSequencerBaker::FSignedObjectChanges::RemoveObject(UMovieSceneSignedObject* InObject, TSharedPtr<ISequencerBakeRecorder>& Owner)
{
	if (InObject)
	{
		if (FSignedObjectState* AlreadyHere = ObjectSignatures.Find(InObject))
		{
			AlreadyHere->Owners.Remove(Owner);
			if (AlreadyHere->Owners.Num() == 0)
			{
				ObjectSignatures.Remove(InObject);
			}
		}
	}
}

TArray<TSharedPtr<ISequencerBakeRecorder>> FSequencerBaker::FSignedObjectChanges::UpdateSignatures()
{
	TArray<TSharedPtr<ISequencerBakeRecorder>> ChangedRecorders;
	for (TPair<TWeakObjectPtr<UMovieSceneSignedObject>, FSignedObjectState>& Pair : ObjectSignatures)
	{
		if (Pair.Key.IsValid())
		{
			FGuid Signature = Pair.Key->GetSignature();

			if (Signature != Pair.Value.Guid)
			{
				TArray<bool> DisabledArray;
				if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Pair.Key.Get()))
				{
					DisabledArray = GetDisabledArray(Track);
				}
				if (IgnoredObjects.Contains(Pair.Key) == false) //if ignored update sig but 
				{
					if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Pair.Key.Get()))
					{
						DisabledArray = GetDisabledArray(Track);
					}
					if (DisabledArray.Num() == 0 || DisabledArray == Pair.Value.DisabledArray)
					{
						for (TSharedPtr < ISequencerBakeRecorder>& Owner : Pair.Value.Owners)
						{
							ChangedRecorders.Add(Owner);
						}
					}
				}
				ObjectSignatures[Pair.Key].DisabledArray = DisabledArray;
				ObjectSignatures[Pair.Key].Guid = Signature;
			}
		}
	}
	return ChangedRecorders;
}

};


