// Copyright Epic Games, Inc. All Rights Reserved.


#include "Baker/SequencerBakerSubsystem.h"
#include "Baker/SequencerBaker.h"
#include "ISequencerModule.h"

#include "Evaluation/MovieScenePlayback.h"
#include "LevelSequence.h"
#include "ISceneOutliner.h"
#include "MovieScenePossessable.h"
#include "SequenceBindingTree.h"
#include "MovieScene.h"

#include "MovieSceneSpawnable.h"

void USequencerBakeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	UE_LOGF(LogMovieScene, Log, "USequencerBakeSubsystem subsystem initialized.");

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	OnSequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate::CreateUObject(this, &USequencerBakeSubsystem::OnSequencerCreated));
}

void USequencerBakeSubsystem::Deinitialize()
{
	for (TPair<TWeakPtr<ISequencer>, TSharedPtr<UE::Sequencer::FSequencerBaker>>& Pair: Sequencers)
	{
		if (Pair.Value.IsValid())
		{
			Pair.Value->Release();
		}

	}
	
	Sequencers.Reset();

	Super::Deinitialize();

	UE_LOGF(LogMovieScene, Log, "USequencerBakeSubsystem subsystem deinitialized.");

	ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModulePtr)
	{
		SequencerModulePtr->UnregisterOnSequencerCreated(OnSequencerCreatedHandle);
	}
}

void USequencerBakeSubsystem::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	UE_LOGF(LogMovieScene, VeryVerbose, "ULevelSequenceEditorSubsystem::OnSequencerCreated");

	TSharedPtr<UE::Sequencer::FSequencerBaker> BakerPtr = MakeShared<UE::Sequencer::FSequencerBaker>();
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.ToSharedPtr();
	BakerPtr->Initialize(SequencerPtr);
	Sequencers.Add(TWeakPtr<ISequencer>(InSequencer), BakerPtr);
	
	InSequencer->OnCloseEvent().AddUObject(this, &USequencerBakeSubsystem::OnSequencerClosed);
}

void USequencerBakeSubsystem::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	UE_LOGF(LogMovieScene, VeryVerbose, "ULevelSequenceEditorSubsystem::OnSequencerClosed");

	InSequencer.Get().OnCloseEvent().RemoveAll(this);

	if (TSharedPtr<UE::Sequencer::FSequencerBaker>* BakerPtr = Sequencers.Find(TWeakPtr<ISequencer>(InSequencer)))
	{
		(*BakerPtr)->Release();
		Sequencers.Remove(TWeakPtr<ISequencer>(InSequencer));
	}
}

TSharedPtr<UE::Sequencer::FSequencerBaker> USequencerBakeSubsystem::GetSequencerBaker(const TSharedPtr<ISequencer>& InSequencer)
{
	TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerPtr = nullptr;
	if (TSharedPtr<UE::Sequencer::FSequencerBaker>* SequencerPtrPtr = Sequencers.Find(TWeakPtr<ISequencer>(InSequencer)))
	{
		SequencerPtr = *SequencerPtrPtr;
	}
	return SequencerPtr;
}
