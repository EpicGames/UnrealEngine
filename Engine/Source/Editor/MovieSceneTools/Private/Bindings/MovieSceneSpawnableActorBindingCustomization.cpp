// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneSpawnableActorBindingCustomization.h"

#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "Widgets/Input/STextComboBox.h"
#include "DetailWidgetRow.h"
#include "ISequencer.h"
#include "Misc/PackageName.h"

#define LOCTEXT_NAMESPACE "MovieSceneTools"

TSharedRef<IDetailCustomization> FMovieSceneSpawnableActorBindingBaseCustomization::MakeInstance(TWeakPtr<ISequencer> InSequencer, UMovieScene* InMovieScene, FGuid InBindingGuid)
{
	return MakeShareable(new FMovieSceneSpawnableActorBindingBaseCustomization(InSequencer, InMovieScene, InBindingGuid));
}

void FMovieSceneSpawnableActorBindingBaseCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FMovieSceneSpawnableBindingCustomization::CustomizeDetails(DetailBuilder);

	SpawnLevelProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneSpawnableActorBindingBase, LevelName), UMovieSceneSpawnableActorBindingBase::StaticClass());
	SpawnLevelProperty->MarkHiddenByCustomization();

	// fill combo with level names
	RefreshComboList();

	const int32 FoundIndex = LevelNameList.Find(LevelNameComboSelectedName);
	const TSharedPtr<FString> ComboBoxSelectedItem = LevelNameComboListItems.IsValidIndex(FoundIndex)
		? LevelNameComboListItems[FoundIndex]
		: nullptr;
	IDetailCategoryBuilder& SectionCategory = DetailBuilder.EditCategory("Actor");
	SectionCategory.AddCustomRow(FText())
		.NameContent()
		[
			SpawnLevelProperty->CreatePropertyNameWidget()
		]
		.ValueContent() 
		[
			SNew(SBox)
			[
				SAssignNew(LevelNameComboBox, STextComboBox)
				.OptionsSource(&LevelNameComboListItems)
				.OnSelectionChanged(this, &FMovieSceneSpawnableActorBindingBaseCustomization::OnLevelNameChanged)
				.InitiallySelectedItem(ComboBoxSelectedItem)
				.ContentPadding(2.f)
			]
		];
}


void FMovieSceneSpawnableActorBindingBaseCustomization::RefreshComboList()
{
	SpawnLevelProperty->GetValue(LevelNameComboSelectedName);

	// Refresh Level Names
	{
		TArray< TSharedPtr< FString > > NewLevelNameComboListItems;
		TArray< FName > NewLevelNameList;

		// Persistent Level is coded as NAME_None in the FName list
		NewLevelNameList.Add(FName(NAME_None));
		FString PersistentLevelString = FText(LOCTEXT("UnrealEd", "Persistent Level")).ToString();
		NewLevelNameComboListItems.Add(MakeShareable(new FString(PersistentLevelString)));

		if (SequencerPtr.IsValid())
		{
			UObject* PlaybackContext = SequencerPtr.Pin()->GetPlaybackContext();
			if (!PlaybackContext)
			{
				return;
			}

			UWorld* World = PlaybackContext->GetWorld();
			if (!World)
			{
				return;
			}

			for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
			{
				if (LevelStreaming)
				{
					FName StreamingLevelName = FPackageName::GetShortFName(LevelStreaming->GetWorldAssetPackageFName());
					NewLevelNameList.Add(StreamingLevelName);
					NewLevelNameComboListItems.Add(MakeShareable(new FString(StreamingLevelName.ToString())));
				}
			}

			LevelNameComboListItems = NewLevelNameComboListItems;
			LevelNameList = NewLevelNameList;

			if (LevelNameComboBox.IsValid())
			{
				int32 FoundIndex = LevelNameList.Find(LevelNameComboSelectedName);
				TSharedPtr<FString> ComboItem = LevelNameComboListItems[FoundIndex];

				LevelNameComboBox->SetSelectedItem(ComboItem);
				LevelNameComboBox->SetToolTipText(FText::FromString(*ComboItem));
				LevelNameComboBox->RefreshOptions();
			}
		}
	}
}

void FMovieSceneSpawnableActorBindingBaseCustomization::OnLevelNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		int32 ItemIndex = LevelNameComboListItems.Find(NewSelection);
		if (ItemIndex != INDEX_NONE)
		{
			LevelNameComboSelectedName = LevelNameList[ItemIndex];
			if (LevelNameComboBox.IsValid())
			{
				LevelNameComboBox->SetToolTipText(FText::FromString(*NewSelection));
			}

			ensure(SpawnLevelProperty->SetValue(LevelNameComboSelectedName.ToString()) == FPropertyAccess::Result::Success);
		}
	}
}

#undef LOCTEXT_NAMESPACE
