// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneSpawnableBindingCustomization.h"

#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneFwd.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "MovieSceneSpawnable.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"

TSharedRef<IDetailCustomization> FMovieSceneSpawnableBindingCustomization::MakeInstance(UMovieScene* InMovieScene, FGuid InBindingGuid)
{
	return MakeShareable(new FMovieSceneSpawnableBindingCustomization(InMovieScene, InBindingGuid));
}

void FMovieSceneSpawnableBindingCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	SpawnOwnershipProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneSpawnableBindingBase, SpawnOwnership), UMovieSceneSpawnableBindingBase::StaticClass());
	SpawnOwnershipProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FMovieSceneSpawnableBindingCustomization::OnSpawnOwnershipChanged));
}

void FMovieSceneSpawnableBindingCustomization::OnSpawnOwnershipChanged()
{
	ESpawnOwnership* SpawnOwnership = nullptr;
	TArray<void*> RawData;
	if (MovieScene && BindingGuid.IsValid() && SpawnOwnershipProperty.IsValid())
	{
		SpawnOwnershipProperty->AccessRawData(RawData);
		if (RawData.Num() > 0)
		{
			SpawnOwnership = (ESpawnOwnership*)(RawData[0]);

			MovieScene->Modify();

			// Overwrite the completion state for all spawn sections to ensure the expected behaviour.
			EMovieSceneCompletionMode NewCompletionMode = *SpawnOwnership == ESpawnOwnership::InnerSequence ? EMovieSceneCompletionMode::RestoreState : EMovieSceneCompletionMode::KeepState;

			// Make all binding lifetime and spawn track sections retain state
			UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = MovieScene->FindTrack<UMovieSceneBindingLifetimeTrack>(BindingGuid);
			if (BindingLifetimeTrack)
			{
				for (UMovieSceneSection* Section : BindingLifetimeTrack->GetAllSections())
				{
					Section->Modify();
					Section->EvalOptions.CompletionMode = NewCompletionMode;
				}
			}

			UMovieSceneSpawnTrack* SpawnTrack = MovieScene->FindTrack<UMovieSceneSpawnTrack>(BindingGuid);
			if (SpawnTrack)
			{
				for (UMovieSceneSection* Section : SpawnTrack->GetAllSections())
				{
					Section->Modify();
					Section->EvalOptions.CompletionMode = NewCompletionMode;
				}
			}
		}
	}
}
