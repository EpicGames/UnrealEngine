// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCurveCachePool.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Async/ParallelFor.h"
#include "Cache/MovieSceneCachedCurve.h"

namespace UE::MovieSceneTools 
{ 	
	int32 GCullCachedCurves = 1; 

	static FAutoConsoleVariableRef CCullCachedCurves(
		TEXT("MovieSceneTools.CullCachedCurves"),
		UE::MovieSceneTools::GCullCachedCurves,
		TEXT("When set to true, movie scene cached curves cull draw params"),
		ECVF_Default);

	FMovieSceneCurveCachePool& FMovieSceneCurveCachePool::Get()
	{
		static TSharedRef<FMovieSceneCurveCachePool> Instance = MakeShared<FMovieSceneCurveCachePool>();
		return *Instance;
	}

	void FMovieSceneCurveCachePool::DrawCachedCurves(TWeakPtr<const FCurveEditor> WeakCurveEditor)
	{
		const TArray<TWeakPtr<IMovieSceneCachedCurve>>* CachedCurvesPtr = CurveEditorToCachedCurvesMap.Find(WeakCurveEditor);
		if (CachedCurvesPtr)
		{
			DrawCachedCurves(*CachedCurvesPtr);
			CullCachedCurves(*CachedCurvesPtr);
		}
	}

	void FMovieSceneCurveCachePool::Join(TWeakPtr<const FCurveEditor> WeakCurveEditor, const TSharedRef<IMovieSceneCachedCurve>& CachedCurve)
	{
		CurveEditorToCachedCurvesMap.FindOrAdd(WeakCurveEditor).AddUnique(CachedCurve);
	}

	void FMovieSceneCurveCachePool::Leave(const IMovieSceneCachedCurve& CachedCurve)
	{
		// Find where the curve was added
		TTuple<TWeakPtr<const FCurveEditor>, TArray<TWeakPtr<IMovieSceneCachedCurve>>>* CurveEditorToCachedCurvesPairPtr =
			Algo::FindByPredicate(CurveEditorToCachedCurvesMap, 
				[&CachedCurve](const TTuple<TWeakPtr<const FCurveEditor>, TArray<TWeakPtr<IMovieSceneCachedCurve>>>& CurveEditorToCachedCurvesPair)
				{
					return Algo::FindByPredicate(CurveEditorToCachedCurvesPair.Value,
						[&CachedCurve](const TWeakPtr<IMovieSceneCachedCurve>& OtherCachedCurve)
						{
							return 
								!OtherCachedCurve.IsValid() ||
								OtherCachedCurve.Pin().Get() == &CachedCurve;
						}) != nullptr;
				});
	
		if (CurveEditorToCachedCurvesPairPtr)
		{
			for (auto MapIt = CurveEditorToCachedCurvesMap.CreateIterator(); MapIt; ++MapIt)
			{
				// Remove the curve
				const int32 OldSize = (*CurveEditorToCachedCurvesPairPtr).Value.Num();
				const int32 NewSize = Algo::RemoveIf((*CurveEditorToCachedCurvesPairPtr).Value,
					[&CachedCurve](const TWeakPtr<IMovieSceneCachedCurve>& OtherCachedCurve)
					{
						return 
							!OtherCachedCurve.IsValid() || 
							OtherCachedCurve.Pin().Get() == &CachedCurve;
					});

				if (OldSize != NewSize)
				{
					(*CurveEditorToCachedCurvesPairPtr).Value.SetNum(NewSize);

					// Remove the curve editor if its there's no related curves anymore
					if ((*CurveEditorToCachedCurvesPairPtr).Value.IsEmpty())
					{
						MapIt.RemoveCurrent();
					}

					break;
				}
			}
		}
	}

	void FMovieSceneCurveCachePool::AddTask(const TSharedRef<IMovieSceneCachedCurve>& CachedCurve, const TSharedRef<IMovieSceneInterpolatingPointsDrawTask>& Task)
	{
		check(IsInGameThread()); // Note, if concurrency is ever desired here, we can lock access to CachedCurveToTaskMap

		if (CachedCurveToTaskMap.IsEmpty() &&
			!FCoreDelegates::OnEndFrame.IsBoundToObject(this))
		{
			// Ticking was disabled if there is was no task
			FCoreDelegates::OnEndFrame.AddSP(this, &FMovieSceneCurveCachePool::OnEndFrame);
		}
		
		TSharedPtr<IMovieSceneInterpolatingPointsDrawTask>* RunningTaskPtr = CachedCurveToTaskMap.Find(CachedCurve);
		if (RunningTaskPtr && (*RunningTaskPtr).IsValid())
		{
			// Let a currently running task know he's now void
			(*RunningTaskPtr)->SetFlags(ECurvePainterTaskStateFlags::Void);
			// Set the new task
			(*RunningTaskPtr) = Task;
		}
		else
		{
			// Add the new task
			CachedCurveToTaskMap.Add(CachedCurve, Task);
		}
	}

	void FMovieSceneCurveCachePool::DrawCachedCurves(const TArray<TWeakPtr<IMovieSceneCachedCurve>>& CachedCurves) const
	{
		ParallelFor(CachedCurves.Num(),
			[CachedCurves](int32 Index)
			{
				if (const TSharedPtr<IMovieSceneCachedCurve>& CachedCurve = CachedCurves[Index].Pin())
				{
					CachedCurve->DrawCachedCurve();
				}

			}, EParallelForFlags::Unbalanced);
	}

	void FMovieSceneCurveCachePool::CullCachedCurves(const TArray<TWeakPtr<IMovieSceneCachedCurve>>& CachedCurves) const
	{
		if (GCullCachedCurves <= 0)
		{
			return;
		}

		TMap<uint32, const FCurvePointInfo*> HashToKeyMap;
		TMap<uint32, const FCurveDrawParams*> InterpolatingPointsHashToDrawParamsMap;

		// Reverse to avoid culling those curves that are drawn last resp. top-most
		TArray<TWeakPtr<IMovieSceneCachedCurve>> InverseCachedCurves = CachedCurves;
		Algo::Reverse(InverseCachedCurves);

		for (const TWeakPtr<IMovieSceneCachedCurve>& WeakCachedCurve : InverseCachedCurves)
		{
			if (!WeakCachedCurve.IsValid())
			{
				continue;
			}
			const TSharedRef<IMovieSceneCachedCurve> CachedCurve = WeakCachedCurve.Pin().ToSharedRef();

			FCurveDrawParams* DrawParamsPtr = CachedCurve->GetDrawParamsHandle().Get();
			if (!DrawParamsPtr)
			{
				continue;
			}

			for (FCurvePointInfo& Point : (*DrawParamsPtr).Points)
			{
				const uint32 PointHash = HashCombine(FMath::RoundToInt32(Point.ScreenPosition.X), FMath::RoundToInt32(Point.ScreenPosition.Y));

				// Cull keys but not tangents
				if (Point.Type == ECurvePointType::Key)
				{
					const FCurvePointInfo* const* OtherKeyPtr = HashToKeyMap.Find(PointHash);
					if (OtherKeyPtr && (*OtherKeyPtr)->LayerBias <= Point.LayerBias)
					{
						Point.bDraw = false;
					}
					else
					{
						HashToKeyMap.Add(PointHash, &Point);
					}
				}
			}

			const uint32 InterpolatingPointsHash = CachedCurve->GetInterpolatingPointsHash();
			const FCurveDrawParams** OtherDrawParamsPtrPtr = InterpolatingPointsHashToDrawParamsMap.Find(InterpolatingPointsHash);

			bool bCull = false;
			if (OtherDrawParamsPtrPtr && (*OtherDrawParamsPtrPtr)->InterpolatingPoints.Num() == (*DrawParamsPtr).InterpolatingPoints.Num())
			{
				bCull = true;

				// Test for hash collisions
				for (int32 PointIndex = 0; PointIndex < (*DrawParamsPtr).InterpolatingPoints.Num(); PointIndex++)
				{
					if (FMath::RoundToInt((*OtherDrawParamsPtrPtr)->InterpolatingPoints[PointIndex].X) != FMath::RoundToInt((*DrawParamsPtr).InterpolatingPoints[PointIndex].X) ||
						FMath::RoundToInt((*OtherDrawParamsPtrPtr)->InterpolatingPoints[PointIndex].Y) != FMath::RoundToInt((*DrawParamsPtr).InterpolatingPoints[PointIndex].Y))
					{
						bCull = false;
						break;
					}
				}
			}

			if (bCull)
			{
				(*DrawParamsPtr).bDrawInterpolatingPoints = false;
			}
			else
			{
				InterpolatingPointsHashToDrawParamsMap.Add(InterpolatingPointsHash, DrawParamsPtr);
				(*DrawParamsPtr).bDrawInterpolatingPoints = true;
			}
		}
	}

	void FMovieSceneCurveCachePool::OnEndFrame()
	{
		check(IsInGameThread()); // Note, if concurrency is ever desired here, we can lock access to CachedCurveToTaskMap

		// Avoid any concurrency while work is ongoing
		if (bWorking)
		{
			return;
		}

		// Remove completed and void tasks
		for (TMap<TWeakPtr<IMovieSceneCachedCurve>, TSharedPtr<IMovieSceneInterpolatingPointsDrawTask>>::TIterator It(CachedCurveToTaskMap); It; ++It)
		{
			if ((*It).Value->HasAnyFlags(ECurvePainterTaskStateFlags::Completed | ECurvePainterTaskStateFlags::Void))
			{
				It.RemoveCurrent();
			}
		}
		if (CachedCurveToTaskMap.IsEmpty())
		{
			// All tasks have terminated, stop ticking
			FCoreDelegates::OnEndFrame.RemoveAll(this);
			return;
		}

		TArray<TSharedPtr<IMovieSceneInterpolatingPointsDrawTask>> Tasks;
		CachedCurveToTaskMap.GenerateValueArray(Tasks);

		UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[Tasks, this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FMovieSceneAsyncCurvePainter DrawInterpolatiedPointsAsync);

				bWorking = true;

				ParallelFor(Tasks.Num(),
					[&Tasks](int32 TaskIndex)
					{
						const TSharedPtr<IMovieSceneInterpolatingPointsDrawTask>& Task = Tasks[TaskIndex];
						if (!Task->HasAnyFlags(ECurvePainterTaskStateFlags::Void))
						{
							Task->RefineFullRangeInterpolatingPoints();
						}
					});


				bWorking = false;
			}, LowLevelTasks::ETaskPriority::BackgroundNormal);
	}
}
