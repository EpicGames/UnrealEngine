// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorCurveCachePool.h"

class FCurveEditor;
namespace UE::CurveEditor { struct FCurveDrawParamsHandle; }

namespace UE::MovieSceneTools
{
	/** Interface for a cached curve in the curve cache pool */
	class IMovieSceneCachedCurve
		: public TSharedFromThis<IMovieSceneCachedCurve>
	{
	public:
		IMovieSceneCachedCurve() = default;
		virtual ~IMovieSceneCachedCurve() = default;

		/** Returns the curve model ID that corresponds to this curve */
		virtual const FCurveModelID& GetID() const = 0;

		/** Draws the cached curve */
		virtual void DrawCachedCurve() = 0;

		/** Returns a hash of the currently visible interpolated points */
		virtual uint32 GetInterpolatingPointsHash() const = 0;

		/** Returns a handle to the actual draw params */
		virtual const UE::CurveEditor::FCurveDrawParamsHandle& GetDrawParamsHandle() const = 0;
	};

	/** States of a task in the pool */
	enum class ECurvePainterTaskStateFlags : uint8
	{
		None = 0,

		/** The visible part was written immediately, synchronous */
		Interactive = 1 << 0,

		/** The task successfully finished */
		Completed = 1 << 1,

		/** The task is running but no longer useful */
		Void = 1 << 2,
	};
	ENUM_CLASS_FLAGS(ECurvePainterTaskStateFlags);

	/** Interface for tasks the curve cache pool can handle */
	class IMovieSceneInterpolatingPointsDrawTask
		: public TSharedFromThis<IMovieSceneInterpolatingPointsDrawTask>
	{
	public:
		virtual ~IMovieSceneInterpolatingPointsDrawTask() = default;

		/** Sets flags for this task */
		virtual void SetFlags(ECurvePainterTaskStateFlags NewFlags) = 0;

		/** Returns true if the task has the specified flag set */
		virtual bool HasAnyFlags(ECurvePainterTaskStateFlags Flags) const = 0;

		/** Refines the full range of the interpolating points */
		virtual void RefineFullRangeInterpolatingPoints() = 0;
	};

	/** A pool of cached curves in a curve editor */
	class FMovieSceneCurveCachePool
		: public UE::CurveEditor::ICurveEditorCurveCachePool
		, public TSharedFromThis<FMovieSceneCurveCachePool>
	{
	public:
		virtual ~FMovieSceneCurveCachePool() = default;

		static FMovieSceneCurveCachePool& Get();

		// ICurveEditorCurveCachePool Interface
		virtual void DrawCachedCurves(TWeakPtr<const FCurveEditor> WeakCurveEditor) override;
		//~ICurveEditorCurveCachePool Interface

		/** Lets a cached curve join the cache pool. Curves that joined should also leave */
		void Join(TWeakPtr<const FCurveEditor> WeakCurveEditor, const TSharedRef<IMovieSceneCachedCurve>& CachedCurve);

		/** Lets a cached curve leave the cache pool */
		void Leave(const IMovieSceneCachedCurve& CachedCurve);

		/** Adds a task to the curve cache pool */
		void AddTask(const TSharedRef<IMovieSceneCachedCurve>& CachedCurve, const TSharedRef<IMovieSceneInterpolatingPointsDrawTask>& Task);

	private:
		/** Lets cached curves draw to their draw params */
		void DrawCachedCurves(const TArray<TWeakPtr<IMovieSceneCachedCurve>>& CachedCurves) const;

		/** Culls cached curves */
		void CullCachedCurves(const TArray<TWeakPtr<IMovieSceneCachedCurve>>& CachedCurves) const;

		/** Called on end frame. Updates work on tick */
		void OnEndFrame();

		/** Cached curves per curve editor */
		TMap<TWeakPtr<const FCurveEditor>, TArray<TWeakPtr<IMovieSceneCachedCurve>>> CurveEditorToCachedCurvesMap;

		/** A map of cached curves with their related tasks that need to be performed asynchronous */
		TMap<TWeakPtr<IMovieSceneCachedCurve>, TSharedPtr<IMovieSceneInterpolatingPointsDrawTask>> CachedCurveToTaskMap;

		/** True while work is ongoing */
		std::atomic<bool> bWorking = false;

		/** Critical section to access tasks */
		FCriticalSection AccessTasksCritSec;
	};
}
