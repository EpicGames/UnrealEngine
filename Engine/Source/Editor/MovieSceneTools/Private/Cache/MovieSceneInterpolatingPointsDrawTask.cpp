// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneInterpolatingPointsDrawTask.h"

#include "Cache/MovieSceneCachedCurve.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieScenePiecewiseCurve.h"

namespace UE::MovieSceneTools
{
	template <typename ChannelType>
	FMovieSceneInterpolatingPointsDrawTask<ChannelType>::FMovieSceneInterpolatingPointsDrawTask(
		const TSharedRef<FMovieSceneCachedCurve<ChannelType>>& InCachedCurve,
		const bool bInInvertInterpolatingPointsY,
		const TFunction<void(TArray<FVector2D>, TArray<int32>)>& InCallback)
		: Callback(InCallback)
		, ScreenSpace(InCachedCurve->GetScreenSpace())
		, TickResolution(InCachedCurve->GetTickResolution())
		, TimeThreshold(InCachedCurve->GetTimeThreshold())
		, ValueThreshold(InCachedCurve->GetValueThreshold())
		, PiecewiseCurve(InCachedCurve->GetPiecewiseCurve())
		, bInvertInterpolatingPointsY(bInInvertInterpolatingPointsY)
	{
		const TArray<const FFrameNumber>& Times = InCachedCurve->GetTimes();
		const TArray<const ChannelValueType>& Values = InCachedCurve->GetValues();
		if (!ensureMsgf(Times.Num() > 1, TEXT("Curve paint tasks should only be created for curves with more than one key")))
		{
			SetFlags(ECurvePainterTaskStateFlags::Completed);
			return;
		}

		// Remember key points
		KeyPoints.Reserve(Times.Num());
		const double Sign = bInInvertInterpolatingPointsY ? -1.0 : 1.0;
		for (int32 DataIndex = 0; DataIndex < Times.Num(); DataIndex++)
		{
			KeyPoints.Emplace(Times[DataIndex] / TickResolution, Sign * double(Values[DataIndex].Value));
		}

		InterpolatingPoints = KeyPoints;
	}

	template <typename ChannelType>
	void FMovieSceneInterpolatingPointsDrawTask<ChannelType>::RefineFullRangeInterpolatingPoints()
	{		
		// Make sure there's no concurrency
		if (!AccessCurvePointsMutex.TryLock())
		{
			return;
		}

		const int32 OldSize = InterpolatingPoints.Num();
		RefineFullRangeInterpolatingPointsInternal();

		const int32 NewSize = InterpolatingPoints.Num();
		if (OldSize == NewSize)
		{
			InvokeCallback();
			SetFlags(ECurvePainterTaskStateFlags::Completed);
		}

		AccessCurvePointsMutex.Unlock();
	}

	template <typename ChannelType>
	void FMovieSceneInterpolatingPointsDrawTask<ChannelType>::SetFlags(ECurvePainterTaskStateFlags NewFlags)
	{
		StateFlags.store(NewFlags);
	}

	template <typename ChannelType>
	bool FMovieSceneInterpolatingPointsDrawTask<ChannelType>::HasAnyFlags(ECurvePainterTaskStateFlags Flags) const
	{
		return EnumHasAnyFlags(StateFlags.load(), Flags);
	}

	template<typename ChannelType>
	void FMovieSceneInterpolatingPointsDrawTask<ChannelType>::RefineFullRangeInterpolatingPointsInternal()
	{			
		constexpr float InterpTimes[] = { 0.25f, 0.5f, 0.6f };
		for (int32 Index = 0; Index < InterpolatingPoints.Num() - 1; Index++)
		{
			const FVector2D& Lower = InterpolatingPoints[Index];
			const FVector2D& Upper = InterpolatingPoints[Index + 1];

			if ((Upper.X - Lower.X) >= TimeThreshold)
			{
				bool bSegmentIsLinear = true;

				FVector2D Evaluated[UE_ARRAY_COUNT(InterpTimes)] = { FVector2D::ZeroVector };
				for (int32 InterpIndex = 0; InterpIndex < UE_ARRAY_COUNT(InterpTimes); ++InterpIndex)
				{
					double& EvalTime = Evaluated[InterpIndex].X;
					EvalTime = FMath::Lerp(Lower.X, Upper.X, InterpTimes[InterpIndex]);
					double Value = 0.0;

					PiecewiseCurve->Evaluate(EvalTime * TickResolution, Value);

					if (bInvertInterpolatingPointsY)
					{
						Value = -Value;
					}

					const double LinearValue = FMath::Lerp(Lower.Y, Upper.Y, InterpTimes[InterpIndex]);
					if (bSegmentIsLinear)
					{
						bSegmentIsLinear = FMath::IsNearlyEqual(Value, LinearValue, ValueThreshold);
					}

					Evaluated[InterpIndex].Y = Value;
				}

				if (!bSegmentIsLinear)
				{
					// Add the point
					InterpolatingPoints.Insert(Evaluated, UE_ARRAY_COUNT(Evaluated), Index + 1);
					--Index;
				}
			}
		}
	}

	template<typename ChannelType>
	void FMovieSceneInterpolatingPointsDrawTask<ChannelType>::InvokeCallback()
	{
		TArray<int32> KeyOffsets;

		int32 Offset = 0;
		for (int32 InterpPointIndex = 0; InterpPointIndex < InterpolatingPoints.Num(); InterpPointIndex++)
		{
			if (KeyPoints.Num() <= Offset)
			{
				KeyOffsets.Add(InterpPointIndex);
				break;
			}
			else if (InterpolatingPoints[InterpPointIndex].X >= KeyPoints[Offset].X)
			{
				KeyOffsets.Add(InterpPointIndex);
				Offset++;
			}
		}

		Callback(InterpolatingPoints, KeyOffsets);
	}

	template struct FMovieSceneInterpolatingPointsDrawTask<FMovieSceneFloatChannel>;
	template struct FMovieSceneInterpolatingPointsDrawTask<FMovieSceneDoubleChannel>;
}
