// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDrawInfo.h"
#include "Templates/UnrealTemplate.h"

class FCurveEditor;
class FCurveModel;
struct FCurveEditorScreenSpace;

namespace UE::MovieSceneTools
{
	/** A struct that holds relevant parameters to cache a curve */
	template <typename ChannelType>
	struct FMovieSceneUpdateCachedCurveData
		: public FNoncopyable
	{
	private:
		/** The type of channel value structs */
		using ChannelValueType = typename ChannelType::ChannelValueType;

	public:
		FMovieSceneUpdateCachedCurveData(
			const FCurveEditor& InCurveEditor,
			const FCurveModel& InCurveModel,
			const ChannelType& InChannel,
			const FCurveEditorScreenSpace& InScreenSpace,
			const FFrameRate& InTickResolution,
			const bool bInInvertInterpolatingPointsY)
			: CurveEditor(InCurveEditor)
			, CurveModel(InCurveModel)
			, Channel(InChannel)
			, ScreenSpace(InScreenSpace)
			, TickResolution(InTickResolution)
			, Times(InChannel.GetTimes())
			, Values(InChannel.GetValues())
			, bInvertInterpolatingPointsY(bInInvertInterpolatingPointsY)
		{};

		const FCurveEditor& CurveEditor;
		const FCurveModel& CurveModel;
		const ChannelType& Channel;
		const FCurveEditorScreenSpace& ScreenSpace;
		const FFrameRate& TickResolution;

		const TArrayView<const FFrameNumber> Times;
		const TArrayView<const ChannelValueType> Values;

		const bool bInvertInterpolatingPointsY;
	};
}
