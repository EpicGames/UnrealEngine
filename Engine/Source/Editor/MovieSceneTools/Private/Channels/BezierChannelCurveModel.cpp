// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/BezierChannelCurveModel.h"

#include "Cache/MovieSceneCachedCurve.h"
#include "Cache/MovieSceneUpdateCachedCurveData.h"
#include "Channels/CurveModelHelpers.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneCurveChannelCommon.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/EnumAsByte.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtr.h"

class FCurveEditor;

template<typename ChannelType>
FBezierChannelBufferedCurveModel<ChannelType>::FBezierChannelBufferedCurveModel(
	const ChannelType* InChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
	TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
	: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
	, Channel(*InChannel)
	, WeakSection(InWeakSection)
{}

template<typename ChannelType>
void FBezierChannelBufferedCurveModel<ChannelType>::DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const
{
	UMovieSceneSection* Section    = WeakSection.Get();
	UMovieScene*        MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (Section && MovieScene)
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		const double StartTimeSeconds = InScreenSpace.GetInputMin();
		const double EndTimeSeconds = InScreenSpace.GetInputMax();
		const double TimeThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerInput());
		const double ValueThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerOutput());

		Channel.PopulateCurvePoints(StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, TickResolution, OutInterpolatingPoints);
	}
}

template <typename ChannelType>
bool FBezierChannelBufferedCurveModel<ChannelType>::Evaluate(double InTime, double& OutValue) const
{
	return UE::MovieSceneTools::CurveHelpers::Evaluate(InTime, OutValue, Channel, WeakSection);
}

template class FBezierChannelBufferedCurveModel<FMovieSceneFloatChannel>;
template class FBezierChannelBufferedCurveModel<FMovieSceneDoubleChannel>;

template<typename ChannelType, typename ChannelValue, typename KeyType> 
FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::FBezierChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
	: FChannelCurveModel<ChannelType, ChannelValue, KeyType>(InChannel, OwningSection, InWeakSequencer)
{
	ChannelType* Channel = InChannel.Get();

	if (Channel && OwningSection && OwningSection->GetTypedOuter<UMovieScene>())
	{
		Channel->SetTickResolution(OwningSection->GetTypedOuter<UMovieScene>()->GetTickResolution());
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::FBezierChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* OwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer)
	: FChannelCurveModel<ChannelType, ChannelValue, KeyType>(InChannel, OwningSection, InOwningObject, InWeakSequencer)
{
	ChannelType* Channel = InChannel.Get();

	if (Channel && OwningSection && OwningSection->GetTypedOuter<UMovieScene>())
	{
		Channel->SetTickResolution(OwningSection->GetTypedOuter<UMovieScene>()->GetTickResolution());
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const
{
	ChannelType*        Channel = this->GetChannelHandle().Get();
	UMovieSceneSection* Section = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*     MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (Channel && Section && MovieScene)
	{
		FFrameRate   TickResolution = MovieScene->GetTickResolution();

		const double StartTimeSeconds = InScreenSpace.GetInputMin();
		const double EndTimeSeconds = InScreenSpace.GetInputMax();
		const double TimeThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerInput());
		const double ValueThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerOutput());

		Channel->PopulateCurvePoints(StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, TickResolution, OutInterpolatingPoints);
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
UE::CurveEditor::ICurveEditorCurveCachePool* FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::DrawCurveToCachePool(
	const TSharedRef<FCurveEditor>& CurveEditor,
	const UE::CurveEditor::FCurveDrawParamsHandle& CurveDrawParamsHandle,
	const FCurveEditorScreenSpace& ScreenSpace)
{
	ChannelType*        Channel = this->GetChannelHandle().Get();
	UMovieSceneSection* Section = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*     MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (!CachedCurve.IsValid())
	{
		CachedCurve = MakeShared<UE::MovieSceneTools::FMovieSceneCachedCurve<ChannelType>>(CurveDrawParamsHandle.GetID());
		CachedCurve->Initialize(CurveEditor);
	}

	if (Channel && Section && MovieScene)
	{
		const FMovieSceneChannelMetaData* MetaData = this->GetChannelHandle().GetMetaData();
		const bool bInvertInterpolatingPointsY = MetaData ? MetaData->bInvertValue : false;

		const FFrameRate TickResolution = MovieScene->GetTickResolution();

		UE::MovieSceneTools::FMovieSceneUpdateCachedCurveData<ChannelType> UpdateData(
			*CurveEditor,
			*this,
			*Channel,
			ScreenSpace,
			TickResolution,
			bInvertInterpolatingPointsY);

		CachedCurve->UpdateCachedCurve(UpdateData, CurveDrawParamsHandle);
	}

	return &UE::MovieSceneTools::FMovieSceneCurveCachePool::Get();
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
bool FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::HasChangedAndResetTest()
{
	if (CachedCurve.IsValid())
	{
		return CachedCurve->HasChanged() || Super::HasChangedAndResetTest();
	}
	
	return Super::HasChangedAndResetTest();
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	if (PointType == ECurvePointType::ArriveTangent || PointType == ECurvePointType::LeaveTangent)
	{
		OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.TangentHandle");
		OutDrawInfo.ScreenSize = FVector2D(8, 8);
	}
	else
	{
		// All keys are the same size by default
		OutDrawInfo.ScreenSize = FVector2D(11, 11);

		ERichCurveInterpMode KeyInterpType = RCIM_None;
		ERichCurveTangentWeightMode KeyTWType = RCTWM_WeightedNone;

		// Get the key type from the supplied key handle if it's valid
		ChannelType* Channel = this->GetChannelHandle().Get();
		if (Channel && InKeyHandle != FKeyHandle::Invalid())
		{
			TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
			const int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
			if (KeyIndex != INDEX_NONE)
			{
				KeyInterpType = ChannelData.GetValues()[KeyIndex].InterpMode;
				KeyTWType = ChannelData.GetValues()[KeyIndex].Tangent.TangentWeightMode;
			}
		}

		switch (KeyInterpType)
		{
		case ERichCurveInterpMode::RCIM_Constant:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.ConstantKey");
			break;
		case ERichCurveInterpMode::RCIM_Linear:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.LinearKey");
			break;
		case ERichCurveInterpMode::RCIM_Cubic:
			if (KeyTWType == ERichCurveTangentWeightMode::RCTWM_WeightedBoth)
			{
				OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.WeightedTangentCubicKey");
			}
			else
			{
				OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.CubicKey");
			}

			break;
		default:
			OutDrawInfo.Brush = FAppStyle::GetBrush("GenericCurveEditor.Key");
			break;
		}

		if (this->IsReadOnly())
		{
			OutDrawInfo.Tint = OutDrawInfo.Tint.IsSet() ? OutDrawInfo.Tint.GetValue() * 0.5f : FLinearColor(0.5f, 0.5f, 0.5f);
		}
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
TPair<ERichCurveInterpMode, ERichCurveTangentMode> FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetInterpolationMode(const double& InTime, ERichCurveInterpMode DefaultInterpolationMode, ERichCurveTangentMode DefaultTangentMode) const
{
	ChannelType*        Channel = this->GetChannelHandle().Get();
	UMovieSceneSection* Section = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*     MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (Channel && Section && MovieScene)
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<const ChannelValue> Values = ChannelData.GetValues();

		const FFrameNumber InFrame = (InTime * TickResolution).RoundToFrame();

		if (Times.Num() > 0)
		{
			int32 InterpolationIndex = Algo::LowerBound(Times, InFrame) - 1;
			if (InterpolationIndex < 0)
			{
				InterpolationIndex = 0;
			}
			const FKeyHandle KeyHandle = ChannelData.GetHandle(InterpolationIndex);
			TArrayView<const FKeyHandle> InKey(&KeyHandle, 1);
			TArray<FKeyAttributes> KeyAttributes;
			KeyAttributes.SetNum(1);
			GetKeyAttributes(InKey, KeyAttributes);
			ERichCurveInterpMode InterpMode = KeyAttributes[0].GetInterpMode();
			ERichCurveTangentMode TangentMode = KeyAttributes[0].HasTangentMode() ? KeyAttributes[0].GetTangentMode() : DefaultTangentMode;
			//if we are cubic, with anything but auto tangents we use the default instead, since they will give us flat tangents which aren't good
			if (InterpMode == ERichCurveInterpMode::RCIM_Cubic &&
				(TangentMode != ERichCurveTangentMode::RCTM_Auto && TangentMode != ERichCurveTangentMode::RCTM_SmartAuto))
			{
				TangentMode = DefaultTangentMode;
			}
			return TPair<ERichCurveInterpMode, ERichCurveTangentMode>(InterpMode, TangentMode);
		}
	}

	return TPair<ERichCurveInterpMode, ERichCurveTangentMode>(DefaultInterpolationMode, DefaultTangentMode);
}

static bool IsAuto(ERichCurveTangentMode TangentMode)
{
	return (TangentMode == RCTM_Auto || TangentMode == RCTM_SmartAuto);
}

/**
 * Shared implementation for getting keys.
 *
 * Typically, attributes reflect the settings that the user has manually configured for the keys, so certain attributes may remain unset.
 * For instance, when TangentMode == RCTM_Auto, tangents and weights are automatically computed, meaning attributes like ArriveTangent are not explicitly set.
 * Setting ArriveTangent would imply a user-defined value, which is incompatible with TangentMode == RCTM_Auto.
 *
 * Sometimes API users still need a way to get the auto-computed values, like those of tangents, though.
 * For this, use bAllAttributes parameter:
 * - true: get all attributes, even the auto-computed ones. The values are useful for e.g. UI visualization. Do not pass them to SetKeyAttributes.
 * - false: get only the attributes that were set by the user (i.e. skip returning auto-computed values). You can pass them to SetKeyAttributes, e.g. for copy paste.
 */
template<bool bAllAttributes, typename ChannelType, typename ChannelValue, typename KeyType>
static void GetKeyAttributesDetail(
	ChannelType* InChannel, UMovieSceneSection* InSection, UMovieScene* InMovieScene,
	TArrayView<const FKeyHandle> InKeys,
	TArrayView<FKeyAttributes> OutAttributes
	)
{
	if (!InChannel || !InSection || !InMovieScene)
	{
		return;
	}
	
	TMovieSceneChannelData<ChannelValue> ChannelData = InChannel->GetData();
	TArrayView<ChannelValue> Values = ChannelData.GetValues();

	const float TimeInterval = InMovieScene->GetTickResolution().AsInterval();
	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		const int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
		if (KeyIndex != INDEX_NONE)
		{
			const ChannelValue& KeyValue    = Values[KeyIndex];
			FKeyAttributes&     Attributes  = OutAttributes[Index];

			Attributes.SetInterpMode(KeyValue.InterpMode);

			// If the previous key is cubic, show the arrive tangent handle even if this key is constant
			const int32 PreviousKeyIndex = KeyIndex - 1;
			const bool bGetArriveTangent = Values.IsValidIndex(PreviousKeyIndex) && Values[PreviousKeyIndex].InterpMode == RCIM_Cubic;
			if (bAllAttributes && bGetArriveTangent)
			{
				Attributes.SetArriveTangent(KeyValue.Tangent.ArriveTangent / TimeInterval);
			}

			if ((KeyValue.InterpMode != RCIM_Constant && KeyValue.InterpMode != RCIM_Linear))
			{
				Attributes.SetTangentMode(KeyValue.TangentMode);

				// The remaining settings (arrive / leave tangent, arrive / leave weight, weight mode) can only be user specified, i.e. cannot be set when in auto.
				// Setting any of them would imply they specified by the user, which would contradict the setting that it's auto. 
				if (!bAllAttributes && IsAuto(KeyValue.TangentMode))
				{
					continue;
				}
				
				Attributes.SetArriveTangent(KeyValue.Tangent.ArriveTangent / TimeInterval);
				Attributes.SetLeaveTangent(KeyValue.Tangent.LeaveTangent / TimeInterval);

				if (KeyValue.InterpMode == RCIM_Cubic)
				{
					Attributes.SetTangentWeightMode(KeyValue.Tangent.TangentWeightMode);
					if (KeyValue.Tangent.TangentWeightMode != RCTWM_WeightedNone)
					{
						Attributes.SetArriveTangentWeight(KeyValue.Tangent.ArriveTangentWeight);
						Attributes.SetLeaveTangentWeight(KeyValue.Tangent.LeaveTangentWeight);
					}
				}
			}
		}
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	ChannelType*        Channel    = this->GetChannelHandle().Get();
	UMovieSceneSection* Section    = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*        MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;
	
	GetKeyAttributesDetail<true, ChannelType, ChannelValue, KeyType>(Channel, Section, MovieScene, InKeys, OutAttributes);
}

template <typename ChannelType, typename ChannelValue, typename KeyType>
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys,
	TArrayView<FKeyAttributes> OutAttributes) const
{
	ChannelType*        Channel    = this->GetChannelHandle().Get();
	UMovieSceneSection* Section    = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*        MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	GetKeyAttributesDetail<false, ChannelType, ChannelValue, KeyType>(Channel, Section, MovieScene, InKeys, OutAttributes);
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::SetKeyAttributes)
	UE::MovieScene::FScopedSignedObjectModifyDefer Defer(false /*do not force upate*/);

	ChannelType*        Channel    = this->GetChannelHandle().Get();
	UMovieSceneSection* Section    = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*        MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	// Rule: Auto means tangents and weights are auto computed.
	// This is used below to correct invalid input InAttributes, e.g. if InAttributes says TangentMode is RCTM_Auto, then ArriveTangent cannot be set.
	const auto ResetModeAndWeightForAuto = [](ChannelValue& KeyValue)
	{
		if (IsAuto(KeyValue.TangentMode))
		{
			KeyValue.TangentMode = RCTM_User;
			KeyValue.Tangent.TangentWeightMode = RCTWM_WeightedNone;
		}
	};

	if (Channel && MovieScene && !this->IsReadOnly())
	{
		bool bAutoSetTangents = false;

		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		TArrayView<ChannelValue> Values = ChannelData.GetValues();

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		float TimeInterval = TickResolution.AsInterval();

		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			const int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				const FKeyAttributes& Attributes = InAttributes[Index];
				ChannelValue& KeyValue			 = Values[KeyIndex];
				if (Attributes.HasInterpMode())    { KeyValue.InterpMode  = Attributes.GetInterpMode();  bAutoSetTangents = true; }
				if (Attributes.HasTangentMode())
				{
					KeyValue.TangentMode = Attributes.GetTangentMode();
					if (IsAuto(KeyValue.TangentMode))
					{
						KeyValue.Tangent.TangentWeightMode = RCTWM_WeightedNone;
					}
					bAutoSetTangents = true;
				}
				if (Attributes.HasTangentWeightMode()) 
				{ 
					if (KeyValue.Tangent.TangentWeightMode == RCTWM_WeightedNone) //set tangent weights to default use
					{
						TArrayView<const FFrameNumber> Times = Channel->GetTimes();
						const float OneThird = 1.0f / 3.0f;

						//calculate a tangent weight based upon tangent and time difference
						//calculate arrive tangent weight
						if (KeyIndex > 0 )
						{
							const float X = TickResolution.AsSeconds(Times[KeyIndex].Value - Times[KeyIndex - 1].Value);
							const float ArriveTangentNormal = KeyValue.Tangent.ArriveTangent / (TimeInterval);
							const float Y = ArriveTangentNormal * X;
							KeyValue.Tangent.ArriveTangentWeight = FMath::Sqrt(X*X + Y * Y) * OneThird;
						}
						//calculate leave weight
						if(KeyIndex < ( Times.Num() - 1))
						{
							const float X = TickResolution.AsSeconds(Times[KeyIndex].Value - Times[KeyIndex + 1].Value);
							const float LeaveTangentNormal = KeyValue.Tangent.LeaveTangent / (TimeInterval);
							const float Y = LeaveTangentNormal * X;
							KeyValue.Tangent.LeaveTangentWeight = FMath::Sqrt(X*X + Y*Y) * OneThird;
						}
					}
					KeyValue.Tangent.TangentWeightMode = Attributes.GetTangentWeightMode();

					if( KeyValue.Tangent.TangentWeightMode != RCTWM_WeightedNone )
					{
						if (KeyValue.TangentMode != RCTM_User && KeyValue.TangentMode != RCTM_Break)
						{
							// Input attribute is invalid: Weights can only be set for user or break tangent modes. Correct it.
							KeyValue.TangentMode = RCTM_User;
						}
					}
					bAutoSetTangents = true;
				}

				if (Attributes.HasArriveTangent())
				{
					// If TangentMode is auto, then the input attribute is invalid: it implies arrive tangent is user specified. Correct TangentMode.
					ResetModeAndWeightForAuto(KeyValue);

					KeyValue.Tangent.ArriveTangent = Attributes.GetArriveTangent() * TimeInterval;
					if (KeyValue.InterpMode == RCIM_Cubic && KeyValue.TangentMode != RCTM_Break)
					{
						KeyValue.Tangent.LeaveTangent = KeyValue.Tangent.ArriveTangent;
					}
					bAutoSetTangents = true;
				}

				if (Attributes.HasLeaveTangent())
				{
					// If TangentMode is auto, then the input attribute is invalid: it implies leave tangent is user specified. Correct TangentMode.
					ResetModeAndWeightForAuto(KeyValue);

					KeyValue.Tangent.LeaveTangent = Attributes.GetLeaveTangent() * TimeInterval;
					if (KeyValue.InterpMode == RCIM_Cubic && KeyValue.TangentMode != RCTM_Break)
					{
						KeyValue.Tangent.ArriveTangent = KeyValue.Tangent.LeaveTangent;
					}
					bAutoSetTangents = true;
				}

				if (Attributes.HasArriveTangentWeight())
				{
					// If TangentMode is auto, then the input attribute is invalid: it implies arrive weight is user specified. Correct TangentMode.
					ResetModeAndWeightForAuto(KeyValue);

					KeyValue.Tangent.ArriveTangentWeight = Attributes.GetArriveTangentWeight(); 
					if (KeyValue.InterpMode == RCIM_Cubic && KeyValue.TangentMode != RCTM_Break)
					{
						KeyValue.Tangent.LeaveTangentWeight = KeyValue.Tangent.ArriveTangentWeight;
					}
					bAutoSetTangents = true;
				}

				if (Attributes.HasLeaveTangentWeight())
				{
					// If TangentMode is auto, then the input attribute is invalid: it implies leave weight is user specified. Correct TangentMode.
					ResetModeAndWeightForAuto(KeyValue);

					KeyValue.Tangent.LeaveTangentWeight = Attributes.GetLeaveTangentWeight();
					if (KeyValue.InterpMode == RCIM_Cubic && KeyValue.TangentMode != RCTM_Break)
					{
						KeyValue.Tangent.ArriveTangentWeight = KeyValue.Tangent.LeaveTangentWeight;
					}
					bAutoSetTangents = true;
				}
			}
		}

		if (bAutoSetTangents)
		{
			Channel->AutoSetTangents();
		}
		
		OnPostOwnerChanged(Section);
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	ChannelType* Channel = this->GetChannelHandle().Get();
	if (Channel)
	{
		OutCurveAttributes.SetPreExtrapolation(Channel->PreInfinityExtrap);
		OutCurveAttributes.SetPostExtrapolation(Channel->PostInfinityExtrap);
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
	ChannelType*        Channel = this->GetChannelHandle().Get();
	UMovieSceneSection* Section = this->template GetOwningObjectOrOuter<UMovieSceneSection>();

	if (Channel && Section && !this->IsReadOnly())
	{
		if (InCurveAttributes.HasPreExtrapolation())
		{
			Channel->PreInfinityExtrap = InCurveAttributes.GetPreExtrapolation();
		}

		if (InCurveAttributes.HasPostExtrapolation())
		{
			Channel->PostInfinityExtrap = InCurveAttributes.GetPostExtrapolation();
		}

		OnPostOwnerChanged(Section);
	}
}

template <typename ChannelType, typename ChannelValue, typename KeyType>
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::OnPostOwnerChanged(TNotNull<UMovieSceneSignedObject*> InSignedOwner)
{
	this->LastSignature.Invalidate();

	InSignedOwner->MarkAsChanged();
	InSignedOwner->MarkPackageDirty();

	if (UMovieSceneSignedObject* SignedOwner = Cast<UMovieSceneSignedObject>(this->WeakOwningObject.Get());
		SignedOwner && SignedOwner != InSignedOwner)
	{
		SignedOwner->MarkAsChanged();
		SignedOwner->MarkPackageDirty();
	}

	this->CurveModifiedDelegate.Broadcast();
}

/*	 Finds min/max for cubic curves:
Looks for feature points in the signal(determined by change in direction of local tangent), these locations are then re-examined in closer detail recursively
Similar to function in RichCurve but usees the Channel::Evaluate function, instead of CurveModel::Eval*/

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::FeaturePointMethod(double StartTime, double EndTime, double StartValue, double Mu, int Depth, int MaxDepth, double& MaxV, double& MinVal) const
{
	if (Depth >= MaxDepth)
	{
		return;
	}
	double PrevValue = StartValue;
	double EvalValue;
	this->Evaluate(StartTime - Mu, EvalValue);
	double PrevTangent = StartValue - EvalValue;
	EndTime += Mu;
	for (double f = StartTime + Mu; f < EndTime; f += Mu)
	{
		double Value;
		this->Evaluate(f, Value);

		MaxV = FMath::Max(Value, MaxV);
		MinVal = FMath::Min(Value, MinVal);
		double CurTangent = Value - PrevValue;
		if (FMath::Sign(CurTangent) != FMath::Sign(PrevTangent))
		{
			//feature point centered around the previous tangent
			double FeaturePointTime = f - Mu * 2.0f;
			double NewVal;
			this->Evaluate(FeaturePointTime, NewVal);
			FeaturePointMethod(FeaturePointTime, f,NewVal, Mu*0.4f, Depth + 1, MaxDepth, MaxV, MinVal);
		}
		PrevTangent = CurTangent;
		PrevValue = Value;
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetValueRange(double InMinTime, double InMaxTime, double& MinValue, double& MaxValue) const
{
	ChannelType*        Channel    = this->GetChannelHandle().Get();
	UMovieSceneSection* Section    = this->template GetOwningObjectOrOuter<UMovieSceneSection>();
	UMovieScene*        MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (Channel && MovieScene && !this->IsReadOnly())
	{
		const TConstArrayView<FFrameNumber> Times = Channel->GetData().GetTimes();
		const TConstArrayView<ChannelValue> Values = Channel->GetData().GetValues();

		if (Times.Num() == 0)
		{
			// If there are no keys we just use the default value for the channel, defaulting to zero if there is no default.
			MinValue = MaxValue = Channel->GetDefault().Get(0.f);
		}
		else
		{
			const FFrameRate TickResolution = MovieScene->GetTickResolution();
			const double ToTime = TickResolution.AsInterval();
			const int32 LastKeyIndex = Values.Num() - 1;
			MinValue = TNumericLimits<double>::Max();
			MaxValue = TNumericLimits<double>::Lowest();

			for (int32 i = 0; i < Values.Num(); i++)
			{
				const double KeyTime = static_cast<double>(Times[i].Value) * ToTime;
				if (KeyTime < InMinTime)
				{
					continue;
				}
				if (KeyTime > InMaxTime)
				{
					break;
				}
				
				const ChannelValue& Key = Values[i];
				MinValue = FMath::Min(MinValue, static_cast<double>(Key.Value));
				MaxValue = FMath::Max(MaxValue, static_cast<double>(Key.Value));

				const bool bNeedsCubicEstimation = Key.InterpMode == RCIM_Cubic && i != LastKeyIndex;
				const bool bIsWeighted = Key.Tangent.TangentWeightMode != RCTWM_WeightedNone;
				if (bNeedsCubicEstimation && !bIsWeighted)
				{
					// This method is faster than FeaturePointMethod's regression method because the extrema are computed by solving the derivatives.
					// 17th of July, 2025: On ControlRigExample -> PossessableBindings_CopyPaste -> Possessable_FKCR_CopyPaste, open Curve Editor.
					// In SCurveEditorViewNormalized::UpdateViewToTransformCurves usage of GetValueRange totalling 110.000 keys accross all curves,
					// we saved 135ms. 
					const UE::MovieScene::Interpolation::FInterpolationExtents Extents = Channel->ComputeExtents(Times[i].Value, Times[i + 1].Value);
					MinValue = FMath::Min(MinValue, Extents.MinValue);
					MaxValue = FMath::Max(MaxValue, Extents.MaxValue);
				}
				else if (bNeedsCubicEstimation && bIsWeighted)
				{
					// UE-305634: FCachedInterpolation::ComputeExtents is not yet implemented correctly for weighted tangents.
					// So we need to fall back to the slow regression method. 
					const double NextTime = static_cast<double>(Times[i + 1].Value) * ToTime;
					const double TimeStep = (NextTime - KeyTime) * 0.2f;
					FeaturePointMethod(KeyTime, NextTime, Key.Value, TimeStep, 0, 3, MaxValue, MinValue);
				}
			}
		}
	}
	//if nothing found just set to zero
	if (MinValue == TNumericLimits<double>::Max())
	{
		MinValue = 0.0;
	}
	if (MaxValue == TNumericLimits<double>::Lowest())
	{
		MaxValue = 0.0;
	}
}


template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetValueRange(double& MinValue, double& MaxValue) const
{
	const double InMinTime = TNumericLimits<double>::Lowest();
	const double InMaxTime = TNumericLimits<double>::Max();
	GetValueRange(InMinTime, InMaxTime, MinValue, MaxValue);
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
double FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyValue(TArrayView<const ChannelValue> Values, int32 Index) const
{
	return Values[Index].Value;
}

template<typename ChannelType, typename ChannelValue, typename KeyType> 
void FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::SetKeyValue(int32 Index, double KeyValue) const
{
	ChannelType* Channel = this->GetChannelHandle().Get();

	if (Channel)
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		ChannelData.GetValues()[Index].Value = KeyValue;
	}
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
bool FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::HasDefaultValue() const
{
	if (const ChannelType* Channel = this->GetChannelHandle().Get())
	{
		return Channel->GetDefault().IsSet();
	}
	return false;
}

template<typename ChannelType, typename ChannelValue, typename KeyType>
double FBezierChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetDefaultValue() const
{
	if (const ChannelType* Channel = this->GetChannelHandle().Get())
	{
		if (HasDefaultValue())
		{
			return static_cast<double>(Channel->GetDefault().GetValue());
		}
	}
	return 0.0;
}

template class FBezierChannelCurveModel<FMovieSceneFloatChannel, FMovieSceneFloatValue, float>;
template class FBezierChannelCurveModel<FMovieSceneDoubleChannel, FMovieSceneDoubleValue, double>;

