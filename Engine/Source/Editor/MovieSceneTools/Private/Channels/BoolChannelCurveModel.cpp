// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/BoolChannelCurveModel.h"

#include "Algo/BinarySearch.h"
#include "Channels/BoolChannelKeyProxy.h"
#include "Channels/CurveModelHelpers.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class UObject;

/**
 * Buffered curve implementation for a bool channel curve model, stores a copy of the bool channel in order to draw itself.
 */
class FBoolChannelBufferedCurveModel : public IBufferedCurveModel
{
public:
	/** Create a copy of the float channel while keeping the reference to the section */
	FBoolChannelBufferedCurveModel(const FMovieSceneBoolChannel* InMovieSceneBoolChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
		TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
		: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
		, Channel(*InMovieSceneBoolChannel)
		, WeakSection(InWeakSection)
	{}

	virtual void DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override
	{
		UMovieSceneSection* Section = WeakSection.Get();

		if (Section)
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

			TMovieSceneChannelData<const bool> ChannelData = Channel.GetData();
			TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
			TArrayView<const bool> Values = ChannelData.GetValues();

			const double StartTimeSeconds = InScreenSpace.GetInputMin();
			const double EndTimeSeconds = InScreenSpace.GetInputMax();

			const FFrameNumber StartFrame = (StartTimeSeconds * TickResolution).FloorToFrame();
			const FFrameNumber EndFrame = (EndTimeSeconds * TickResolution).CeilToFrame();

			const int32 StartingIndex = Algo::UpperBound(Times, StartFrame);
			const int32 EndingIndex = Algo::LowerBound(Times, EndFrame);

			for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
			{
				OutInterpolatingPoints.Add(MakeTuple(Times[KeyIndex] / TickResolution, double(Values[KeyIndex])));
			}
		}
	}

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		return UE::MovieSceneTools::CurveHelpers::Evaluate(InTime, OutValue, Channel, WeakSection);
	}

private:
	FMovieSceneBoolChannel Channel;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};

FBoolChannelCurveModel::FBoolChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneBoolChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
	: FChannelCurveModel<FMovieSceneBoolChannel, bool, bool>(InChannel, OwningSection, InWeakSequencer)
{
}

void FBoolChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		UBoolChannelKeyProxy* NewProxy = NewObject<UBoolChannelKeyProxy>(GetTransientPackage(), NAME_None);

		NewProxy->Initialize(InKeyHandles[Index], GetChannelHandle(), this->GetOwningObjectOrOuter<UMovieSceneSection>());
		OutObjects[Index] = NewProxy;
	}
}

TUniquePtr<IBufferedCurveModel> FBoolChannelCurveModel::CreateBufferedCurveCopy() const
{
	FMovieSceneBoolChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		TArray<FKeyHandle> TargetKeyHandles;
		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();

		TRange<FFrameNumber> TotalRange = ChannelData.GetTotalRange();
		ChannelData.GetKeys(TotalRange, nullptr, &TargetKeyHandles);

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(GetNumKeys());
		TArray<FKeyAttributes> KeyAttributes;
		KeyAttributes.SetNumUninitialized(GetNumKeys());
		GetKeyPositions(TargetKeyHandles, KeyPositions);
		GetKeyAttributes(TargetKeyHandles, KeyAttributes);

		double ValueMin = 0.f, ValueMax = 1.f;
		GetValueRange(ValueMin, ValueMax);

		return MakeUnique<FBoolChannelBufferedCurveModel>(Channel, this->GetOwningObjectOrOuter<UMovieSceneSection>(), MoveTemp(KeyPositions), MoveTemp(KeyAttributes), GetLongDisplayName().ToString(), ValueMin, ValueMax);
	}
	return nullptr;
}

void FBoolChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	FMovieSceneBoolChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		OutCurveAttributes.SetPreExtrapolation(Channel->PreInfinityExtrap);
		OutCurveAttributes.SetPostExtrapolation(Channel->PostInfinityExtrap);
	}
}

void FBoolChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
	FMovieSceneBoolChannel* Channel = GetChannelHandle().Get();
	UMovieSceneSection* Section = this->GetOwningObjectOrOuter<UMovieSceneSection>();
	if (Channel && Section && !IsReadOnly())
	{
		Section->MarkAsChanged();

		if (InCurveAttributes.HasPreExtrapolation())
		{
			Channel->PreInfinityExtrap = InCurveAttributes.GetPreExtrapolation();
		}

		if (InCurveAttributes.HasPostExtrapolation())
		{
			Channel->PostInfinityExtrap = InCurveAttributes.GetPostExtrapolation();
		}

		CurveModifiedDelegate.Broadcast();
	}
}

double FBoolChannelCurveModel::GetKeyValue(TArrayView<const bool> Values, int32 Index) const
{
	return (double)Values[Index];
}

void FBoolChannelCurveModel::SetKeyValue(int32 Index, double KeyValue) const
{
	FMovieSceneBoolChannel* Channel = GetChannelHandle().Get();

	if (Channel)
	{
		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
		ChannelData.GetValues()[Index] = KeyValue > 0 ? true : false;
	}
}
