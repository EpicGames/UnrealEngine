// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/BoolChannelKeyProxy.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelHandle.h"

#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoolChannelKeyProxy)

struct FPropertyChangedEvent;

void UBoolChannelKeyProxy::Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneBoolChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSection> InWeakSection)
{
	KeyHandle     = InKeyHandle;
	ChannelHandle = InChannelHandle;
	WeakSection   = InWeakSection;
}

void UBoolChannelKeyProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bNewValue = bValue;

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		bNewValue = !bNewValue;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnProxyValueChanged(ChannelHandle, WeakSection.Get(), KeyHandle, bNewValue, Time);
}

void UBoolChannelKeyProxy::UpdateValuesFromRawData()
{
	RefreshCurrentValue(ChannelHandle, KeyHandle, bValue, Time);

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		bValue = !bValue;
	}
}
