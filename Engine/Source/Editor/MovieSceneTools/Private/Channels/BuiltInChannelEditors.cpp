// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/BuiltInChannelEditors.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceEditor.h"
#include "MovieSceneEventUtils.h"
#include "Channels/InvertedBezierChannelCurveModel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "ISequencerChannelInterface.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SNullWidget.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "SequencerSettings.h"
#include "CurveEditor.h"
#include "MovieSceneCommonHelpers.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "CurveKeyEditors/SNumericKeyEditor.h"
#include "CurveKeyEditors/SBoolCurveKeyEditor.h"
#include "CurveKeyEditors/SStringCurveKeyEditor.h"
#include "CurveKeyEditors/STextCurveKeyEditor.h"
#include "CurveKeyEditors/SEnumKeyEditor.h"
#include "UObject/StructOnScope.h"
#include "MVVM/Views/KeyDrawParams.h"
#include "MVVM/ViewModels/TimeWarpChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/DoubleChannelCurveModel.h"
#include "Channels/FloatChannelCurveModel.h"
#include "Channels/IntegerChannelCurveModel.h"
#include "Channels/BoolChannelCurveModel.h"
#include "Channels/ByteChannelCurveModel.h"
#include "Channels/TimeWarpChannelCurveModel.h"
#include "Channels/PiecewiseCurveModel.h"
#include "Variants/MovieScenePlayRateCurve.h"
#include "EventChannelCurveModel.h"
#include "InvertedCurveModel.h"
#include "PropertyCustomizationHelpers.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "SSocketChooser.h"
#include "SComponentChooser.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Widgets/Input/SComboButton.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "ISequencerModule.h"
#include "MovieScene.h"
#include "MovieSceneTracksComponentTypes.h"
#include "KeyStructTypes/MovieSceneTextKeyStruct.h"

#define LOCTEXT_NAMESPACE "BuiltInChannelEditors"


template<typename ChannelType, typename ValueType>
FKeyHandle AddOrUpdateKeyImpl(ChannelType* Channel, UMovieSceneSection* SectionToKey, const TMovieSceneExternalValue<ValueType>& ExternalValue, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	using namespace UE::MovieScene;

	const FMovieSceneSequenceID SequenceID = Sequencer.GetFocusedTemplateID();

	// Find the first bound object so we can get the current property channel value on it.
	UObject* FirstBoundObject = nullptr;
	TOptional<ValueType> CurrentBoundObjectValue;
	if (InObjectBindingID.IsValid())
	{
		for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(InObjectBindingID, SequenceID))
		{
			if (UObject* Object = WeakObject.Get())
			{
				FirstBoundObject = Object;

				if (ExternalValue.OnGetExternalValue)
				{
					CurrentBoundObjectValue = ExternalValue.OnGetExternalValue(*Object, PropertyBindings);
				}

				break;
			}
		}
	}

	// If we got the current property channel value on the object, let's get the current evaluated property channel value at the given time (which is the value that the
	// object *would* be at if we scrubbed here and let the sequence evaluation do its thing). This will help us figure out the difference between the current object value
	// and the evaluated sequencer value: we will compute a new value for the channel so that a new sequence evaluation would come out at the "desired" value, which is
	// what the current object value.
	ValueType NewValue = Channel->GetDefault().Get(0.f);

	const bool bWasEvaluated = Channel->Evaluate(InTime, NewValue);

	if (CurrentBoundObjectValue.IsSet() && SectionToKey)
	{
		if (ExternalValue.OnGetCurrentValueAndWeight)
		{
			// We have a custom callback that can provide us with the evaluated value of this channel.
			ValueType CurrentValue = CurrentBoundObjectValue.Get(0.0f);
			float CurrentWeight = 1.0f;
			FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer.GetEvaluationTemplate();
			ExternalValue.OnGetCurrentValueAndWeight(FirstBoundObject, SectionToKey, InTime, Sequencer.GetFocusedTickResolution(), EvaluationTemplate, CurrentValue, CurrentWeight);

			if (CurrentBoundObjectValue.IsSet()) //need to get the diff between Value(Global) and CurrentValue and apply that to the local
			{
				if (bWasEvaluated)
				{
					ValueType CurrentGlobalValue = CurrentBoundObjectValue.GetValue();
					NewValue = (CurrentBoundObjectValue.Get(0.0f) - CurrentValue) * CurrentWeight + NewValue;
				}
				else //Nothing set (key or default) on channel so use external value
				{
					NewValue = CurrentBoundObjectValue.Get(0.0f);
				}
			}
		}
		else
		{
			// No custom callback... we need to run the blender system on our property.
			FSystemInterrogator Interrogator;
			Interrogator.TrackImportedEntities(true);

			TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);

			UMovieSceneTrack* TrackToKey = SectionToKey->GetTypedOuter<UMovieSceneTrack>();

			// If we are keying something for a property track, give the interrogator all the info it needs
			// to know about the bound object. This will let it, for instance, cache the correct initial values
			// for that property.
			FInterrogationKey InterrogationKey(FInterrogationKey::Default());
			UMovieScenePropertyTrack* PropertyTrackToKey = Cast<UMovieScenePropertyTrack>(TrackToKey);
			if (PropertyTrackToKey != nullptr)
			{
				const FInterrogationChannel InterrogationChannel = Interrogator.AllocateChannel(FirstBoundObject, PropertyTrackToKey->GetPropertyBinding());
				InterrogationKey.Channel = InterrogationChannel;
				Interrogator.ImportTrack(TrackToKey, InterrogationChannel);
			}
			else
			{
				Interrogator.ImportTrack(TrackToKey, FInterrogationChannel::Default());
			}

			// Interrogate!
			Interrogator.AddInterrogation(InTime);
			Interrogator.Update();

			const FMovieSceneEntityID EntityID = Interrogator.FindEntityFromOwner(InterrogationKey, SectionToKey, 0);

			UMovieSceneInterrogatedPropertyInstantiatorSystem* System = Interrogator.GetLinker()->FindSystem<UMovieSceneInterrogatedPropertyInstantiatorSystem>();

			if (ensure(System) && EntityID)  // EntityID can be invalid here if we are keying a section that is currently empty
			{
				const FMovieSceneChannelProxy& SectionChannelProxy = SectionToKey->GetChannelProxy();
				const FName ChannelTypeName = Channel->StaticStruct()->GetFName();
				int32 ChannelIndex = SectionChannelProxy.FindIndex(ChannelTypeName, Channel);

				FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

				// Find the property definition based on the property tag that our section entity has.
				int32 BoundPropertyDefinitionIndex = INDEX_NONE;
				TArrayView<const FPropertyDefinition> PropertyDefinitions = BuiltInComponents->PropertyRegistry.GetProperties();
				for (int32 Index = 0; Index < PropertyDefinitions.Num(); ++Index)
				{
					const FPropertyDefinition& PropertyDefinition = PropertyDefinitions[Index];
					if (Interrogator.GetLinker()->EntityManager.HasComponent(EntityID, PropertyDefinition.PropertyType))
					{
						BoundPropertyDefinitionIndex = Index;
						break;
					}
				}

				if (ensure(ChannelIndex != INDEX_NONE && BoundPropertyDefinitionIndex != INDEX_NONE))
				{
					const FPropertyDefinition& BoundPropertyDefinition = PropertyDefinitions[BoundPropertyDefinitionIndex];

					check(FirstBoundObject != nullptr);
					TComponentLock<TReadOptional<FBoundObjectResolver>> Resolver =
							Interrogator.GetLinker()->EntityManager.ReadComponent(EntityID, BuiltInComponents->BoundObjectResolver);
					if (Resolver)
					{
						FirstBoundObject = (*Resolver)(FirstBoundObject);
						check(FirstBoundObject != nullptr);
					}

					FDecompositionQuery Query;
					Query.Entities = MakeArrayView(&EntityID, 1);
					Query.bConvertFromSourceEntityIDs = false;
					Query.Object = FirstBoundObject;

					FIntermediate3DTransform InTransformData;

					TRecompositionResult<double> RecomposeResult = System->RecomposeBlendChannel(BoundPropertyDefinition, ChannelIndex, Query, (double)CurrentBoundObjectValue.Get(0.f));

					NewValue = (ValueType)RecomposeResult.Values[0];
				}
			}
		}
	}
	using namespace UE::MovieScene;
	EMovieSceneKeyInterpolation KeyInterpolation = GetInterpolationMode(Channel,InTime,Sequencer.GetKeyInterpolation());
	return AddKeyToChannel(Channel, InTime, NewValue, KeyInterpolation);
}

FKeyHandle AddOrUpdateKey(FMovieSceneFloatChannel* Channel, UMovieSceneSection* SectionToKey, const TMovieSceneExternalValue<float>& ExternalValue, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	return AddOrUpdateKeyImpl<FMovieSceneFloatChannel, float>(Channel, SectionToKey, ExternalValue, InTime, Sequencer, InObjectBindingID, PropertyBindings);
}

FKeyHandle AddOrUpdateKey(FMovieSceneDoubleChannel* Channel, UMovieSceneSection* SectionToKey, const TMovieSceneExternalValue<double>& ExternalValue, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	return AddOrUpdateKeyImpl<FMovieSceneDoubleChannel, double>(Channel, SectionToKey, ExternalValue, InTime, Sequencer, InObjectBindingID, PropertyBindings);
}

FKeyHandle AddOrUpdateKey(FMovieSceneTimeWarpChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	TMovieSceneExternalValue<double> ExternalValue;
	return AddOrUpdateKey(Channel, SectionToKey, ExternalValue, InTime, Sequencer, InObjectBindingID, PropertyBindings);
}

FKeyHandle AddOrUpdateKey(FMovieSceneActorReferenceChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	if (PropertyBindings && InObjectBindingID.IsValid())
	{
		for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(InObjectBindingID, Sequencer.GetFocusedTemplateID()))
		{
			if (UObject* Object = WeakObject.Get())
			{
				// Care is taken here to ensure that we call GetCurrentValue with the correct instantiation of UObject* rather than AActor*
				AActor* CurrentActor = Cast<AActor>(PropertyBindings->GetCurrentValue<UObject*>(*Object));
				if (CurrentActor)
				{
					FMovieSceneObjectBindingID Binding;

					TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(CurrentActor);
					if (Spawnable.IsSet())
					{
						// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
						Binding = UE::MovieScene::FRelativeObjectBindingID(Sequencer.GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, Sequencer);
					}
					else
					{
						FGuid ThisGuid = Sequencer.GetHandleToObject(CurrentActor);
						Binding = UE::MovieScene::FRelativeObjectBindingID(ThisGuid);
					}

					int32 NewIndex = Channel->GetData().AddKey(InTime, Binding);

					return Channel->GetData().GetHandle(NewIndex);
				}
			}
		}
	}

	FMovieSceneActorReferenceKey NewValue;

	Channel->Evaluate(InTime, NewValue);

	return Channel->GetData().UpdateOrAddKey(InTime, NewValue);
}

bool CanCreateKeyEditor(const FMovieSceneBoolChannel*    Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneByteChannel*    Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneIntegerChannel* Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneFloatChannel*   Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneDoubleChannel*   Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneStringChannel*  Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneTextChannel* Channel)
{
	return true;
}
bool CanCreateKeyEditor(const FMovieSceneObjectPathChannel* Channel)
{
	return true;
}

bool CanCreateKeyEditor(const FMovieSceneActorReferenceChannel* Channel)
{
	return true;
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneBoolChannel>&    Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<bool>* ExternalValue = Channel.GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneBoolChannel, bool> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));
	return SNew(SBoolCurveKeyEditor, KeyEditor);
}


TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneIntegerChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<int32>* ExternalValue = Channel.GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneIntegerChannel, int32> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	typedef SNumericKeyEditor<FMovieSceneIntegerChannel, int32> KeyEditorType;
	return SNew(KeyEditorType, KeyEditor);
}


TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneFloatChannel>&   Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<float>* ExternalValue = Channel.GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneFloatChannel, float> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	typedef SNumericKeyEditor<FMovieSceneFloatChannel, float> KeyEditorType;
	return SNew(KeyEditorType, KeyEditor);
}


TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>&   Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	TFunction<TOptional<double>(UObject&, FTrackInstancePropertyBindings*)> ExternalValue;
	if (const TMovieSceneExternalValue<double>* ExternalValuePtr = Channel.GetExtendedEditorData())
	{
		ExternalValue = ExternalValuePtr->OnGetExternalValue;
	}

	TSequencerKeyEditor<FMovieSceneDoubleChannel, double> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	typedef SNumericKeyEditor<FMovieSceneDoubleChannel, double> KeyEditorType;
	return SNew(KeyEditorType, KeyEditor);
}


TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneStringChannel>&  Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<FString>* ExternalValue = Channel.GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneStringChannel, FString> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	return SNew(SStringCurveKeyEditor, KeyEditor);
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneTextChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<FText>* ExternalValue = Channel.GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneTextChannel, FText> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	return SNew(STextCurveKeyEditor, KeyEditor);
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneByteChannel>&    Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<uint8>* ExternalValue = Channel.GetExtendedEditorData();
	const FMovieSceneByteChannel* RawChannel = Channel.Get();
	if (!ExternalValue || !RawChannel)
	{
		return SNullWidget::NullWidget;
	}

	TSequencerKeyEditor<FMovieSceneByteChannel, uint8> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
		);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	if (UEnum* Enum = RawChannel->GetEnum())
	{
		return SNew(SEnumCurveKeyEditor, KeyEditor, Enum);
	}
	else
	{
		typedef SNumericKeyEditor<FMovieSceneByteChannel, uint8> KeyEditorType;
		return SNew(KeyEditorType, KeyEditor);
	}
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneObjectPathChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const TMovieSceneExternalValue<UObject*>* ExternalValue = Channel.GetExtendedEditorData();
	const FMovieSceneObjectPathChannel*       RawChannel    = Channel.Get();
	UMovieSceneObjectPropertyTrack* ObjectPathTrack = Cast<UMovieSceneObjectPropertyTrack>(Params.OwningSection->GetOuter());

	if (ExternalValue && RawChannel)
	{
		TSequencerKeyEditor<FMovieSceneObjectPathChannel, UObject*> KeyEditor(Params.ObjectBindingID, Channel, Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue);
		KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

		UClass* PropertyClass = ObjectPathTrack ? ObjectPathTrack->PropertyClass.Get() : nullptr;
		const bool bClassPicker = ObjectPathTrack ? ObjectPathTrack->bClassProperty : false;
		if (bClassPicker)
		{
			auto OnSetClassLambda = [KeyEditor](const UClass* Class) mutable
			{
				FScopedTransaction Transaction(LOCTEXT("SetObjectPathKey", "Set Object Path Key Value"));
				KeyEditor.SetValueWithNotify(const_cast<UClass*>(Class), EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
			};

			auto GetSelectedClassLambda = [KeyEditor]() -> const UClass*
			{
				return Cast<UClass>(KeyEditor.GetCurrentValue());
			};

			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(100.f)
					[
						SNew(SClassPropertyEntryBox)
						.MetaClass(PropertyClass)
						.SelectedClass_Lambda(GetSelectedClassLambda)
						.OnSetClass_Lambda(OnSetClassLambda)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(SSpacer)
				];
		}
		else
		{
			auto OnSetObjectLambda = [KeyEditor](const FAssetData& Asset) mutable
			{
				FScopedTransaction Transaction(LOCTEXT("SetObjectPathKey", "Set Object Path Key Value"));
				KeyEditor.SetValueWithNotify(Asset.GetAsset(), EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
			};

			auto GetObjectPathLambda = [KeyEditor]() -> FString
			{
				UObject* Obj = KeyEditor.GetCurrentValue();
				return Obj ? Obj->GetPathName() : FString();
			};

			TArray<FAssetData> AssetDataArray;

			UMovieSceneSequence* Sequence = Params.Sequencer->GetFocusedMovieSceneSequence();
			AssetDataArray.Add((FAssetData)Sequence);

			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(100.f)
					[
						SNew(SObjectPropertyEntryBox)
						.DisplayBrowse(true)
						.DisplayUseSelected(false)
						.ObjectPath_Lambda(GetObjectPathLambda)
						.AllowedClass(RawChannel->GetPropertyClass())
						.OnObjectChanged_Lambda(OnSetObjectLambda)
						.OwnerAssetDataArray(AssetDataArray)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(SSpacer)
				];
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	TFunction<TOptional<double>(UObject&, FTrackInstancePropertyBindings*)> ExternalValue;

	TSequencerKeyEditor<FMovieSceneTimeWarpChannel, double> KeyEditor(
		Params.ObjectBindingID, Channel,
		Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue
		);
	
	KeyEditor.SetApplyInUnwarpedLocalSpace(true);
	KeyEditor.SetOwningObject(Cast<UMovieSceneSignedObject>(Params.OwningObject));

	const FMovieSceneTimeWarpChannel* ChannelPtr = Channel.Get();
	if (ChannelPtr && ChannelPtr->Domain == UE::MovieScene::ETimeWarpChannelDomain::Time)
	{
		// Set the numeric type interface for frame numbers if the channel is in the time domain
		KeyEditor.SetNumericTypeInterface(Params.Sequencer->GetNumericTypeInterface());
	}

	typedef SNumericKeyEditor<FMovieSceneTimeWarpChannel, double> KeyEditorType;
	return SNew(KeyEditorType, KeyEditor);
}

template<typename ChannelType>
void GetTypedChannels(ISequencer* Sequencer, const TSet<TWeakObjectPtr<UMovieSceneSection>>& WeakSections, TArray<ChannelType*>& Channels)
{
	// Get selected channels
	TArray<const IKeyArea*> KeyAreas;
	Sequencer->GetSelectedKeyAreas(KeyAreas);
	for (const IKeyArea* KeyArea : KeyAreas)
	{
		FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
		if (Handle.GetChannelTypeName() == ChannelType::StaticStruct()->GetFName())
		{
			ChannelType* Channel = static_cast<ChannelType*>(Handle.Get());
			Channels.Add(Channel);
		}
	}

	// Otherwise, the channels of all the sections
	if (Channels.Num() == 0)
	{
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				for (ChannelType* Channel : ChannelProxy.GetChannels<ChannelType>())
				{
					Channels.Add(Channel);
				}
			}
		}
	}
}

/** Delegate used to set a class */
DECLARE_DELEGATE_OneParam(FOnSetActorReferenceKey, FMovieSceneActorReferenceKey);

class SActorReferenceBox : public SCompoundWidget, public FMovieSceneObjectBindingIDPicker
{
public:
	SLATE_BEGIN_ARGS(SActorReferenceBox)
	{}
	SLATE_ATTRIBUTE(FMovieSceneActorReferenceKey, ActorReferenceKey)
	SLATE_EVENT(FOnSetActorReferenceKey, OnSetActorReferenceKey)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<ISequencer> InSequencer)
	{
		WeakSequencer = InSequencer;
		LocalSequenceID = InSequencer.Pin()->GetFocusedTemplateID();

		Key = InArgs._ActorReferenceKey;
		SetKey = InArgs._OnSetActorReferenceKey;

		OnGlobalTimeChangedHandle = WeakSequencer.Pin()->OnGlobalTimeChanged().AddRaw(this, &SActorReferenceBox::GlobalTimeChanged);
		OnMovieSceneDataChangedHandle = WeakSequencer.Pin()->OnMovieSceneDataChanged().AddRaw(this, &SActorReferenceBox::MovieSceneDataChanged);

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &SActorReferenceBox::GetPickerMenu)
					.ContentPadding(FMargin(0.0, 0.0))
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.ButtonContent()
					[
						GetCurrentItemWidget(
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
						)
					]
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				GetWarningWidget()
			]
		];

		Update();
	}

	virtual ~SActorReferenceBox()
	{
		if (WeakSequencer.IsValid())
		{
			WeakSequencer.Pin()->OnGlobalTimeChanged().Remove(OnGlobalTimeChangedHandle);
			WeakSequencer.Pin()->OnMovieSceneDataChanged().Remove(OnMovieSceneDataChangedHandle);
		}
	}

	virtual UMovieSceneSequence* GetSequence() const override
	{
		return WeakSequencer.Pin()->GetFocusedMovieSceneSequence();
	}

	/** Set the current binding ID */
	virtual void SetCurrentValue(const FMovieSceneObjectBindingID& InBindingId) override
	{
		SetKey.Execute(FMovieSceneActorReferenceKey(InBindingId));
	}

	/** Get the current binding ID */
	virtual FMovieSceneObjectBindingID GetCurrentValue() const override
	{
		return Key.Get().Object;
	}

	void GlobalTimeChanged()
	{
		Update();
	}

	void MovieSceneDataChanged(EMovieSceneDataChangeType)
	{
		Update();
	}

	void Update()
	{
		if (IsEmpty())
		{
			Initialize();
		}
		else
		{
			UpdateCachedData();
		}
	}

private:

	TAttribute< FMovieSceneActorReferenceKey> Key;

	FOnSetActorReferenceKey SetKey;

	FDelegateHandle OnGlobalTimeChangedHandle, OnMovieSceneDataChangedHandle;
};


TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneActorReferenceChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	const FMovieSceneActorReferenceChannel* RawChannel = Channel.Get();
	if (!RawChannel)
	{
		return SNullWidget::NullWidget;
	}

	TFunction<TOptional<FMovieSceneActorReferenceKey>(UObject&, FTrackInstancePropertyBindings*)> Func;

	TSequencerKeyEditor<FMovieSceneActorReferenceChannel, FMovieSceneActorReferenceKey> KeyEditor(Params.ObjectBindingID, Channel, Params.OwningSection, Params.Sequencer, Params.PropertyBindings, Func);

	auto OnSetCurrentValueLambda = [KeyEditor](const FMovieSceneActorReferenceKey& ActorKey) mutable
	{
		FScopedTransaction Transaction(LOCTEXT("SetActorReferenceKey", "Set Actor Reference Key Value"));
		KeyEditor.SetValueWithNotify(ActorKey, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);

		// Look for components to choose
		ISequencer* Sequencer = KeyEditor.GetSequencer();
		TArray<USceneComponent*> ComponentsWithSockets;
		AActor* Actor = nullptr;
		for (TWeakObjectPtr<> WeakObject : ActorKey.Object.ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer))
		{
			Actor = Cast<AActor>(WeakObject.Get());
			if (Actor)
			{
				TInlineComponentArray<USceneComponent*> Components(Actor);

				for (USceneComponent* Component : Components)
				{
					if (Component && Component->HasAnySockets())
					{
						ComponentsWithSockets.Add(Component);
					}
				}
				break;
			}
		}

		if (ComponentsWithSockets.Num() == 0 || !Actor)
		{
			return;
		}

		// Pop up a component chooser
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

		TSharedPtr<SWidget> ComponentMenuWidget =
			SNew(SComponentChooserPopup)
			.Actor(Actor)
			.OnComponentChosen_Lambda([Actor, LevelEditor, KeyEditor, ActorKey = ActorKey](FName InComponentName) mutable
				{
					// ActorKey is self-captured so that the lambda can mutate its copy.

					ActorKey.ComponentName = InComponentName;
					KeyEditor.SetValueWithNotify(ActorKey, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);

					// Look for sockets to choose
					USceneComponent* ComponentWithSockets = nullptr;
					TInlineComponentArray<USceneComponent*> Components(Actor);

					for (USceneComponent* Component : Components)
					{
						if (Component && Component->GetFName() == InComponentName)
						{
							ComponentWithSockets = Component;
							break;
						}
					}

					if (!ComponentWithSockets)
					{
						return;
					}
							
					// Pop up a socket chooser
					TSharedPtr<SWidget> SocketMenuWidget =
						SNew(SSocketChooserPopup)
						.SceneComponent(ComponentWithSockets)
						.OnSocketChosen_Lambda([=](FName InSocketName) mutable
							{
								ActorKey.SocketName = InSocketName;
								KeyEditor.SetValueWithNotify(ActorKey, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
							}
						);

					// Create as context menu
					FSlateApplication::Get().PushMenu(
						LevelEditor.ToSharedRef(),
						FWidgetPath(),
						SocketMenuWidget.ToSharedRef(),
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
					);
				}
			);

		// Create as context menu
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			ComponentMenuWidget.ToSharedRef(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	};

	auto GetCurrentValueLambda = [KeyEditor]() -> FMovieSceneActorReferenceKey
	{
		return KeyEditor.GetCurrentValue();
	};

	return SNew(SActorReferenceBox, Params.Sequencer)
		.ActorReferenceKey_Lambda(GetCurrentValueLambda)
		.OnSetActorReferenceKey_Lambda(OnSetCurrentValueLambda);
}

UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneByteChannel* Channel, FSequencerKeyStructGenerator* Generator)
{
	UEnum* ByteEnum = Channel->GetEnum();
	if (!ByteEnum)
	{
		// No enum so just use the default (which will create a generated struct with a byte property)
		return Generator->DefaultInstanceGeneratedStruct(FMovieSceneByteChannel::StaticStruct());
	}

	FName GeneratedTypeName = *FString::Printf(TEXT("MovieSceneByteChannel_%s"), *ByteEnum->GetName());

	UMovieSceneKeyStructType* Existing = Generator->FindGeneratedStruct(GeneratedTypeName);
	if (Existing)
	{
		return Existing;
	}

	UMovieSceneKeyStructType* NewStruct = FSequencerKeyStructGenerator::AllocateNewKeyStruct(FMovieSceneByteChannel::StaticStruct());
	if (!NewStruct)
	{
		return nullptr;
	}

	FByteProperty* NewValueProperty = new FByteProperty(NewStruct, "Value");
	NewValueProperty->SetPropertyFlags(CPF_Edit);
	NewValueProperty->SetMetaData("Category", TEXT("Key"));
	NewValueProperty->ArrayDim = 1;
	NewValueProperty->Enum = ByteEnum;

	NewStruct->AddCppProperty(NewValueProperty);
	NewStruct->DestValueProperty = NewValueProperty;

	FSequencerKeyStructGenerator::FinalizeNewKeyStruct(NewStruct);

	Generator->AddGeneratedStruct(GeneratedTypeName, NewStruct);
	return NewStruct;
}

UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneTimeWarpChannel* Channel, FSequencerKeyStructGenerator* Generator)
{
	FName GeneratedTypeName("FMovieSceneTimeWarpChannelGeneratedStruct");

	UMovieSceneKeyStructType* Existing = Generator->FindGeneratedStruct(GeneratedTypeName);
	if (Existing)
	{
		return Existing;
	}

	UMovieSceneKeyStructType* NewStruct = FSequencerKeyStructGenerator::AllocateNewKeyStruct(FMovieSceneTimeWarpChannel::StaticStruct());
	if (!NewStruct)
	{
		return nullptr;
	}

	FStructProperty* NewValueProperty = new FStructProperty(NewStruct, "Value");
	NewValueProperty->SetPropertyFlags(CPF_Edit);
	NewValueProperty->SetMetaData("Category", TEXT("Key"));
	NewValueProperty->Struct = TBaseStructure<FFrameNumber>::Get();

	NewStruct->AddCppProperty(NewValueProperty);
	NewStruct->DestValueProperty = NewValueProperty;

	FSequencerKeyStructGenerator::FinalizeNewKeyStruct(NewStruct);

	Generator->AddGeneratedStruct(GeneratedTypeName, NewStruct);
	return NewStruct;
}

UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneTextChannel* Channel, FSequencerKeyStructGenerator* Generator)
{
	UScriptStruct* ChannelType = FMovieSceneTextChannel::StaticStruct();
	check(ChannelType);

	UPackage* const Package = Channel->GetPackage();
	if (!ensureMsgf(Package, TEXT("No valid package could be found for Text Channel")))
	{
		return nullptr;
	}

	// Generated struct to be unique per Package
	const FName InstanceStructName = *(ChannelType->GetName() + Package->GetPersistentGuid().ToString());

	UMovieSceneKeyStructType* Existing = Generator->FindGeneratedStruct(InstanceStructName);
	if (Existing)
	{
		return Existing;
	}

	static const FName TimesMetaDataTag("KeyTimes");
	static const FName ValuesMetaDataTag("KeyValues");

	FArrayProperty* SourceTimes  = FSequencerKeyStructGenerator::FindArrayPropertyWithTag(ChannelType, TimesMetaDataTag);
	FArrayProperty* SourceValues = FSequencerKeyStructGenerator::FindArrayPropertyWithTag(ChannelType, ValuesMetaDataTag);

	if (!ensureAlwaysMsgf(SourceTimes && SourceValues
		, TEXT("FMovieSceneTextChannel does not have KeyTimes & KeyValues meta data tags in their respective array properties. Did something change ?")))
	{
		return nullptr;
	}

	constexpr EObjectFlags ObjectFlags = RF_Public | RF_Standalone | RF_Transient;

	UMovieSceneTextKeyStruct* NewStruct = NewObject<UMovieSceneTextKeyStruct>(Package, NAME_None, ObjectFlags);
	NewStruct->SourceTimesProperty  = SourceTimes;
	NewStruct->SourceValuesProperty = SourceValues;

	FTextProperty* NewValueProperty = new FTextProperty(NewStruct, "Value");
	NewValueProperty->SetPropertyFlags(CPF_Edit);
	NewValueProperty->SetMetaData("Category", TEXT("Key"));
	NewValueProperty->ArrayDim = 1;

	NewStruct->AddCppProperty(NewValueProperty);
	NewStruct->DestValueProperty = NewValueProperty;

	FSequencerKeyStructGenerator::FinalizeNewKeyStruct(NewStruct);

	Generator->AddGeneratedStruct(InstanceStructName, NewStruct);
	return NewStruct;
}

void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)
{
	const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<const UMovieSceneKeyStructType>(Struct->GetStruct());

	uint8* StructMemory = Struct->GetStructMemory();

	FStructProperty*     ValueProperty = CastFieldChecked<FStructProperty>(GeneratedStructType->DestValueProperty.Get());
	FStructProperty*     TimeProperty  = CastFieldChecked<FStructProperty>(GeneratedStructType->DestTimeProperty.Get());

	const FFrameNumber*  TimeAddress   = TimeProperty->ContainerPtrToValuePtr<FFrameNumber>(StructMemory);
	FFrameNumber*        ValueAddress  = ValueProperty->ContainerPtrToValuePtr<FFrameNumber>(StructMemory);

	// It is safe to capture the property and address in this lambda because the lambda is owned by the struct itself, so cannot be invoked if the struct has been destroyed
	auto CopyInstanceToKeyLambda = [ChannelHandle, InHandle, GeneratedStructType, ValueProperty, ValueAddress, TimeAddress](const FPropertyChangedEvent&)
	{
		if (FMovieSceneTimeWarpChannel* DestinationChannel = ChannelHandle.Get())
		{
			const int32 KeyIndex = DestinationChannel->GetData().GetIndex(InHandle);
			if (KeyIndex != INDEX_NONE)
			{
				DestinationChannel->GetData().GetValues()[KeyIndex].Value = ValueAddress->Value;

				// Set the new key time
				DestinationChannel->SetKeyTime(InHandle, *TimeAddress);
			}
		}
	};

	FGeneratedMovieSceneKeyStruct* KeyStruct = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(Struct->GetStructMemory());
	KeyStruct->OnPropertyChangedEvent = CopyInstanceToKeyLambda;

	// Copy the initial value for the struct
	FMovieSceneTimeWarpChannel* Channel = ChannelHandle.Get();
	if (Channel)
	{
		// Copy the initial value into the struct
		const int32 KeyIndex = Channel->GetData().GetIndex(InHandle);
		if (KeyIndex != INDEX_NONE)
		{
			double InitialTime = Channel->GetData().GetValues()[KeyIndex].Value;
			*ValueAddress = FFrameTime::FromDecimal(InitialTime).FloorToFrame();
		}
	}
}

UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneObjectPathChannel* Channel, FSequencerKeyStructGenerator* Generator)
{
	UClass* PropertyClass = Channel->GetPropertyClass();
	if (!PropertyClass)
	{
		// No specific property class so just use the default (which will create a generated struct with an object property)
		return Generator->DefaultInstanceGeneratedStruct(FMovieSceneObjectPathChannel::StaticStruct());
	}

	FName GeneratedTypeName = *FString::Printf(TEXT("MovieSceneObjectPathChannel_%s"), *PropertyClass->GetName());

	UMovieSceneKeyStructType* Existing = Generator->FindGeneratedStruct(GeneratedTypeName);
	if (Existing)
	{
		return Existing;
	}

	UMovieSceneKeyStructType* NewStruct = FSequencerKeyStructGenerator::AllocateNewKeyStruct(FMovieSceneObjectPathChannel::StaticStruct());
	if (!NewStruct)
	{
		return nullptr;
	}

	FObjectProperty* NewValueProperty = new FObjectProperty(NewStruct, "Value");
	NewValueProperty->SetPropertyFlags(CPF_Edit | CPF_TObjectPtrWrapper);
	NewValueProperty->SetMetaData("Category", TEXT("Key"));
	NewValueProperty->PropertyClass = PropertyClass;
	NewValueProperty->ArrayDim = 1;

	NewStruct->AddCppProperty(NewValueProperty);
	NewStruct->DestValueProperty = NewValueProperty;

	FSequencerKeyStructGenerator::FinalizeNewKeyStruct(NewStruct);

	Generator->AddGeneratedStruct(GeneratedTypeName, NewStruct);
	return NewStruct;
}


void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneObjectPathChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)
{
	const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<const UMovieSceneKeyStructType>(Struct->GetStruct());

	uint8* StructMemory = Struct->GetStructMemory();

	FObjectProperty*     ValueProperty = CastFieldChecked<FObjectProperty>(GeneratedStructType->DestValueProperty.Get());
	FStructProperty*     TimeProperty  = CastFieldChecked<FStructProperty>(GeneratedStructType->DestTimeProperty.Get());

	const FFrameNumber*  TimeAddress   = TimeProperty->ContainerPtrToValuePtr<FFrameNumber>(StructMemory);
	void*                ValueAddress  = ValueProperty->ContainerPtrToValuePtr<uint8>(StructMemory);

	// It is safe to capture the property and address in this lambda because the lambda is owned by the struct itself, so cannot be invoked if the struct has been destroyed
	auto CopyInstanceToKeyLambda = [ChannelHandle, InHandle, GeneratedStructType, ValueProperty, ValueAddress, TimeAddress](const FPropertyChangedEvent&)
	{
		if (FMovieSceneObjectPathChannel* DestinationChannel = ChannelHandle.Get())
		{
			const int32 KeyIndex = DestinationChannel->GetData().GetIndex(InHandle);
			if (KeyIndex != INDEX_NONE)
			{
				UObject* ObjectPropertyValue = ValueProperty->GetObjectPropertyValue(ValueAddress);
				DestinationChannel->GetData().GetValues()[KeyIndex] = ObjectPropertyValue;

				// Set the new key time
				DestinationChannel->SetKeyTime(InHandle, *TimeAddress);
			}
		}
	};

	FGeneratedMovieSceneKeyStruct* KeyStruct = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(Struct->GetStructMemory());
	KeyStruct->OnPropertyChangedEvent = CopyInstanceToKeyLambda;

	// Copy the initial value for the struct
	FMovieSceneObjectPathChannel* Channel = ChannelHandle.Get();
	if (Channel)
	{
		// Copy the initial value into the struct
		const int32 KeyIndex = Channel->GetData().GetIndex(InHandle);
		if (KeyIndex != INDEX_NONE)
		{
			UObject* InitialObject = Channel->GetData().GetValues()[KeyIndex].Get();
			ValueProperty->SetObjectPropertyValue(ValueAddress, InitialObject);
		}
	}
}

void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)
{
	bool bInvertValue = false;
	FGeneratedMovieSceneKeyStruct* KeyStruct = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(Struct->GetStructMemory());
	const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<const UMovieSceneKeyStructType>(Struct->GetStruct());
	void* StructMemory = Struct->GetStructMemory();
	FStructProperty* ValueProperty = CastFieldChecked<FStructProperty>(GeneratedStructType->DestValueProperty.Get());
	FStructProperty* TimeProperty = CastFieldChecked<FStructProperty>(GeneratedStructType->DestTimeProperty.Get());

	const FFrameNumber* TimeAddress = TimeProperty->ContainerPtrToValuePtr<FFrameNumber>(StructMemory);

	FMovieSceneDoubleChannel* Channel = ChannelHandle.Get();
	if (Channel)
	{
		if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
		{
			bInvertValue = MetaData->bInvertValue;
		}

		const int32 KeyIndex = Channel->GetData().GetIndex(InHandle);

		// Copy the initial value into the struct
		if (KeyIndex != INDEX_NONE)
		{
			double InitialValue = Channel->GetData().GetValues()[KeyIndex].Value;
			*ValueProperty->ContainerPtrToValuePtr<double>(StructMemory) = bInvertValue ? -InitialValue : InitialValue;
		}
	}

	// It is safe to capture the property and address in this lambda because the lambda is owned by the struct itself, so cannot be invoked if the struct has been destroyed
	auto CopyInstanceToKeyLambda = [ChannelHandle, InHandle, StructMemory, ValueProperty, bInvertValue, TimeAddress](const FPropertyChangedEvent&)
	{
		if (FMovieSceneDoubleChannel* DestinationChannel = ChannelHandle.Get())
		{
			const int32 KeyIndex = DestinationChannel->GetData().GetIndex(InHandle);
			if (KeyIndex != INDEX_NONE)
			{
				double Value = *ValueProperty->ContainerPtrToValuePtr<double>(StructMemory);
				DestinationChannel->GetData().GetValues()[KeyIndex].Value = bInvertValue ? -Value : Value;

				// Set the new key time
				DestinationChannel->SetKeyTime(InHandle, *TimeAddress);
			}
		}
	};

	KeyStruct->OnPropertyChangedEvent = CopyInstanceToKeyLambda;
}

void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneTextChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)
{
	FMovieSceneTextChannel* Channel = ChannelHandle.Get();
	if (!Channel)
	{
		return;
	}

	// Set Package that the Text Property will need for Localization
	Struct->SetPackage(ChannelHandle.Get()->GetPackage());

	const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<const UMovieSceneKeyStructType>(Struct->GetStruct());

	const int32 InitialKeyIndex = Channel->GetData().GetIndex(InHandle);

	// Copy the initial value into the struct
	if (InitialKeyIndex != INDEX_NONE)
	{
		const uint8* SrcValueData  = GeneratedStructType->SourceValuesProperty->ContainerPtrToValuePtr<uint8>(Channel);
		uint8* DestValueData = GeneratedStructType->DestValueProperty->ContainerPtrToValuePtr<uint8>(Struct->GetStructMemory());

		FScriptArrayHelper SourceValuesArray(GeneratedStructType->SourceValuesProperty.Get(), SrcValueData);
		GeneratedStructType->SourceValuesProperty->Inner->CopyCompleteValue(DestValueData, SourceValuesArray.GetRawPtr(InitialKeyIndex));
	}
}

template<typename ChannelType>
void DrawKeysImpl(ChannelType* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;

	static const FName CircleKeyBrushName("Sequencer.KeyCircle");
	static const FName DiamondKeyBrushName("Sequencer.KeyDiamond");
	static const FName SquareKeyBrushName("Sequencer.KeySquare");
	static const FName TriangleKeyBrushName("Sequencer.KeyTriangle");

	const FSlateBrush* CircleKeyBrush = FAppStyle::GetBrush(CircleKeyBrushName);
	const FSlateBrush* DiamondKeyBrush = FAppStyle::GetBrush(DiamondKeyBrushName);
	const FSlateBrush* SquareKeyBrush = FAppStyle::GetBrush(SquareKeyBrushName);
	const FSlateBrush* TriangleKeyBrush = FAppStyle::GetBrush(TriangleKeyBrushName);

	TMovieSceneChannelData<ChannelValueType> ChannelData = Channel->GetData();
	TArrayView<const ChannelValueType> Values = ChannelData.GetValues();

	FKeyDrawParams TempParams;
	TempParams.BorderBrush = TempParams.FillBrush = DiamondKeyBrush;
	TempParams.ConnectionStyle = EKeyConnectionStyle::Solid;

	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		FKeyHandle Handle = InKeyHandles[Index];

		const int32 KeyIndex = ChannelData.GetIndex(Handle);

		ERichCurveInterpMode InterpMode   = KeyIndex == INDEX_NONE ? RCIM_None : Values[KeyIndex].InterpMode.GetValue();
		ERichCurveTangentMode TangentMode = KeyIndex == INDEX_NONE ? RCTM_None : Values[KeyIndex].TangentMode.GetValue();

		TempParams.FillOffset = FVector2D(0.f, 0.f);
		TempParams.ConnectionStyle = EKeyConnectionStyle::Solid;

		switch (InterpMode)
		{
		case RCIM_Linear:
			TempParams.BorderBrush = TempParams.FillBrush = TriangleKeyBrush;
			TempParams.FillTint = FLinearColor(0.0f, 0.617f, 0.449f, 1.0f); // blueish green
			TempParams.FillOffset = FVector2D(0.0f, 1.0f);
			break;

		case RCIM_Constant:
			TempParams.BorderBrush = TempParams.FillBrush = SquareKeyBrush;
			TempParams.FillTint = FLinearColor(0.0f, 0.445f, 0.695f, 1.0f); // blue
			TempParams.ConnectionStyle = EKeyConnectionStyle::Dashed;
			break;

		case RCIM_Cubic:
			TempParams.BorderBrush = TempParams.FillBrush = CircleKeyBrush;

			switch (TangentMode)
			{
			case RCTM_SmartAuto:  TempParams.FillTint = FLinearColor(0.759f, 0.176f, 0.67f, 1.0f);break; // little vermillion
			case RCTM_Auto:  TempParams.FillTint = FLinearColor(0.972f, 0.2f, 0.2f, 1.0f);     break; // vermillion
			case RCTM_Break: TempParams.FillTint = FLinearColor(0.336f, 0.703f, 0.5f, 0.91f);  break; // sky blue
			case RCTM_User:  TempParams.FillTint = FLinearColor(0.797f, 0.473f, 0.5f, 0.652f); break; // reddish purple
			default:         TempParams.FillTint = FLinearColor(0.75f, 0.75f, 0.75f, 1.0f);    break; // light gray
			}
			break;

		default:
			TempParams.BorderBrush = TempParams.FillBrush = DiamondKeyBrush;
			TempParams.FillTint   = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f); // white
			break;
		}

		OutKeyDrawParams[Index] = TempParams;
	}
}

void DrawKeys(FMovieSceneFloatChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	DrawKeysImpl(Channel, InKeyHandles, InOwner, OutKeyDrawParams);
}

void DrawKeys(FMovieSceneDoubleChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	DrawKeysImpl(Channel, InKeyHandles, InOwner, OutKeyDrawParams);
}

void DrawKeys(FMovieSceneParticleChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	static const FName KeyLeftBrushName("Sequencer.KeyLeft");
	static const FName KeyRightBrushName("Sequencer.KeyRight");
	static const FName KeyDiamondBrushName("Sequencer.KeyDiamond");

	const FSlateBrush* LeftKeyBrush = FAppStyle::GetBrush(KeyLeftBrushName);
	const FSlateBrush* RightKeyBrush = FAppStyle::GetBrush(KeyRightBrushName);
	const FSlateBrush* DiamondBrush = FAppStyle::GetBrush(KeyDiamondBrushName);

	TMovieSceneChannelData<uint8> ChannelData = Channel->GetData();

	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		FKeyHandle Handle = InKeyHandles[Index];

		FKeyDrawParams Params;
		Params.BorderBrush = Params.FillBrush = DiamondBrush;

		const int32 KeyIndex = ChannelData.GetIndex(Handle);
		if ( KeyIndex != INDEX_NONE )
		{
			const EParticleKey Value = (EParticleKey)ChannelData.GetValues()[KeyIndex];
			if ( Value == EParticleKey::Activate )
			{
				Params.BorderBrush = Params.FillBrush = LeftKeyBrush;
				Params.FillOffset = FVector2D(-1.0f, 1.0f);
			}
			else if ( Value == EParticleKey::Deactivate )
			{
				Params.BorderBrush = Params.FillBrush = RightKeyBrush;
				Params.FillOffset = FVector2D(1.0f, 1.0f);
			}
		}

		OutKeyDrawParams[Index] = Params;
	}
}

void DrawKeys(FMovieSceneEventChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	UMovieSceneEventSectionBase* EventSection = CastChecked<UMovieSceneEventSectionBase>(const_cast<UMovieSceneSection*>(InOwner));

	FKeyDrawParams ValidEventParams, InvalidEventParams;

	ValidEventParams.BorderBrush   = ValidEventParams.FillBrush   = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");

	InvalidEventParams.FillBrush   = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
	InvalidEventParams.BorderBrush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamondBorder");
	InvalidEventParams.FillTint    = FLinearColor(1.f,1.f,1.f,.2f);

	TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();
	TArrayView<FMovieSceneEvent>             Events = ChannelData.GetValues();

	UMovieSceneSequence*       Sequence           = InOwner->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor     = Sequence ? FMovieSceneSequenceEditor::Find(Sequence) : nullptr;
	UBlueprint*                SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		int32 KeyIndex = ChannelData.GetIndex(InKeyHandles[Index]);

		if (KeyIndex != INDEX_NONE && SequenceDirectorBP && FMovieSceneEventUtils::FindEndpoint(&Events[KeyIndex], EventSection, SequenceDirectorBP))
		{
			OutKeyDrawParams[Index] = ValidEventParams;
		}
		else
		{
			OutKeyDrawParams[Index] = InvalidEventParams;
		}
	}
}

template<typename ChannelType>
struct TCurveChannelKeyMenuExtension : TSharedFromThis<TCurveChannelKeyMenuExtension<ChannelType>>
{
	using ChannelValueType = typename ChannelType::ChannelValueType;

	TCurveChannelKeyMenuExtension(TWeakPtr<ISequencer> InSequencer, TArray<TExtendKeyMenuParams<ChannelType>>&& InChannels)
		: WeakSequencer(InSequencer)
		, ChannelAndHandles(MoveTemp(InChannels))
	{}

	void ExtendMenu(FMenuBuilder& MenuBuilder)
	{
		ISequencer* SequencerPtr = WeakSequencer.Pin().Get();
		if (!SequencerPtr)
		{
			return;
		}

		TSharedRef<TCurveChannelKeyMenuExtension<ChannelType>> SharedThis = this->AsShared();

		MenuBuilder.BeginSection("SequencerInterpolation", LOCTEXT("KeyInterpolationMenu", "Key Interpolation"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationSmartAuto", "Cubic (Smart Auto)"),
				LOCTEXT("SetKeyInterpolationSmartAutoTooltip", "Set key interpolation to smart auto"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeySmartAuto"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis] { SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_SmartAuto); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis] { return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_SmartAuto); })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationAuto", "Cubic (Auto)"),
				LOCTEXT("SetKeyInterpolationAutoTooltip", "Set key interpolation to auto"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyAuto"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis]{ SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_Auto); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis]{ return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_Auto); }) ),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationUser", "Cubic (User)"),
				LOCTEXT("SetKeyInterpolationUserTooltip", "Set key interpolation to user"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyUser"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis]{ SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_User); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis]{ return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_User); }) ),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationBreak", "Cubic (Break)"),
				LOCTEXT("SetKeyInterpolationBreakTooltip", "Set key interpolation to break"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyBreak"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis]{ SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_Break); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis]{ return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_Break); }) ),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationLinear", "Linear"),
				LOCTEXT("SetKeyInterpolationLinearTooltip", "Set key interpolation to linear"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyLinear"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis]{ SharedThis->SetInterpTangentMode(RCIM_Linear, RCTM_Auto); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis]{ return SharedThis->IsInterpTangentModeSelected(RCIM_Linear, RCTM_Auto); }) ),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetKeyInterpolationConstant", "Constant"),
				LOCTEXT("SetKeyInterpolationConstantTooltip", "Set key interpolation to constant"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyConstant"),
				FUIAction(
					FExecuteAction::CreateLambda([SharedThis]{ SharedThis->SetInterpTangentMode(RCIM_Constant, RCTM_Auto); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([SharedThis]{ return SharedThis->IsInterpTangentModeSelected(RCIM_Constant, RCTM_Auto); }) ),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
		MenuBuilder.EndSection(); // SequencerInterpolation
	}

	void SetInterpTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode)
	{
		FScopedTransaction SetInterpTangentModeTransaction(NSLOCTEXT("Sequencer", "SetInterpTangentMode_Transaction", "Set Interpolation and Tangent Mode"));
		bool bAnythingChanged = false;

		for (const TExtendKeyMenuParams<ChannelType>& Channel : ChannelAndHandles)
		{
			if (UMovieSceneSignedObject* OwningObject = Cast<UMovieSceneSignedObject>(Channel.WeakOwner.Get()))
			{
				OwningObject->Modify();
			}

			if (ChannelType* ChannelPtr = Channel.Channel.Get())
			{
				TMovieSceneChannelData<ChannelValueType> ChannelData = ChannelPtr->GetData();
				TArrayView<ChannelValueType> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : Channel.Handles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE)
					{
						Values[KeyIndex].InterpMode = InterpMode;
						Values[KeyIndex].TangentMode = TangentMode;
						bAnythingChanged = true;
					}
				}

				ChannelPtr->AutoSetTangents();
			}
		}

		if (bAnythingChanged)
		{
			if (ISequencer* Sequencer = WeakSequencer.Pin().Get())
			{
				Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
			}
		}
	}

	bool IsInterpTangentModeSelected(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode) const
	{
		for (const TExtendKeyMenuParams<ChannelType>& Channel : ChannelAndHandles)
		{
			ChannelType* ChannelPtr = Channel.Channel.Get();
			if (ChannelPtr)
			{
				TMovieSceneChannelData<ChannelValueType> ChannelData = ChannelPtr->GetData();
				TArrayView<ChannelValueType> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : Channel.Handles)
				{
					int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex == INDEX_NONE || Values[KeyIndex].InterpMode != InterpMode || Values[KeyIndex].TangentMode != TangentMode)
					{
						return false;
					}
				}
			}
		}
		return true;
	}

private:

	/** Hidden AsShared() methods to prevent CreateSP delegate use since this extender disappears with its menu. */
	using TSharedFromThis<TCurveChannelKeyMenuExtension<ChannelType>>::AsShared;

	TWeakPtr<ISequencer> WeakSequencer;
	TArray<TExtendKeyMenuParams<ChannelType>> ChannelAndHandles;
};

struct FCurveChannelSectionMenuExtension : TSharedFromThis<FCurveChannelSectionMenuExtension>, ISidebarChannelExtension
{
	static TSharedRef<FCurveChannelSectionMenuExtension> GetOrCreate(TWeakPtr<ISequencer> InSequencer)
	{
		TSharedPtr<FCurveChannelSectionMenuExtension> CurrentExtension = WeakCurrentExtension.Pin();
		if (!CurrentExtension)
		{
			CurrentExtension = MakeShared<FCurveChannelSectionMenuExtension>(InSequencer);
			WeakCurrentExtension = CurrentExtension;
		}
		else
		{
			ensure(CurrentExtension->NumCurveChannelTypes > 0);
			ensure(CurrentExtension->WeakSequencer == InSequencer);
		}
		return CurrentExtension.ToSharedRef();
	}

	FCurveChannelSectionMenuExtension(TWeakPtr<ISequencer> InSequencer)
		: WeakSequencer(InSequencer)
		, NumCurveChannelTypes(0)
		, bMenusAdded(false)
	{
	}
	virtual ~FCurveChannelSectionMenuExtension() {}

	void AddSections(const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections)
	{
		WeakSections = TSet(InWeakSections);

		++NumCurveChannelTypes;
	}

	virtual TSharedPtr<ISidebarChannelExtension> ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu) override
	{
		--NumCurveChannelTypes;

		if (bMenusAdded)
		{
			// Only add menus once -- not once per curve channel type (float, double, etc)
			return nullptr;
		}

		bMenusAdded = true;

		ISequencer* SequencerPtr = WeakSequencer.Pin().Get();
		if (!SequencerPtr)
		{
			return nullptr;
		}

		TSharedRef<FCurveChannelSectionMenuExtension> SharedThis = this->AsShared();

		MenuBuilder.AddSubMenu(
			LOCTEXT("CurveChannelsMenuLabel", "Curve Channels"),
			LOCTEXT("CurveChannelsMenuToolTip", "Edit parameters for curve channels"),
			FNewMenuDelegate::CreateLambda([SharedThis](FMenuBuilder& SubMenuBuilder)
			{
				SubMenuBuilder.AddSubMenu(
						LOCTEXT("SetPreInfinityExtrap", "Pre-Infinity"),
						LOCTEXT("SetPreInfinityExtrapTooltip", "Set pre-infinity extrapolation"),
						FNewMenuDelegate::CreateLambda([SharedThis](FMenuBuilder& SubMenuBuilder){ SharedThis->AddExtrapolationMenu(SubMenuBuilder, true); })
						);

				SubMenuBuilder.AddSubMenu(
						LOCTEXT("SetPostInfinityExtrap", "Post-Infinity"),
						LOCTEXT("SetPostInfinityExtrapTooltip", "Set post-infinity extrapolation"),
						FNewMenuDelegate::CreateLambda([SharedThis](FMenuBuilder& SubMenuBuilder){ SharedThis->AddExtrapolationMenu(SubMenuBuilder, false); })
						);

				SubMenuBuilder.AddSubMenu(
						LOCTEXT("DisplayOpyions", "Display"),
						LOCTEXT("DisplayOptionsTooltip", "Display options"),
						FNewMenuDelegate::CreateLambda([SharedThis](FMenuBuilder& SubMenuBuilder){ SharedThis->AddDisplayOptionsMenu(SubMenuBuilder); })
						);

				if (SharedThis->CanInterpolateLinearKeys())
				{
					SubMenuBuilder.AddMenuEntry(
							LOCTEXT("InterpolateLinearKeys", "Interpolate Linear Keys"),
							LOCTEXT("InterpolateLinearKeysTooltip", "Interpolate linear keys"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([SharedThis] { SharedThis->ToggleInterpolateLinearKeys(); }),
								FCanExecuteAction(),
								FGetActionCheckState::CreateLambda([SharedThis] { return SharedThis->IsInterpolateLinearKeys(); })
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton
						);
				}
			}));

		return SharedThis;
	}

	void AddDisplayOptionsMenu(FMenuBuilder& MenuBuilder)
	{
		TSharedRef<FCurveChannelSectionMenuExtension> SharedThis = this->AsShared();

		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		if (!Sequencer)
		{
			return;
		}

		USequencerSettings* Settings = Sequencer->GetSequencerSettings();
		if (!Settings)
		{
			return;
		}

		// Menu entry for key area height
		auto OnKeyAreaHeightChanged = [=](int32 NewValue) { Settings->SetKeyAreaHeightWithCurves((float)NewValue); };
		auto GetKeyAreaHeight = [=]() { return (int)Settings->GetKeyAreaHeightWithCurves(); };

		auto OnKeyAreaCurveNormalized = [=](FString KeyAreaName) 
		{ 
			if (Settings->HasKeyAreaCurveExtents(KeyAreaName))
			{	
				Settings->RemoveKeyAreaCurveExtents(KeyAreaName);
			}
			else
			{
				// Initialize to some arbitrary value
				Settings->SetKeyAreaCurveExtents(KeyAreaName, 0.f, 6.f); 
			}
		};
		auto GetKeyAreaCurveNormalized = [=](FString KeyAreaName) { return !Settings->HasKeyAreaCurveExtents(KeyAreaName); };

		auto OnKeyAreaCurveMinChanged = [=](double NewValue, FString KeyAreaName) { double CurveMin = 0.f; double CurveMax = 0.f; Settings->GetKeyAreaCurveExtents(KeyAreaName, CurveMin, CurveMax); Settings->SetKeyAreaCurveExtents(KeyAreaName, NewValue, CurveMax); };
		auto GetKeyAreaCurveMin = [=](FString KeyAreaName) { double CurveMin = 0.f; double CurveMax = 0.f; Settings->GetKeyAreaCurveExtents(KeyAreaName, CurveMin, CurveMax); return CurveMin; };

		auto OnKeyAreaCurveMaxChanged = [=](double NewValue, FString KeyAreaName) { double CurveMin = 0.f; double CurveMax = 0.f; Settings->GetKeyAreaCurveExtents(KeyAreaName, CurveMin, CurveMax); Settings->SetKeyAreaCurveExtents(KeyAreaName, CurveMin, NewValue); };
		auto GetKeyAreaCurveMax = [=](FString KeyAreaName) { double CurveMin = 0.f; double CurveMax = 0.f; Settings->GetKeyAreaCurveExtents(KeyAreaName, CurveMin, CurveMax); return CurveMax; };

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleShowCurve", "Show Curve"),
			LOCTEXT("ToggleShowCurveTooltip", "Toggle showing the curve in the track area"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis]{ SharedThis->ToggleShowCurve(); }),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([SharedThis]{ return SharedThis->IsShowCurve(); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);	
		
		FString KeyAreaName;
		TArray<const IKeyArea*> SelectedKeyAreas;
		Sequencer->GetSelectedKeyAreas(SelectedKeyAreas);
		for (const IKeyArea* KeyArea : SelectedKeyAreas)
		{			
			if (KeyArea)
			{
				KeyAreaName = KeyArea->GetName().ToString();
				break;
			}
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleKeyAreaCurveNormalized", "Key Area Curve Normalized"),
			LOCTEXT("ToggleKeyAreaCurveNormalizedTooltip", "Toggle showing the curve in the track area as normalized"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=] { OnKeyAreaCurveNormalized(KeyAreaName); }),
				FCanExecuteAction(FCanExecuteAction::CreateLambda([SharedThis]{ return SharedThis->IsAnyShowCurve(); })),
				FIsActionChecked::CreateLambda([=] { return GetKeyAreaCurveNormalized(KeyAreaName); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);	

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(50.f)
					.IsEnabled_Lambda([=]() { return SharedThis->IsAnyShowCurve() && Settings->HasKeyAreaCurveExtents(KeyAreaName); })
					[
						SNew(SSpinBox<double>)
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueCommitted_Lambda([=](double NewValue, ETextCommit::Type CommitType) { OnKeyAreaCurveMinChanged(NewValue, KeyAreaName); })
						.OnValueChanged_Lambda([=](double NewValue) { OnKeyAreaCurveMinChanged(NewValue, KeyAreaName); })
						.Value_Lambda([=]() -> double { return GetKeyAreaCurveMin(KeyAreaName); })
					]
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(50.f)
					.IsEnabled_Lambda([=]() { return SharedThis->IsAnyShowCurve() && Settings->HasKeyAreaCurveExtents(KeyAreaName); })
					[
						SNew(SSpinBox<double>)
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueCommitted_Lambda([=](double NewValue, ETextCommit::Type CommitType) { OnKeyAreaCurveMaxChanged(NewValue, KeyAreaName); })
						.OnValueChanged_Lambda([=](double NewValue) { OnKeyAreaCurveMaxChanged(NewValue, KeyAreaName); })
						.Value_Lambda([=]() -> double { return GetKeyAreaCurveMax(KeyAreaName); })
					]
				],
				LOCTEXT("KeyAreaCurveRangeText", "Key Area Curve Range")
		);		
		
		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(50.f)
					[
						SNew(SSpinBox<int32>)
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueCommitted_Lambda([=](int32 Value, ETextCommit::Type CommitType) { OnKeyAreaHeightChanged(Value); })
						.OnValueChanged_Lambda(OnKeyAreaHeightChanged)
						.MinValue(15)
						.MaxValue(300)
						.Value_Lambda([=]() -> int32 { return GetKeyAreaHeight(); })
					]
				],
				LOCTEXT("KeyAreaHeightText", "Key Area Height")
		);
	}

	void AddExtrapolationMenu(FMenuBuilder& MenuBuilder, bool bPreInfinity)
	{
		TSharedRef<FCurveChannelSectionMenuExtension> SharedThis = this->AsShared();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetExtrapCycle", "Cycle"),
			LOCTEXT("SetExtrapCycleTooltip", "Set extrapolation cycle"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis, bPreInfinity]{ SharedThis->SetExtrapolationMode(RCCE_Cycle, bPreInfinity); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([SharedThis, bPreInfinity]{ return SharedThis->IsExtrapolationModeSelected(RCCE_Cycle, bPreInfinity); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetExtrapCycleWithOffset", "Cycle with Offset"),
			LOCTEXT("SetExtrapCycleWithOffsetTooltip", "Set extrapolation cycle with offset"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis, bPreInfinity]{ SharedThis->SetExtrapolationMode(RCCE_CycleWithOffset, bPreInfinity); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([SharedThis, bPreInfinity]{ return SharedThis->IsExtrapolationModeSelected(RCCE_CycleWithOffset, bPreInfinity); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetExtrapOscillate", "Oscillate"),
			LOCTEXT("SetExtrapOscillateTooltip", "Set extrapolation oscillate"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis, bPreInfinity]{ SharedThis->SetExtrapolationMode(RCCE_Oscillate, bPreInfinity); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([SharedThis, bPreInfinity]{ return SharedThis->IsExtrapolationModeSelected(RCCE_Oscillate, bPreInfinity); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetExtrapLinear", "Linear"),
			LOCTEXT("SetExtrapLinearTooltip", "Set extrapolation linear"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis, bPreInfinity]{ SharedThis->SetExtrapolationMode(RCCE_Linear, bPreInfinity); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([SharedThis, bPreInfinity]{ return SharedThis->IsExtrapolationModeSelected(RCCE_Linear, bPreInfinity); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetExtrapConstant", "Constant"),
			LOCTEXT("SetExtrapConstantTooltip", "Set extrapolation constant"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SharedThis, bPreInfinity]{ SharedThis->SetExtrapolationMode(RCCE_Constant, bPreInfinity); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([SharedThis, bPreInfinity]{ return SharedThis->IsExtrapolationModeSelected(RCCE_Constant, bPreInfinity); })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	void GetChannels(TArray<FMovieSceneFloatChannel*>& FloatChannels, TArray<FMovieSceneDoubleChannel*>& DoubleChannels,
		TArray<FMovieSceneIntegerChannel*>& IntegerChannels, TArray<FMovieSceneBoolChannel*>& BoolChannels,
		TArray<FMovieSceneByteChannel*>& ByteChannels) const
	{
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		if (!Sequencer)
		{
			return;
		}

		// Get selected channels
		TArray<const IKeyArea*> KeyAreas;
		Sequencer->GetSelectedKeyAreas(KeyAreas);
		for (const IKeyArea* KeyArea : KeyAreas)
		{
			FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
			if (Handle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
			{
				FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(Handle.Get());
				FloatChannels.Add(Channel);
			}
			else if (Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
			{
				FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(Handle.Get());
				DoubleChannels.Add(Channel);
			}
			else if (Handle.GetChannelTypeName() == FMovieSceneIntegerChannel::StaticStruct()->GetFName())
			{
				FMovieSceneIntegerChannel* Channel = static_cast<FMovieSceneIntegerChannel*>(Handle.Get());
				IntegerChannels.Add(Channel);
			}
			else if (Handle.GetChannelTypeName() == FMovieSceneBoolChannel::StaticStruct()->GetFName())
			{
				FMovieSceneBoolChannel* Channel = static_cast<FMovieSceneBoolChannel*>(Handle.Get());
				BoolChannels.Add(Channel);
			}
			else if (Handle.GetChannelTypeName() == FMovieSceneByteChannel::StaticStruct()->GetFName())
			{
				FMovieSceneByteChannel* Channel = static_cast<FMovieSceneByteChannel*>(Handle.Get());
				ByteChannels.Add(Channel);
			}
		}

		// Otherwise, the channels of all the sections
		if (FloatChannels.Num() + DoubleChannels.Num() + IntegerChannels.Num() + BoolChannels.Num() + ByteChannels.Num() == 0)
		{
			for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
			{
				if (UMovieSceneSection* Section = WeakSection.Get())
				{
					FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
					for (FMovieSceneFloatChannel* Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
					{
						FloatChannels.Add(Channel);
					}
					for (FMovieSceneDoubleChannel* Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
					{
						DoubleChannels.Add(Channel);
					}
					for (FMovieSceneIntegerChannel* Channel : ChannelProxy.GetChannels<FMovieSceneIntegerChannel>())
					{
						IntegerChannels.Add(Channel);
					}
					for (FMovieSceneBoolChannel* Channel : ChannelProxy.GetChannels<FMovieSceneBoolChannel>())
					{
						BoolChannels.Add(Channel);
					}
					for (FMovieSceneByteChannel* Channel : ChannelProxy.GetChannels<FMovieSceneByteChannel>())
					{
						ByteChannels.Add(Channel);
					}
				}
			}
		}
	}

	void SetExtrapolationMode(ERichCurveExtrapolation ExtrapMode, bool bPreInfinity)
	{
		TArray<FMovieSceneFloatChannel*> FloatChannels;
		TArray<FMovieSceneDoubleChannel*> DoubleChannels;
		TArray<FMovieSceneIntegerChannel*> IntegerChannels;
		TArray<FMovieSceneBoolChannel*> BoolChannels;
		TArray<FMovieSceneByteChannel*> ByteChannels;

		GetChannels(FloatChannels, DoubleChannels, IntegerChannels, BoolChannels, ByteChannels);

		if (FloatChannels.Num() + DoubleChannels.Num() + IntegerChannels.Num() + BoolChannels.Num() + ByteChannels.Num() == 0)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("SetExtrapolationMode_Transaction", "Set Extrapolation Mode"));

		bool bAnythingChanged = false;

		// Modify all sections
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				Section->Modify();
			}
		}

		// Apply to all channels
		for (FMovieSceneFloatChannel* Channel : FloatChannels)
		{
			TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			DestExtrap = ExtrapMode;
			bAnythingChanged = true;
		}
		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			DestExtrap = ExtrapMode;
			bAnythingChanged = true;
		}
		for (FMovieSceneIntegerChannel* Channel : IntegerChannels)
		{
			TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			DestExtrap = ExtrapMode;
			bAnythingChanged = true;
		}
		for (FMovieSceneBoolChannel* Channel : BoolChannels)
		{
			TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			DestExtrap = ExtrapMode;
			bAnythingChanged = true;
		}
		for (FMovieSceneByteChannel* Channel : ByteChannels)
		{
			TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			DestExtrap = ExtrapMode;
			bAnythingChanged = true;
		}

		if (bAnythingChanged)
		{
			if (ISequencer* Sequencer = WeakSequencer.Pin().Get())
			{
				Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
			}
		}
		else
		{
			Transaction.Cancel();
		}
	}

	bool IsExtrapolationModeSelected(ERichCurveExtrapolation ExtrapMode, bool bPreInfinity) const
	{
		TArray<FMovieSceneFloatChannel*> FloatChannels;
		TArray<FMovieSceneDoubleChannel*> DoubleChannels;
		TArray<FMovieSceneIntegerChannel*> IntegerChannels;
		TArray<FMovieSceneBoolChannel*> BoolChannels;
		TArray<FMovieSceneByteChannel*> ByteChannels;

		GetChannels(FloatChannels, DoubleChannels, IntegerChannels, BoolChannels, ByteChannels);

		for (FMovieSceneFloatChannel* Channel : FloatChannels)
		{
			ERichCurveExtrapolation SourceExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			if (SourceExtrap != ExtrapMode)
			{
				return false;
			}
		}
		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			ERichCurveExtrapolation SourceExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			if (SourceExtrap != ExtrapMode)
			{
				return false;
			}
		}
		for (FMovieSceneIntegerChannel* Channel : IntegerChannels)
		{
			ERichCurveExtrapolation SourceExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			if (SourceExtrap != ExtrapMode)
			{
				return false;
			}
		}
		for (FMovieSceneBoolChannel* Channel : BoolChannels)
		{
			ERichCurveExtrapolation SourceExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			if (SourceExtrap != ExtrapMode)
			{
				return false;
			}
		}
		for (FMovieSceneByteChannel* Channel : ByteChannels)
		{
			ERichCurveExtrapolation SourceExtrap = bPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
			if (SourceExtrap != ExtrapMode)
			{
				return false;
			}
		}

		return true;
	}
	
	bool CanInterpolateLinearKeys() const
	{
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		if (!Sequencer)
		{
			return false;
		}

		TArray<FMovieSceneIntegerChannel*> IntegerChannels;
		GetTypedChannels<FMovieSceneIntegerChannel>(Sequencer, WeakSections, IntegerChannels);

		return IntegerChannels.Num() > 0;
	}

	void ToggleInterpolateLinearKeys()
	{
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		if (!Sequencer)
		{
			return;
		}

		TArray<FMovieSceneIntegerChannel*> IntegerChannels;
		GetTypedChannels<FMovieSceneIntegerChannel>(Sequencer, WeakSections, IntegerChannels);

		FScopedTransaction Transaction(LOCTEXT("ToggleInterpolateLinearKeys_Transaction", "Toggle Interpolate Linear Keys"));

		bool bAnythingChanged = false;

		// Modify all sections
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				Section->Modify();
			}
		}

		for (FMovieSceneIntegerChannel* Channel : IntegerChannels)
		{
			bAnythingChanged = true;
			Channel->bInterpolateLinearKeys = !Channel->bInterpolateLinearKeys;
		}

		if (!bAnythingChanged)
		{
			Transaction.Cancel();
		}
	}

	ECheckBoxState IsInterpolateLinearKeys()
	{
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		if (!Sequencer)
		{
			return ECheckBoxState::Undetermined;
		}

		TArray<FMovieSceneIntegerChannel*> IntegerChannels;
		GetTypedChannels<FMovieSceneIntegerChannel>(Sequencer, WeakSections, IntegerChannels);

		int32 NumInterpolatedAndNotInterpolated[2] = { 0, 0 };

		for (FMovieSceneIntegerChannel* Channel : IntegerChannels)
		{
			NumInterpolatedAndNotInterpolated[Channel->bInterpolateLinearKeys ? 0 : 1]++;
		}

		if (NumInterpolatedAndNotInterpolated[0] == 0 && NumInterpolatedAndNotInterpolated[1] > 0)  // No curve showed, some hidden
		{
			return ECheckBoxState::Unchecked;
		}
		else if (NumInterpolatedAndNotInterpolated[0] > 0 && NumInterpolatedAndNotInterpolated[1] == 0) // Some curves showed, none hidden
		{
			return ECheckBoxState::Checked;
		}
		return ECheckBoxState::Undetermined;  // Mixed states, or no curves
	}

	void ToggleShowCurve()
	{
		const ECheckBoxState CurrentState = IsShowCurve();
		const bool bShowCurve = (CurrentState != ECheckBoxState::Checked); // If unchecked or mixed, check it

		FScopedTransaction Transaction(LOCTEXT("ToggleShowCurve_Transaction", "Toggle Show Curve"));

		bool bAnythingChanged = false;

		// Modify all sections

		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				Section->Modify();
			}
		}

		// Apply to all channels
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				for (FMovieSceneFloatChannel* Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
				{
					if (Channel)
					{
						Channel->SetShowCurve(bShowCurve);
						bAnythingChanged = true;
					}
				}
				for (FMovieSceneDoubleChannel* Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
				{
					if (Channel)
					{
						Channel->SetShowCurve(bShowCurve);
						bAnythingChanged = true;
					}
				}
			}
		}

		if (!bAnythingChanged)
		{
			Transaction.Cancel();
		}
	}

	ECheckBoxState IsShowCurve() const
	{
		int32 NumShowedAndHidden[2] = { 0, 0 };
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				for (FMovieSceneFloatChannel* Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
				{
					if (Channel)
					{
						NumShowedAndHidden[Channel->GetShowCurve() ? 0 : 1]++;
					}
				}
				for (FMovieSceneDoubleChannel* Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
				{
					if (Channel)
					{
						NumShowedAndHidden[Channel->GetShowCurve() ? 0 : 1]++;
					}
				}
			}
		}

		if (NumShowedAndHidden[0] == 0 && NumShowedAndHidden[1] > 0)  // No curve showed, some hidden
		{
			return ECheckBoxState::Unchecked;
		}
		else if (NumShowedAndHidden[0] > 0 && NumShowedAndHidden[1] == 0) // Some curves showed, none hidden
		{
			return ECheckBoxState::Checked;
		}
		return ECheckBoxState::Undetermined;  // Mixed states, or no curves
	}

	bool IsAnyShowCurve() const
	{
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				for (FMovieSceneFloatChannel* Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
				{
					if (Channel && Channel->GetShowCurve())
					{
						return true;
					}
				}
				for (FMovieSceneDoubleChannel* Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
				{
					if (Channel && Channel->GetShowCurve())
					{
						return true;
					}
				}
			}
		}
		return false;
	}

private:

	/** Hidden AsShared() methods to prevent CreateSP delegate use since this extender disappears with its menu. */
	using TSharedFromThis<FCurveChannelSectionMenuExtension>::AsShared;

	/** Held weekly so that only the context menu owns the instance, and it gets naturally deleted when the menu closes */
	static TWeakPtr<FCurveChannelSectionMenuExtension> WeakCurrentExtension;

	TWeakPtr<ISequencer> WeakSequencer;
	TSet<TWeakObjectPtr<UMovieSceneSection>> WeakSections;
	int32 NumCurveChannelTypes;
	bool bMenusAdded;
};

TWeakPtr<FCurveChannelSectionMenuExtension> FCurveChannelSectionMenuExtension::WeakCurrentExtension;

void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneFloatChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	MenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, true); }));
}

void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneDoubleChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	MenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, true); }));
}

void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneIntegerChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	MenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, true); }));
}

void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneBoolChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	MenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, true); }));
}

void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneByteChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	MenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, true); }));
}

TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneFloatChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	InMenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, false); }));

	return Extension;
}

TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneDoubleChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	InMenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, false); }));

	return Extension;
}

TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneIntegerChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	InMenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, false); }));

	return Extension;
}

TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneBoolChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	InMenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, false); }));

	return Extension;
}

TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneByteChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer)
{
	TSharedRef<FCurveChannelSectionMenuExtension> Extension = FCurveChannelSectionMenuExtension::GetOrCreate(InWeakSequencer);
	Extension->AddSections(InWeakSections);

	InMenuExtender->AddMenuExtension("SequencerChannels", EExtensionHook::First, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder, false); }));

	return Extension;
}

void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneFloatChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer)
{
	using ExtensionType = TCurveChannelKeyMenuExtension<FMovieSceneFloatChannel>;
	TSharedRef<ExtensionType> Extension = MakeShared<ExtensionType>(InSequencer, MoveTemp(Channels));

	MenuExtender->AddMenuExtension("SequencerKeyEdit", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder); }));
}

void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneDoubleChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer)
{
	using ExtensionType = TCurveChannelKeyMenuExtension<FMovieSceneDoubleChannel>;
	TSharedRef<ExtensionType> Extension = MakeShared<ExtensionType>(InSequencer, MoveTemp(Channels));

	MenuExtender->AddMenuExtension("SequencerKeyEdit", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder); }));
}

void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneTimeWarpChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer)
{
	using ExtensionType = TCurveChannelKeyMenuExtension<FMovieSceneTimeWarpChannel>;
	TSharedRef<ExtensionType> Extension = MakeShared<ExtensionType>(InSequencer, MoveTemp(Channels));

	MenuExtender->AddMenuExtension("SequencerKeyEdit", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder) { Extension->ExtendMenu(MenuBuilder); }));
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneFloatChannel>& FloatChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (FloatChannel.GetMetaData() != nullptr && FloatChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::MovieSceneTools::TInvertedBezierChannelCurveModel<FFloatChannelCurveModel>>(FloatChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FFloatChannelCurveModel>(FloatChannel, Params.OwningSection, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>& DoubleChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (DoubleChannel.GetMetaData() != nullptr && DoubleChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::MovieSceneTools::TInvertedBezierChannelCurveModel<FDoubleChannelCurveModel>>(DoubleChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FDoubleChannelCurveModel>(DoubleChannel, Params.OwningSection, Params.OwningObject, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneIntegerChannel>& IntegerChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (IntegerChannel.GetMetaData() != nullptr && IntegerChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::CurveEditor::TInvertedCurveModel<FIntegerChannelCurveModel>>(IntegerChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FIntegerChannelCurveModel>(IntegerChannel, Params.OwningSection, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneBoolChannel>& BoolChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (BoolChannel.GetMetaData() != nullptr && BoolChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::CurveEditor::TInvertedCurveModel<FBoolChannelCurveModel>>(BoolChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FBoolChannelCurveModel>(BoolChannel, Params.OwningSection, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneByteChannel>& ByteChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (ByteChannel.GetMetaData() != nullptr && ByteChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::CurveEditor::TInvertedCurveModel<FByteChannelCurveModel>>(ByteChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FByteChannelCurveModel>(ByteChannel, Params.OwningSection, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneEventChannel>& EventChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (EventChannel.GetMetaData() != nullptr && EventChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::CurveEditor::TInvertedCurveModel<FEventChannelCurveModel>>(EventChannel, Params.OwningSection, Params.Sequencer);
	}
	return MakeUnique<FEventChannelCurveModel>(EventChannel, Params.OwningSection, Params.Sequencer);
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& TimeWarpChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (TimeWarpChannel.GetMetaData() != nullptr && TimeWarpChannel.GetMetaData()->bInvertValue)
	{
		return MakeUnique<UE::CurveEditor::TInvertedCurveModel<FTimeWarpChannelCurveModel>>(TimeWarpChannel, Params.OwningSection, Params.OwningObject, Params.Sequencer);
	}
	return MakeUnique<FTimeWarpChannelCurveModel>(TimeWarpChannel, Params.OwningSection, Params.OwningObject, Params.Sequencer);
}

bool ShouldShowCurve(const FMovieSceneFloatChannel* Channel, UMovieSceneSection* InSection)
{
	return Channel->GetShowCurve();
}
bool ShouldShowCurve(const FMovieSceneDoubleChannel* Channel, UMovieSceneSection* InSection)
{
	return Channel->GetShowCurve();
}

TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName)
{
	return MakeShared<UE::Sequencer::FTimeWarpChannelModel>(InChannelName, InSection.GetSectionInterface(), InChannelHandle);
}


#undef LOCTEXT_NAMESPACE
