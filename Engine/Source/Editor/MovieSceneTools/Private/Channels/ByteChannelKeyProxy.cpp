// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/ByteChannelKeyProxy.h"

#include "Channels/MovieSceneByteChannel.h"
#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ByteChannelKeyProxy)

struct FPropertyChangedEvent;

void UByteChannelKeyProxy::Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneByteChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSection> InWeakSection)
{
	KeyHandle     = InKeyHandle;
	ChannelHandle = InChannelHandle;
	WeakSection   = InWeakSection;
}

void UByteChannelKeyProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnProxyValueChanged(ChannelHandle, WeakSection.Get(), KeyHandle, Value, Time);
}

void UByteChannelKeyProxy::UpdateValuesFromRawData()
{
	RefreshCurrentValue(ChannelHandle, KeyHandle, Value, Time);
}
