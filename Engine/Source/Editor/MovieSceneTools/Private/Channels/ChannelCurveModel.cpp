// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/ChannelCurveModel.h"

#include "Algo/BinarySearch.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieScene.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Algo/IndexOf.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"

class UObject;
struct FMovieSceneChannelMetaData;

template <class ChannelType, class ChannelValue, class KeyType>
FChannelCurveModel<ChannelType, ChannelValue, KeyType>::FChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
{
	ChannelHandle = InChannel;
	WeakSection = OwningSection;
	WeakSequencer = InWeakSequencer;
	LastSignature = OwningSection->GetSignature();

	if (FMovieSceneChannelProxy* ChannelProxy = InChannel.GetChannelProxy())
	{
		OnDestroyHandle = ChannelProxy->OnDestroy.AddRaw(this, &FChannelCurveModel<ChannelType, ChannelValue, KeyType>::FixupCurve);
	}

	SupportedViews = ECurveEditorViewID::Absolute | ECurveEditorViewID::Normalized | ECurveEditorViewID::Stacked;
}

template <class ChannelType, class ChannelValue, class KeyType>
FChannelCurveModel<ChannelType, ChannelValue, KeyType>::FChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer)
	: FChannelCurveModel(InChannel, InOwningSection, InWeakSequencer)
{
	WeakOwningObject = InOwningObject;
}

template <class ChannelType, class ChannelValue, class KeyType>
FChannelCurveModel<ChannelType, ChannelValue, KeyType>::~FChannelCurveModel()
{
	if (FMovieSceneChannelProxy* ChannelProxy = ChannelHandle.GetChannelProxy())
	{
		ChannelProxy->OnDestroy.Remove(OnDestroyHandle);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
FTransform2d FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetCurveTransform() const
{
	FTransform2d Transform;

	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
	const UMovieSceneSection*         Section  = WeakSection.Get();

	if (MetaData && Section)
	{
		FFrameNumber Offset = MetaData->GetOffsetTime(Section);
		if (Offset != 0)
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			Transform = Concatenate(Transform, FVector2d(-Offset / TickResolution, 0.0));
		}
	}
	return Transform;
}


template <class ChannelType, class ChannelValue, class KeyType>
const void* FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetCurve() const
{
	return ChannelHandle.Get();
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::Modify()
{
	if (UObject* Object = GetOwningObject())
	{
		Object->Modify();
	}
	LastSignature.Invalidate();
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::OnCloseRootChangeScope()
{
	// The events are no longer deferred... invoke the delegates we deferred.
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		InvokePostChannelChangedEvents(Section);
	}
}


template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<const ChannelValue> Values = ChannelData.GetValues();

		const double DisplayOffset = GetInputDisplayOffset();
		const double StartTimeSeconds = ScreenSpace.GetInputMin() - DisplayOffset;
		const double EndTimeSeconds = ScreenSpace.GetInputMax() - DisplayOffset;

		const FFrameNumber StartFrame = (StartTimeSeconds * TickResolution).FloorToFrame();
		const FFrameNumber EndFrame = (EndTimeSeconds * TickResolution).CeilToFrame();

		const int32 StartingIndex = [&Times, &StartFrame]()
			{
				const int32 StartingFrameIndex = Algo::LowerBound(Times, StartFrame);
				if (Times.IsValidIndex(StartingFrameIndex - 1))
				{
					return StartingFrameIndex - 1;
				}
				
				return StartingFrameIndex;
			}();

		const int32 EndingIndex = Algo::UpperBound(Times, EndFrame);

		// Add the lower bound of the visible space
		const bool bValidRange = StartingIndex < EndingIndex;
		//if we aren't just doing the default constant then we need to sample
		const bool PreNotConstant = (Channel->PreInfinityExtrap != RCCE_None && Channel->PreInfinityExtrap != RCCE_Constant);
		const bool PostNotConstant = (Channel->PostInfinityExtrap != RCCE_None && Channel->PostInfinityExtrap != RCCE_Constant);
		if (bValidRange && (PreNotConstant || PostNotConstant))
		{
			const FFrameRate DisplayResolution = Section->GetTypedOuter<UMovieScene>()->GetDisplayRate();
			const FFrameNumber StartTimeInDisplay = FFrameRate::TransformTime(FFrameTime(StartFrame), TickResolution, DisplayResolution).FloorToFrame();
			const FFrameNumber EndTimeInDisplay = FFrameRate::TransformTime(FFrameTime(EndFrame), TickResolution, DisplayResolution).CeilToFrame();

			double Value = 0.0;
			TOptional<double> PreviousValue;
			for (FFrameNumber DisplayFrameNumber = StartTimeInDisplay; DisplayFrameNumber <= EndTimeInDisplay; ++DisplayFrameNumber)
			{
				FFrameNumber TickFrameNumber = FFrameRate::TransformTime(FFrameTime(DisplayFrameNumber), DisplayResolution, TickResolution).FrameNumber;
				Evaluate(TickFrameNumber / TickResolution, Value);
				if (PreviousValue.IsSet() && PreviousValue.GetValue() != Value)
				{
					OutInterpolatingPoints.Add(MakeTuple(TickFrameNumber / TickResolution, PreviousValue.GetValue()));
				}
				OutInterpolatingPoints.Add(MakeTuple(TickFrameNumber / TickResolution, Value));
				PreviousValue = Value;
			}
		}
		else
		{
			if (bValidRange)
			{
				OutInterpolatingPoints.Add(MakeTuple(StartFrame / TickResolution, GetKeyValue(Values, StartingIndex)));
			}

			TOptional<double> PreviousValue;
			for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
			{
				double Value = GetKeyValue(Values, KeyIndex);
				if (PreviousValue.IsSet() && PreviousValue.GetValue() != Value)
				{
					OutInterpolatingPoints.Add(MakeTuple(Times[KeyIndex] / TickResolution, PreviousValue.GetValue()));
				}

				OutInterpolatingPoints.Add(MakeTuple(Times[KeyIndex] / TickResolution, Value));
				PreviousValue = Value;
			}

			// Add the upper bound of the visible space
			if (bValidRange)
			{
				OutInterpolatingPoints.Add(MakeTuple(EndFrame / TickResolution, GetKeyValue(Values, EndingIndex - 1)));
			}
		}
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<const ChannelValue> Values = ChannelData.GetValues();

		const FFrameNumber StartFrame = MinTime <= MIN_int32 ? MIN_int32 : (MinTime * TickResolution).CeilToFrame();
		const FFrameNumber EndFrame = MaxTime >= MAX_int32 ? MAX_int32 : (MaxTime * TickResolution).FloorToFrame();

		const int32 StartingIndex = Algo::LowerBound(Times, StartFrame);
		const int32 EndingIndex = Algo::UpperBound(Times, EndFrame);

		for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
		{
			if (GetKeyValue(Values, KeyIndex) >= MinValue && GetKeyValue(Values, KeyIndex) <= MaxValue)
			{
				OutKeyHandles.Add(ChannelData.GetHandle(KeyIndex));
			}
		}
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetAllKeys(TArray<FKeyHandle>& OutKeyHandles) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	if (!Channel || !Section)
	{
		return;
	}
	
	TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
	const int32 Num = ChannelData.GetTimes().Num();
	for (int32 KeyIndex = 0; KeyIndex < Num; ++KeyIndex)
	{
		OutKeyHandles.Add(ChannelData.GetHandle(KeyIndex));
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	OutDrawInfo.Brush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
	OutDrawInfo.ScreenSize = FVector2D(10, 10);
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<const ChannelValue> Values = ChannelData.GetValues();

		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				OutKeyPositions[Index].InputValue = Times[KeyIndex] / TickResolution;
				OutKeyPositions[Index].OutputValue = GetKeyValue(Values, KeyIndex);
			}
		}
	}
}

namespace UE::MovieSceneTools::Private
{
/** Finds the index of InKeyHandle if it is positioned after AfterIndex. */
static int32 IndexOfKeyAfter(const TConstArrayView<FKeyHandle>& InHandles, const FKeyHandle& InKeyHandle, int32 AfterIndex)
{
	for (int32 LaterIndex = AfterIndex + 1; LaterIndex < InHandles.Num(); ++LaterIndex)
	{
		if (InHandles[LaterIndex] == InKeyHandle)
		{
			return LaterIndex;
		}
	}
	return INDEX_NONE;
}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	UE::MovieScene::FScopedSignedObjectModifyDefer Defer;

	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (Channel && MetaData && Section && !IsReadOnly())
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		if (Times.IsEmpty())
		{
			return;
		}

		FFrameRate   TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber KeyOffset      = 0;

		// Expand to frame first, then offset
		if (InKeyPositions.Num() > 0)
		{
			double Min = TNumericLimits<double>::Max();
			double Max = TNumericLimits<double>::Lowest();

			for (const FKeyPosition& Value : InKeyPositions)
			{
				Min = FMath::Min(Min, Value.InputValue);
				Max = FMath::Max(Max, Value.InputValue);
			}

			FFrameNumber Offset  = MetaData->GetOffsetTime(Section);
			FFrameNumber MinTime = (Min * TickResolution).RoundToFrame() + Offset;
			FFrameNumber MaxTime = (Max * TickResolution).RoundToFrame() + Offset;

			if (!Section->GetRange().Contains(MinTime))
			{
				// TODO UE-305992 & UE-331831: This may cause Modify() to be called while FScopedCurveChange is active. That may cause ordering issues with undo / redo.
				Section->ExpandToFrame(MinTime);
				KeyOffset += MetaData->GetOffsetTime(Section) - Offset;
			}

			if (Min != Max && !Section->GetRange().Contains(MaxTime))
			{
				// TODO UE-305992 & UE-331831: This may cause Modify() to be called while FScopedCurveChange is active. That may cause ordering issues with undo / redo.
				Section->ExpandToFrame(MaxTime);
			}
		}

		const auto ComputeFrameTime = [&InKeyPositions, &TickResolution, &KeyOffset](int32 Index)
		{
			return (InKeyPositions[Index].InputValue * TickResolution).RoundToFrame() - KeyOffset;
		};
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				const FFrameNumber NewTime = ComputeFrameTime(Index);

				const bool bRemoveDuplicates = ChangeType == EPropertyChangeType::ValueSet;
				// Skip removing a duplicate key handle if it appears in InKeys after our current Index and the x-position will be changed.
				// Example: InKeys { 42, 54 } with current positions { (0.5, OldY), (0.533, OldY2) } and InKeyPositions = { (0.533, NewY1), (0.9, NewY2) }.
				// When we get to key handle 42, MoveKey will detect that key handle 52 already has time 0.533 and attempt to remove it. However, because
				// we'll later move 52 from 0.533 to 0.9, we should skip it.
				const auto CanRemoveDuplicateHandle = [Index, &InKeys, &ChannelData, &ComputeFrameTime](const FKeyHandle& DuplicateKeyHandle)
				{
					const int32 LaterKeyPositionIndex =  UE::MovieSceneTools::Private::IndexOfKeyAfter(InKeys, DuplicateKeyHandle, Index);
					if (LaterKeyPositionIndex != INDEX_NONE)
					{
						const int32 InternalIndex = ChannelData.GetIndex(DuplicateKeyHandle);
						const FFrameNumber& CurrentTime = ChannelData.GetTimes()[InternalIndex];
						const FFrameNumber NewTime = ComputeFrameTime(LaterKeyPositionIndex);
						return CurrentTime != NewTime;
					}
					return false;
				};
				
				KeyIndex = ChannelData.MoveKey(KeyIndex, NewTime, bRemoveDuplicates, CanRemoveDuplicateHandle);
				SetKeyValue(KeyIndex, InKeyPositions[Index].OutputValue);
			}
		}
		
		PostProcessChanges(Channel, Section);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	OutCurveAttributes.SetPreExtrapolation(RCCE_None);
	OutCurveAttributes.SetPostExtrapolation(RCCE_None);
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetTimeRange(double& MinTime, double& MaxTime) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		if (Times.Num() == 0)
		{
			MinTime = 0.f;
			MaxTime = 0.f;
		}
		else
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			double ToTime = TickResolution.AsInterval();
			MinTime = static_cast<double>(Times[0].Value) * ToTime;
			MaxTime = static_cast<double>(Times[Times.Num() - 1].Value) * ToTime;
		}
	}
}


template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetValueRange(double& MinValue, double& MaxValue) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		TArrayView<const ChannelValue> Values = Channel->GetData().GetValues();

		if (Times.Num() == 0)
		{
			// If there are no keys we just use the default value for the channel, defaulting to zero if there is no default.
			MinValue = MaxValue = Channel->GetDefault().Get(0.f);
		}
		else
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			double ToTime = TickResolution.AsInterval();
			int32 LastKeyIndex = Values.Num() - 1;
			MinValue = MaxValue = GetKeyValue(Values, 0);

			for (int32 i = 0; i < Values.Num(); i++)
			{
				double Key = GetKeyValue(Values, i);

				MinValue = FMath::Min(MinValue, Key);
				MaxValue = FMath::Max(MaxValue, Key);
			}
		}
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
int32 FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetNumKeys() const
{
	ChannelType* Channel = ChannelHandle.Get();

	if (Channel)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();

		return Times.Num();
	}

	return 0;
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const
{
	ChannelType* Channel = ChannelHandle.Get();

	if (Channel)
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();

		const int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
		if (KeyIndex != INDEX_NONE)
		{
			if (KeyIndex - 1 >= 0)
			{
				OutPreviousKeyHandle = ChannelData.GetHandle(KeyIndex - 1);
			}

			if (KeyIndex + 1 < ChannelData.GetTimes().Num())
			{
				OutNextKeyHandle = ChannelData.GetHandle(KeyIndex + 1);
			}
		}
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
bool FChannelCurveModel<ChannelType, ChannelValue, KeyType>::Evaluate(double Time, double& OutValue) const
{
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		KeyType ThisValue = 0.f;
		if (Channel->Evaluate(Time * TickResolution, ThisValue))
		{
			OutValue = ThisValue;
			return true;
		}
	}

	return false;
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InKeyAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::AddKeys)
	check(InKeyPositions.Num() == InKeyAttributes.Num() && (!OutKeyHandles || OutKeyHandles->Num() == InKeyPositions.Num()));

	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (Channel && MetaData && Section && !IsReadOnly())
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TArray<FKeyHandle> NewKeyHandles;
		NewKeyHandles.SetNumUninitialized(InKeyPositions.Num());

		FFrameNumber MinFrame = TNumericLimits<FFrameNumber>::Max();
		FFrameNumber MaxFrame = TNumericLimits<FFrameNumber>::Min();
		for (int32 Index = 0; Index < InKeyPositions.Num(); ++Index)
		{
			FKeyPosition Position = InKeyPositions[Index];

			FFrameNumber Time = (Position.InputValue * TickResolution).RoundToFrame();
			MinFrame = FMath::Min(MinFrame, Time);
			MaxFrame = FMath::Max(MaxFrame, Time);

			ChannelValue Value = (ChannelValue)(Position.OutputValue);
			const FKeyHandle NewHandle = [&]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::SetKeyAttributes);
				return ChannelData.UpdateOrAddKey(Time, Value);
			}();
			
			if (NewHandle != FKeyHandle::Invalid())
			{
				NewKeyHandles[Index] = NewHandle;

				if (OutKeyHandles)
				{
					(*OutKeyHandles)[Index] = NewHandle;
				}
			}
		}

		if (InKeyPositions.Num() > 0)
		{
			FFrameNumber Offset = MetaData->GetOffsetTime(Section);
			// TODO UE-305992 & UE-331831: This may cause Modify() to be called while FScopedCurveChange is active. That may cause ordering issues with undo / redo.
			Section->ExpandToFrame(MinFrame + Offset);
		}

		// We reuse SetKeyAttributes here as there is complex logic determining which parts of the attributes are valid to set.
		// For now we need to duplicate the new key handle array due to API mismatch. This will auto calculate tangents if needed.
		SetKeyAttributes(NewKeyHandles, InKeyAttributes);
		PostProcessChanges(Channel, Section);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::RemoveKeys)
	
	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	if (Channel && Section && !IsReadOnly())
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();		
		double CurrentValue = 0.0;
		Evaluate(InCurrentTime, CurrentValue);

		// Iterate in reverse so RemoveAt always operates at or after the current index, avoiding O(n²) index shifting.
		// TMovieSceneChannelData stores keys sorted by time, so back-to-front removal keeps earlier indices stable.
		// InKeys is usually passed in pre-sorted by ascending time, which we abuse here for performance gains.
		for (int32 Index = InKeys.Num() - 1; Index >= 0; --Index)
		{
			const FKeyHandle Handle = InKeys[Index];
			const int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::RemoveKey)
				ChannelData.RemoveKey(KeyIndex);
			}
		}
		if (Channel->GetNumKeys() == 0)
		{
			using CurveValueType = typename ChannelType::CurveValueType;
			CurveValueType ValueAsCurveValue = (CurveValueType)(CurrentValue); //needed for double to other type conversions
			Channel->SetDefault(ValueAsCurveValue);
		}

		PostProcessChanges(Channel, Section);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::ReplaceKeyHandles(
	TConstArrayView<FKeyHandle> InCurrentHandles, TConstArrayView<FKeyHandle> InNewHandles
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::ReplaceKeyHandles)
	if (!ensure(InCurrentHandles.Num() == InNewHandles.Num()))
	{
		return;
	}

	ChannelType* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (Channel && MetaData && Section && !IsReadOnly())
	{
		TMovieSceneChannelData<ChannelValue> ChannelData = Channel->GetData();
		for (int32 Index = 0; Index < InCurrentHandles.Num(); ++Index)
		{
			const int32 KeyIndex = ChannelData.GetIndex(InCurrentHandles[Index]);

			// Skip remapping to keys that are already in use.
			const bool bCannotReplace = KeyIndex == INDEX_NONE || ChannelData.GetIndex(InNewHandles[Index]) != INDEX_NONE;
			if (bCannotReplace)
			{
				continue;
			}

			ChannelData.ReplaceKeyHandle(KeyIndex, InNewHandles[Index]);
		}
		
		PostProcessChanges(Channel, Section);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
bool FChannelCurveModel<ChannelType, ChannelValue, KeyType>::IsReadOnly() const
{
	UMovieSceneSection* Section = WeakSection.Get();
	if (Section)
	{
		return Section->IsReadOnly();
	}

	return false;
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::FixupCurve()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		FMovieSceneChannelProxy* NewChannelProxy = &Section->GetChannelProxy();
		ChannelHandle = NewChannelProxy->CopyHandle(ChannelHandle);
		OnDestroyHandle = NewChannelProxy->OnDestroy.AddRaw(this, &FChannelCurveModel<ChannelType, ChannelValue, KeyType>::FixupCurve);
	}
}


template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::GetCurveColorObjectAndName(UObject** OutObject, FString& OutName) const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		*OutObject = Section->GetImplicitObjectOwner();

		const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
		const bool bHasMetaData = MetaData && !MetaData->Group.IsEmpty() && !MetaData->DisplayText.IsEmpty();
		if (bHasMetaData)
		{
			OutName = FString::Printf(TEXT("%s.%s"), *MetaData->Group.ToString(), *MetaData->DisplayText.ToString());
		}
		else if (!GetIntentionName().IsEmpty())
		{
			OutName = GetIntentionName();
		}
		else if (UMovieSceneNameableTrack* NamedTrack = Section->GetTypedOuter<UMovieSceneNameableTrack>())
		{
			OutName = NamedTrack->GetDisplayName().ToString();
		}
		else
		{
			OutName = FString();
		}
	}
	else
	{
		// Just call base if it doesn't work
		FCurveModel::GetCurveColorObjectAndName(OutObject, OutName);
	}
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::PostProcessChanges(TNotNull<ChannelType*> Channel, TNotNull<UMovieSceneSection*> Section)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelCurveModel::PostProcessChanges)
	LastSignature.Invalidate();
	
	// Allow channel to make any follow up changes to the keys before executing any other delegates
	Channel->PostEditChange();

	// These cause interactive changes, such as evaluating sequencer as the keys are moved. Do not defer even when in a change scope.
	Section->MarkAsChanged();
	Section->MarkPackageDirty();
	if (UMovieSceneSignedObject* SignedOwner = Cast<UMovieSceneSignedObject>(WeakOwningObject.Get()))
	{
		SignedOwner->MarkAsChanged();
		SignedOwner->MarkPackageDirty();
	}

	InvokePostChannelChangedEvents(Section);
}

template <class ChannelType, class ChannelValue, class KeyType>
void FChannelCurveModel<ChannelType, ChannelValue, KeyType>::InvokePostChannelChangedEvents(TNotNull<UMovieSceneSection*> Section)
{
	// FScopeCurveChange starts a change scope and closes it again after it has appended its change commands.
	// We need to avoid Modify() calls made to the section because it causes subtle ordering issues (@see FScopeCurveChange).
	// OnChannelChangedWithTime may cause Modify() calls.
	const bool bIsDeferringCallbacks = IsInChangeScope();
	if (bIsDeferringCallbacks)
	{
		return;
	}
	
	if (WeakSequencer.IsValid())
	{
		const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
		TArray<FFrameNumber> Frames;
		WeakSequencer.Pin()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Section);
	}

	CurveModifiedDelegate.Broadcast();
}

// Explicit template instantiation
template class FChannelCurveModel<FMovieSceneDoubleChannel, FMovieSceneDoubleValue, double>;
template class FChannelCurveModel<FMovieSceneFloatChannel, FMovieSceneFloatValue, float>;
template class FChannelCurveModel<FMovieSceneIntegerChannel, int32, int32>;
template class FChannelCurveModel<FMovieSceneBoolChannel, bool, bool>;
template class FChannelCurveModel<FMovieSceneByteChannel, uint8, uint8>;
