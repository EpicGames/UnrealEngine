// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/ConstraintChannelCurveModel.h"

#include "Algo/BinarySearch.h"
#include "Channels/ConstraintChannelEditor.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "ConstraintChannel.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "SequencerClipboardReconciler.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtr.h"
#include "Sections/MovieSceneConstrainedSection.h"

class FCurveEditor;
class UObject;
struct FCurveEditorScreenSpace;

ECurveEditorViewID FConstraintChannelCurveModel::ViewID = ECurveEditorViewID::Invalid;

FConstraintChannelCurveModel::FConstraintChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneConstraintChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
{
	using namespace UE::Sequencer;

	FMovieSceneChannelProxy* NewChannelProxy = &OwningSection->GetChannelProxy();
	ChannelHandle = NewChannelProxy->MakeHandle<FMovieSceneConstraintChannel>(InChannel.GetChannelIndex());
	if (FMovieSceneChannelProxy* ChannelProxy = InChannel.GetChannelProxy())
	{
		OnDestroyHandle = NewChannelProxy->OnDestroy.AddRaw(this, &FConstraintChannelCurveModel::FixupCurve);
	}

	WeakSection = OwningSection;
	LastSignature = OwningSection->GetSignature();
	WeakSequencer = InWeakSequencer;
	SupportedViews = ViewID;
	Constraint = nullptr;
	Color = STrackAreaView::BlendDefaultTrackColor(OwningSection->GetTypedOuter<UMovieSceneTrack>()->GetColorTint());
}

UObject* FConstraintChannelCurveModel::GetOwningObject() const
{
	return WeakSection.Get();
}

bool FConstraintChannelCurveModel::HasChangedAndResetTest()
{
	if (const UMovieSceneSection* Section = WeakSection.Get())
	{
		if (Section->GetSignature() != LastSignature)
		{
			LastSignature = Section->GetSignature();
			return true;
		}
		return false;
	}
	return true;
}

FConstraintChannelCurveModel::~FConstraintChannelCurveModel()
{
	if (FMovieSceneChannelProxy* ChannelProxy = ChannelHandle.GetChannelProxy())
	{
		ChannelProxy->OnDestroy.Remove(OnDestroyHandle);
	}
}

void FConstraintChannelCurveModel::FixupCurve()
{
	if (const UMovieSceneSection* Section = WeakSection.Get())
	{
		FMovieSceneChannelProxy* NewChannelProxy = &Section->GetChannelProxy();
		ChannelHandle = NewChannelProxy->MakeHandle<FMovieSceneConstraintChannel>(ChannelHandle.GetChannelIndex());
		OnDestroyHandle = NewChannelProxy->OnDestroy.AddRaw(this, &FConstraintChannelCurveModel::FixupCurve);
	}
}

TArray<FKeyBarCurveModel::FBarRange> FConstraintChannelCurveModel::FindRanges()
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	const UMovieSceneSection* Section = WeakSection.Get();
	TArray<FKeyBarCurveModel::FBarRange> Ranges = FConstraintChannelEditor::GetBarRanges(Channel, Section);
	return Ranges;
}

const void* FConstraintChannelCurveModel::GetCurve() const
{
	return GetChannel();
}

FMovieSceneConstraintChannel* FConstraintChannelCurveModel::GetChannel() const
{
	return ChannelHandle.Get();
}

void FConstraintChannelCurveModel::Modify()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->Modify();
	}
}

void FConstraintChannelCurveModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InKeyAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	check(InKeyPositions.Num() == InKeyAttributes.Num() && (!OutKeyHandles || OutKeyHandles->Num() == InKeyPositions.Num()));

	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();
		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		for (int32 Index = 0; Index < InKeyPositions.Num(); ++Index)
		{
			FKeyPosition   Position = InKeyPositions[Index];
			FKeyAttributes Attributes = InKeyAttributes[Index];

			FFrameNumber Time = (Position.InputValue * TickResolution).RoundToFrame();
			Section->ExpandToFrame(Time);

			bool Value = false;
			FKeyHandle NewHandle = ChannelData.UpdateOrAddKey(Time, Value);
			if (NewHandle != FKeyHandle::Invalid())
			{
				if (OutKeyHandles)
				{
					(*OutKeyHandles)[Index] = NewHandle;
				}
			}
		}
	}
}

bool FConstraintChannelCurveModel::Evaluate(double Time, double& OutValue) const
{
	// Constraints don't evaluate into a valid value
	OutValue = 0.0;
	return false;
}

void FConstraintChannelCurveModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();

		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();

		for (FKeyHandle Handle : InKeys)
		{
			int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				ChannelData.RemoveKey(KeyIndex);
			}
		}
	}
}

void FConstraintChannelCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	// Constraint Channels don't draw any lines so there's no need to fill out the Interpolating Points array.
}

void FConstraintChannelCurveModel::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times = ChannelData.GetTimes();

		const FFrameNumber StartFrame = MinTime <= MIN_int32 ? MIN_int32 : (MinTime * TickResolution).CeilToFrame();
		const FFrameNumber EndFrame = MaxTime >= MAX_int32 ? MAX_int32 : (MaxTime * TickResolution).FloorToFrame();

		const int32 StartingIndex = Algo::LowerBound(Times, StartFrame);
		const int32 EndingIndex = Algo::UpperBound(Times, EndFrame);

		for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
		{
			// Space Channels don't have Values associated with them so we ignore the Min/Max value and always return the key.
			OutKeyHandles.Add(ChannelData.GetHandle(KeyIndex));
		}
	}
}

//we actually still draw the 'key' as a the border which you can move and right click, context menu with.
void FConstraintChannelCurveModel::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	OutDrawInfo.Brush = FAppStyle::Get().GetBrush("FilledBorder");
	// NOTE Y is set to SCurveEditorKeyBarView::TrackHeight
	OutDrawInfo.ScreenSize = FVector2D(10, 24);
}

void FConstraintChannelCurveModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times = ChannelData.GetTimes();

		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				OutKeyPositions[Index].InputValue = Times[KeyIndex] / TickResolution;
				// Spaces have no values so we just output zero.
				OutKeyPositions[Index].OutputValue = 0.0;
			}
		}
	}
}

void FConstraintChannelCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		Section->Modify();

		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				FFrameNumber NewTime = (InKeyPositions[Index].InputValue * TickResolution).FloorToFrame();

				const bool bRemoveDuplicates = ChangeType == EPropertyChangeType::ValueSet;
				KeyIndex = ChannelData.MoveKey(KeyIndex, NewTime, bRemoveDuplicates);
				Section->ExpandToFrame(NewTime);
			}
		}
	}
}

void FConstraintChannelCurveModel::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
}

void FConstraintChannelCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
}

void FConstraintChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	// Constraint Channels have no Pre/Post Extrapolation
	OutCurveAttributes.SetPreExtrapolation(RCCE_None);
	OutCurveAttributes.SetPostExtrapolation(RCCE_None);
}

void FConstraintChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
}

void FConstraintChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
}

void FConstraintChannelCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	FMovieSceneConstraintChannel* Channel = GetChannel();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		if (Times.Num() == 0)
		{
			MinTime = 0.f;
			MaxTime = 0.f;
		}
		else
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			double ToTime = TickResolution.AsInterval();
			MinTime = static_cast<double> (Times[0].Value) * ToTime;
			MaxTime = static_cast<double>(Times[Times.Num() - 1].Value) * ToTime;
		}
	}
}

void FConstraintChannelCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	// Constraint channels have no value, thus their value range is just zero.
	MinValue = MaxValue = 0.0;
}

int32 FConstraintChannelCurveModel::GetNumKeys() const
{
	FMovieSceneConstraintChannel* Channel = GetChannel();

	if (Channel)
	{
		return Channel->GetData().GetTimes().Num();
	}

	return 0;
}

void FConstraintChannelCurveModel::BuildContextMenu(const FCurveEditor& CurveEditor, FMenuBuilder& MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint)
{
	if (IsReadOnly())
	{
		return;
	}
	if (ClickedPoint.IsSet())
	{
		MenuBuilder.BeginSection("SpaceContextMenuSection", NSLOCTEXT("CurveEditor", "Constraint", "Constraint"));
		{
			FMovieSceneConstraintChannel* Channel = GetChannel();
			const FKeyHandle KeyHandle = ClickedPoint.GetValue().KeyHandle;
			if (Channel && KeyHandle != FKeyHandle::Invalid())
			{
				TMovieSceneChannelData<bool> ChannelData = Channel->GetData();
				int32 KeyIndex = ChannelData.GetIndex(KeyHandle);
				TArrayView<const bool> Values = ChannelData.GetValues();
				if (KeyIndex >= 0 && KeyIndex < Values.Num())
				{
					//need baking dialog for 
					//FControlRigSpaceChannelHelpers::OpenBakeDialog(WeakSequencer.Pin().Get(), Channel, KeyIndex, WeakSection.Get());
				}

			}
		}
		MenuBuilder.EndSection();
	}
}
