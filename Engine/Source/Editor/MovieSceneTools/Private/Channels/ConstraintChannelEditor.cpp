// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintChannelEditor.h"

#include "Channels/ConstraintChannelCurveModel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "ConstraintChannel.h"
#include "Containers/UnrealString.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "KeyBarCurveModel.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Function.h"
#include "TimeToPixel.h"
#include "CurveEditorSettings.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/SNullWidget.h"

class FCurveModel;
class FTrackInstancePropertyBindings;
class ISequencer;
class SWidget;
struct FGuid;
struct FSlateBrush;
template <typename ChannelType> struct TMovieSceneChannelHandle;

namespace
{
	using IndexRange = ::TRange<int32>;
	IndexRange GetRange(const TArrayView<const bool>& Values, const int32 Offset) 
	{
		IndexRange Range;
		Range = Range.Empty();
			
		int32 FirstActive = INDEX_NONE;
		for (int32 Index = Offset; Index < Values.Num(); ++Index)
		{
			if (Values[Index])
			{
				FirstActive = Index;
				break;
			}
		}

		if (FirstActive == INDEX_NONE)
		{
			return Range;
		}

		Range.SetLowerBound(TRangeBound<int32>(FirstActive));
		Range.SetUpperBound(TRangeBound<int32>(FirstActive));

		for (int32 NextInactive = FirstActive+1; NextInactive < Values.Num(); ++NextInactive)
		{
			if (Values[NextInactive] == false)
			{
				Range.SetUpperBound(TRangeBound<int32>(NextInactive));
				return Range;
			}
		}
			
		return Range;
	};

	TArray<IndexRange> GetIndexRanges(const FMovieSceneConstraintChannel* Channel)
	{
		TArray<IndexRange> Ranges;

		const TMovieSceneChannelData<const bool> ChannelData = Channel->GetData();
		const TArrayView<const bool> Values = ChannelData.GetValues();

		const bool bIsActiveByDefault = Channel->GetDefault() && *Channel->GetDefault();
		if (Values.Num() == 0 && bIsActiveByDefault)
		{
			Ranges.Emplace(INDEX_NONE);
			return Ranges;
		}
		
		if (Values.Num() == 1)
		{
			if (Values[0] == true)
			{
				Ranges.Emplace(0);
			}
			else
			{
				if (bIsActiveByDefault)
				{
					Ranges.Emplace(INDEX_NONE, 0);
				}
			}
			return Ranges;
		}
		
		int32 Offset = 0;

		if (bIsActiveByDefault)
		{
			const int32 FirstInactive = Values.Find(false);
			if (FirstInactive != INDEX_NONE)
			{
				Ranges.Emplace(INDEX_NONE, FirstInactive);
				Offset = FirstInactive+1;
			}
		}
		
		while (Values.IsValidIndex(Offset))
		{
			TRange<int32> Range = GetRange(Values, Offset);
			if (!Range.IsEmpty())
			{
				Offset = Range.GetUpperBoundValue()+1;
				Ranges.Emplace(MoveTemp(Range));
			}
			else
			{
				Offset = INDEX_NONE;
			}
		}
			
		return Ranges;
	}
}

void DrawKeys(
	FMovieSceneConstraintChannel* Channel,
	TArrayView<const FKeyHandle> InKeyHandles,
	const UMovieSceneSection* InOwner,
	TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	if (!InOwner)
	{
		return;
	}
	
	static const FName SquareKeyBrushName("Sequencer.KeySquare");
	static const FName FilledBorderBrushName("FilledBorder");

	static FKeyDrawParams Params;
	Params.FillBrush = FAppStyle::GetBrush(FilledBorderBrushName);
	Params.BorderBrush = FAppStyle::GetBrush(SquareKeyBrushName);
	
	for (FKeyDrawParams& Param : OutKeyDrawParams)
	{
		Param = Params;
	}
}

namespace
{

struct FConstraintChannelLabelBuilder
{
public:
	FConstraintChannelLabelBuilder(const UMovieSceneSection* InOwner, const FMovieSceneConstraintChannel* InChannel)
		: Owner(InOwner)
		, Channel(InChannel)
	{
		ConstructLabel();
	}

	FText Label;
	bool bValid = false;
	
private:
	void ConstructLabel() 
	{
		if (!Channel || !Owner)
		{
			return;
		}

		// get the stored extra label
		const FString ExtraLabel = Channel->ExtraLabel ? Channel->ExtraLabel() : FString();
		if (!ExtraLabel.IsEmpty())
		{
			Label = FText::FromString(ExtraLabel);
			bValid = true;
			return;
		}

		// fallback to meta data
		const FMovieSceneChannelProxy& Proxy = Owner->GetChannelProxy();
		const TArrayView<FMovieSceneConstraintChannel*> ConstraintChannels = Proxy.GetChannels<FMovieSceneConstraintChannel>();
		const TArrayView<const FMovieSceneChannelMetaData> MetaData = Proxy.GetMetaData<FMovieSceneConstraintChannel>();
		
		const int32 ChannelIndex = ConstraintChannels.IndexOfByKey(Channel);
		if (!ConstraintChannels.IsValidIndex(ChannelIndex) || !MetaData.IsValidIndex(ChannelIndex))
		{
			return;
		}

		const FName& ConstraintName = MetaData[ChannelIndex].Name;
		if (ConstraintName != NAME_None)
		{
			Label = FText::FromName(ConstraintName);
			bValid = true;
		}
	}

	const UMovieSceneSection* Owner = nullptr;
	const FMovieSceneConstraintChannel* Channel = nullptr;
};

}

TArray<FKeyBarCurveModel::FBarRange> FConstraintChannelEditor::GetBarRanges(
	FMovieSceneConstraintChannel* Channel,
	const UMovieSceneSection* Owner)
{
	TArray<FKeyBarCurveModel::FBarRange> Ranges;

	TArray<IndexRange> IndexRanges = GetIndexRanges(Channel);
	if (IndexRanges.IsEmpty())
	{
		return Ranges;
	}

	const FConstraintChannelLabelBuilder LabelBuilder(Owner, Channel);
	auto AddNewRange = [&Ranges, LabelStr = LabelBuilder.Label.ToString()]() -> FKeyBarCurveModel::FBarRange&
	{
		FKeyBarCurveModel::FBarRange& BarRange = Ranges.Emplace_GetRef();
		
		BarRange.Name = FName(*LabelStr);
		BarRange.Color = FLinearColor(.2f, .5f, .1f);
		static FLinearColor ZebraTint = FLinearColor::White.CopyWithNewOpacity(0.01f);
		BarRange.Color = BarRange.Color * (1.f - ZebraTint.A) + ZebraTint * ZebraTint.A;

		return BarRange;
	};

	// infinite range
	if (IndexRanges[0].HasLowerBound() && IndexRanges[0].GetLowerBoundValue() == INDEX_NONE &&
		IndexRanges[0].HasUpperBound() && IndexRanges[0].GetUpperBoundValue() == INDEX_NONE)
	{
		FKeyBarCurveModel::FBarRange& BarRange = AddNewRange();
		BarRange.bRangeIsInfinite = true;
	}
	else
	{
		FFrameRate TickResolution = Owner->GetTypedOuter<UMovieScene>()->GetTickResolution();
		
		// convert to bar range
		const TMovieSceneChannelData<const bool> ChannelData = Channel->GetData();
		const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		for (const IndexRange& ActiveRange : IndexRanges)
		{
			FKeyBarCurveModel::FBarRange& BarRange = AddNewRange();
			
			if (ActiveRange.HasLowerBound() && ActiveRange.GetLowerBoundValue() > INDEX_NONE)
			{
				double LowerValue = Times[ActiveRange.GetLowerBoundValue()] / TickResolution;
				BarRange.Range.SetLowerBound(TRangeBound<double>(LowerValue));
			}

			if (ActiveRange.HasUpperBound() && ActiveRange.GetUpperBoundValue() > INDEX_NONE)
			{
				double UpperValue = Times[ActiveRange.GetUpperBoundValue()] / TickResolution;
				BarRange.Range.SetUpperBound(TRangeBound<double>(UpperValue));
			}
			
			BarRange.bRangeIsInfinite = false;
		}		
	}

	return Ranges;
}

int32 DrawExtra(
	FMovieSceneConstraintChannel* Channel,
	const UMovieSceneSection* Owner,
	const FSequencerChannelPaintArgs& PaintArgs,
	int32 LayerId)
{
	if (!Owner || !Channel)
	{
		return LayerId;
	}
	TArray<FKeyBarCurveModel::FBarRange> Ranges = FConstraintChannelEditor::GetBarRanges(Channel,Owner);

	if (Ranges.IsEmpty())
	{
		return LayerId;
	}
	
	// draw bars + labels
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
	static constexpr ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;

	const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D LocalSize = PaintArgs.Geometry.GetLocalSize();
	static constexpr float LaneTop = 0;

	const double InputMin = PaintArgs.TimeToPixel.PixelToSeconds(0.f);
	const double InputMax = PaintArgs.TimeToPixel.PixelToSeconds(LocalSize.X);

	// label data
	const FConstraintChannelLabelBuilder LabelBuilder(Owner, Channel);
	const FText& Label = LabelBuilder.Label;
	const FVector2D TextSize = LabelBuilder.bValid ? FontMeasure->Measure(Label, FontInfo) : FVector2D::ZeroVector;
	static constexpr double LabelPixelOffset = 10.0;  
	
	for (int32 Index = 0; Index < Ranges.Num(); ++Index)
	{
		const FKeyBarCurveModel::FBarRange& Range = Ranges[Index];

		// draw bar
		double LowerSeconds = InputMin;
		double UpperSeconds = InputMax;
		
		if (Range.Range.HasLowerBound())
		{
			LowerSeconds = Range.Range.GetLowerBoundValue();
		}

		if (Range.Range.HasUpperBound())
		{
			UpperSeconds = Range.Range.GetUpperBoundValue();
			if (Range.Range.HasLowerBound() && UpperSeconds == Range.Range.GetLowerBoundValue())
			{
				UpperSeconds = InputMax;
			}
		}			

		const double BoxStart = PaintArgs.TimeToPixel.SecondsToPixel(LowerSeconds);
		const double BoxEnd = PaintArgs.TimeToPixel.SecondsToPixel(UpperSeconds);
		const double BoxSize = BoxEnd - BoxStart;

		const FVector2D Size = FVector2D(BoxSize, LocalSize.Y);
		const FVector2D Translation = FVector2D(BoxStart, LaneTop);
		const FPaintGeometry BoxGeometry = PaintArgs.Geometry.ToPaintGeometry(Size, FSlateLayoutTransform(Translation));
		
		FSlateDrawElement::MakeBox(PaintArgs.DrawElements, LayerId, BoxGeometry, WhiteBrush, DrawEffects, Range.Color);

		// draw label
		if (LabelBuilder.bValid)
		{
			const double LabelPos = BoxStart + LabelPixelOffset;
			const FVector2D Position(LabelPos, LaneTop + (LocalSize.Y - TextSize.Y) * .5f);
			const FPaintGeometry LabelGeometry = PaintArgs.Geometry.ToPaintGeometry(FSlateLayoutTransform(Position));

			const double LabelMaxSize = BoxSize-LabelPixelOffset;
			if (TextSize.X < LabelMaxSize)
			{
				FSlateDrawElement::MakeText(PaintArgs.DrawElements, LayerId, LabelGeometry, Label, FontInfo);				
			}
			else
			{
				// crop
				const int32 End = FontMeasure->FindLastWholeCharacterIndexBeforeOffset(Label, FontInfo, FMath::RoundToInt(LabelMaxSize));
				if (End >= 0)
				{
					FSlateDrawElement::MakeText(PaintArgs.DrawElements, LayerId, LabelGeometry, Label.ToString(), 0, End, FontInfo);
				}
			}
		}
	}

	return LayerId + 1;
}

FKeyHandle AddOrUpdateKey(
	FMovieSceneConstraintChannel* Channel,
	UMovieSceneSection* SectionToKey,
	FFrameNumber Time,
	ISequencer& Sequencer,
	const FGuid& ObjectBindingID,
	FTrackInstancePropertyBindings* PropertyBindings)
{
	return FKeyHandle::Invalid();
}

FKeyHandle AddOrUpdateKey(
	FMovieSceneConstraintChannel* Channel,
	UMovieSceneSection* SectionToKey,
	const TMovieSceneExternalValue<bool>& EditorData,
	FFrameNumber InTime,
	ISequencer& Sequencer,
	const FGuid& InObjectBindingID,
	FTrackInstancePropertyBindings* PropertyBindings)
{
	return FKeyHandle::Invalid();
}

bool CanCreateKeyEditor(const FMovieSceneConstraintChannel* InChannel)
{
	return false;
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneConstraintChannel>& InChannel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	return SNullWidget::NullWidget;
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneConstraintChannel>& ChannelHandle, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (FMovieSceneConstraintChannel* Channel = ChannelHandle.Get())
	{
		const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
		if (Settings && Settings->GetShowBars())
		{
			return MakeUnique<FConstraintChannelCurveModel>(ChannelHandle, Params.OwningSection, Params.Sequencer);
		}
	}
	return nullptr;
}