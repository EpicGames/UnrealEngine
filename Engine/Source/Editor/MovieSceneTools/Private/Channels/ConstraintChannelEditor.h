// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Curves/KeyHandle.h"
#include "KeyBarCurveModel.h"
#include "SequencerChannelTraits.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
class FSequencerSectionPainter;
class FTrackInstancePropertyBindings;
class ISequencer;
class SWidget;
class UMovieSceneSection;
struct FFrameNumber;
struct FGeometry;
struct FGuid;
struct FKeyDrawParams;
struct FMovieSceneConstraintChannel;
template <typename ChannelType> struct TMovieSceneChannelHandle;

/** Should concillidate this with the FCosntraintChannelHelper once that moves out of the control rig module*/
struct FConstraintChannelEditor
{
	static TArray<FKeyBarCurveModel::FBarRange> GetBarRanges(FMovieSceneConstraintChannel* Channel, const UMovieSceneSection* Owner);
};

/** Key drawing overrides */
void DrawKeys(
	FMovieSceneConstraintChannel* Channel,
	TArrayView<const FKeyHandle> InKeyHandles,
	const UMovieSceneSection* InOwner,
	TArrayView<FKeyDrawParams> OutKeyDrawParams);

int32 DrawExtra(
	FMovieSceneConstraintChannel* Channel,
	const UMovieSceneSection* Owner,
	const FSequencerChannelPaintArgs& PaintArgs,
	int32 LayerId);

/** Overrides for adding or updating a key for non-standard channels */
FKeyHandle AddOrUpdateKey(
	FMovieSceneConstraintChannel* Channel,
	UMovieSceneSection* SectionToKey,
	FFrameNumber Time,
	ISequencer& Sequencer,
	const FGuid& ObjectBindingID,
	FTrackInstancePropertyBindings* PropertyBindings);

FKeyHandle AddOrUpdateKey(
	FMovieSceneConstraintChannel* Channel,
	UMovieSceneSection* SectionToKey,
	const TMovieSceneExternalValue<bool>& EditorData,
	FFrameNumber InTime,
	ISequencer& Sequencer,
	const FGuid& InObjectBindingID,
	FTrackInstancePropertyBindings* PropertyBindings);

/** Key editor overrides */
bool CanCreateKeyEditor(const FMovieSceneConstraintChannel* InChannel);

TSharedRef<SWidget> CreateKeyEditor(
	const TMovieSceneChannelHandle<FMovieSceneConstraintChannel>& InChannel,
	const UE::Sequencer::FCreateKeyEditorParams& Params);

/** Curve editor models */
inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneConstraintChannel>& Channel) { return true; }
TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneConstraintChannel>& Channel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
