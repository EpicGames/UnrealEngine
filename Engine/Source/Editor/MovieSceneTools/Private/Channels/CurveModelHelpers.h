// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene.h"
#include "MovieSceneSection.h"

namespace UE::MovieSceneTools::CurveHelpers
{
template <typename T>
concept CCurveValueTypeable = requires { typename T::CurveValueType; };

/** Shared implementation for evaluating a ChannelType. Useful helper for evaluating IBufferedCurveModel. */
template <CCurveValueTypeable ChannelType>
bool Evaluate(
	double InTime, double& OutValue, const ChannelType& Channel, const TWeakObjectPtr<UMovieSceneSection> WeakSection
	)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		using KeyType = typename ChannelType::CurveValueType;
		KeyType ThisValue = static_cast<KeyType>(0.0);
		if (Channel.Evaluate(InTime * TickResolution, ThisValue))
		{
			OutValue = ThisValue;
			return true;
		}
	}

	return false;
}
}
