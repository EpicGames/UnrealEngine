// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/DoubleChannelCurveModel.h"

#include "Channels/CurveModelHelpers.h"
#include "Channels/DoubleChannelKeyProxy.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneInterpolation.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class ISequencer;
class UObject;

/**
 * Buffered curve implementation for a double channel curve model, stores a copy of the double channel in order to draw itself.
 */
class FDoubleChannelBufferedCurveModel : public IBufferedCurveModel
{
public:
	/** Create a copy of the double channel while keeping the reference to the section */
	FDoubleChannelBufferedCurveModel(const FMovieSceneDoubleChannel* InMovieSceneDoubleChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
		TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
		: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
		, Channel(*InMovieSceneDoubleChannel)
		, WeakSection(InWeakSection)
	{}

	virtual void DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override
	{
		UMovieSceneSection* Section = WeakSection.Get();

		if (Section && Section->GetTypedOuter<UMovieScene>())
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

			const double StartTimeSeconds = InScreenSpace.GetInputMin();
			const double EndTimeSeconds = InScreenSpace.GetInputMax();
			const double TimeThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerInput());
			const double ValueThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerOutput());

			Channel.PopulateCurvePoints(StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, TickResolution, OutInterpolatingPoints);
		}
	}

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		return UE::MovieSceneTools::CurveHelpers::Evaluate(InTime, OutValue, Channel, WeakSection);
	}

private:
	FMovieSceneDoubleChannel Channel;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};

FDoubleChannelCurveModel::FDoubleChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneDoubleChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
	: FBezierChannelCurveModel<FMovieSceneDoubleChannel, FMovieSceneDoubleValue, double>(InChannel, OwningSection, InWeakSequencer)
{
}

FDoubleChannelCurveModel::FDoubleChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneDoubleChannel> InChannel, UMovieSceneSection* OwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer)
	: FBezierChannelCurveModel<FMovieSceneDoubleChannel, FMovieSceneDoubleValue, double>(InChannel, OwningSection, InOwningObject, InWeakSequencer)
{
}

void FDoubleChannelCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	FMovieSceneDoubleChannel* Channel = GetChannelHandle().Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		if (Times.Num() > 0)
		{
			UE::MovieScene::Interpolation::FInterpolationExtents Extents =
				Channel->ComputeExtents(Times[0].Value, Times.Last().Value);
			MinValue = Extents.MinValue;
			MaxValue = Extents.MaxValue;
		}
		else
		{
			UE::MovieScene::Interpolation::FInterpolationExtents Extents =
				Channel->ComputeExtents(0, 1);
			MinValue = Extents.MinValue;
			MaxValue = Extents.MaxValue;
		}
	}
}

namespace UE::MovieSceneTools
{
static FString LexToString(TArrayView<const FFrameNumber> Times)
{
	return FString::JoinBy(Times, TEXT(", "), [](const FFrameNumber& FrameNumber)
	{
		return LexToString (FrameNumber);
	});
}
}

void FDoubleChannelCurveModel::GetValueRange(double InMinTime, double InMaxTime, double& MinValue, double& MaxValue) const
{
	FMovieSceneDoubleChannel* Channel = GetChannelHandle().Get();
	UMovieSceneSection* Section = WeakSection.Get();

	if (Channel && Section)
	{
		const TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();

		const FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		const FFrameTime StartTime = InMinTime * TickResolution;
		const FFrameTime EndTime = InMaxTime * TickResolution;

		// DP 5.7: This exists to debug issue UE-326363. We think that the Times array may be out of order (broken invariant). Remove once the issue is fixed.
		using namespace UE::MovieSceneTools;
		UE_CLOGF(StartTime > EndTime, LogMovieScene, Warning, "StartTime > EndTime: InMinTime: %f, InMaxTime: %f, TickResolution: %ls, StartTime (%ls), EndTime (%ls): Array is %ls",
			InMinTime, InMaxTime, *TickResolution.ToPrettyText().ToString(), *LexToString(StartTime), *LexToString(EndTime), *LexToString(Times)
			);
		
		UE::MovieScene::Interpolation::FInterpolationExtents Extents =
			Channel->ComputeExtents(StartTime, EndTime);

		MinValue = Extents.MinValue;
		MaxValue = Extents.MaxValue;
	}
}

void FDoubleChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		UDoubleChannelKeyProxy* NewProxy = NewObject<UDoubleChannelKeyProxy>(GetTransientPackage(), NAME_None);

		NewProxy->Initialize(InKeyHandles[Index], GetChannelHandle(), this->GetOwningObjectOrOuter<UMovieSceneSignedObject>());
		OutObjects[Index] = NewProxy;
	}
}

TUniquePtr<IBufferedCurveModel> FDoubleChannelCurveModel::CreateBufferedCurveCopy() const
{
	FMovieSceneDoubleChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		TArray<FKeyHandle> TargetKeyHandles;
		TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel->GetData();

		TRange<FFrameNumber> TotalRange = ChannelData.GetTotalRange();
		ChannelData.GetKeys(TotalRange, nullptr, &TargetKeyHandles);

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(GetNumKeys());
		TArray<FKeyAttributes> KeyAttributes;
		KeyAttributes.SetNumUninitialized(GetNumKeys());
		GetKeyPositions(TargetKeyHandles, KeyPositions);
		GetKeyAttributes(TargetKeyHandles, KeyAttributes);

		double ValueMin = 0.f, ValueMax = 1.f;
		GetValueRange(ValueMin, ValueMax);

		return MakeUnique<FDoubleChannelBufferedCurveModel>(Channel, this->GetOwningObjectOrOuter<UMovieSceneSection>(), MoveTemp(KeyPositions), MoveTemp(KeyAttributes), GetLongDisplayName().ToString(), ValueMin, ValueMax);
	}
	return nullptr;
}

