// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/DoubleChannelKeyProxy.h"
#include "Channels/MovieSceneChannelHandle.h"

#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DoubleChannelKeyProxy)

struct FPropertyChangedEvent;

void UDoubleChannelKeyProxy::Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneDoubleChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSignedObject> InWeakSignedObject)
{
	KeyHandle          = InKeyHandle;
	ChannelHandle      = InChannelHandle;
	WeakSignedObject   = InWeakSignedObject;
}


void UDoubleChannelKeyProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FMovieSceneDoubleValue NewValue = Value;

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		NewValue.Value = -NewValue.Value;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnProxyValueChanged(ChannelHandle, WeakSignedObject.Get(), KeyHandle, NewValue, Time);
}

void UDoubleChannelKeyProxy::UpdateValuesFromRawData()
{
	RefreshCurrentValue(ChannelHandle, KeyHandle, Value, Time);

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		Value.Value = -Value.Value;
	}
}
