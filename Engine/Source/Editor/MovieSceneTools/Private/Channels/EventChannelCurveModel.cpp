// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventChannelCurveModel.h"

#include "Algo/BinarySearch.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneEvent.h"
#include "Channels/MovieSceneEventChannel.h"
#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "Curves/RealCurve.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtr.h"

class FCurveEditor;
class UObject;
struct FCurveEditorScreenSpace;

ECurveEditorViewID FEventChannelCurveModel::EventView = ECurveEditorViewID::Invalid;

FEventChannelCurveModel::FEventChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneEventChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
{
	ChannelHandle = InChannel;
	WeakSection = OwningSection;
	WeakSequencer = InWeakSequencer;
	SupportedViews = EventView;

	Color = FSequencerSectionPainter::BlendColor(OwningSection->GetTypedOuter<UMovieSceneTrack>()->GetColorTint());
}

const void* FEventChannelCurveModel::GetCurve() const
{
	return ChannelHandle.Get();
}

void FEventChannelCurveModel::Modify()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->Modify();
	}
}

void FEventChannelCurveModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InKeyAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	check(InKeyPositions.Num() == InKeyAttributes.Num() && (!OutKeyHandles || OutKeyHandles->Num() == InKeyPositions.Num()));

	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();
		TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		for (int32 Index = 0; Index < InKeyPositions.Num(); ++Index)
		{
			FKeyPosition   Position   = InKeyPositions[Index];
			FKeyAttributes Attributes = InKeyAttributes[Index];

			FFrameNumber Time = (Position.InputValue * TickResolution).RoundToFrame();
			Section->ExpandToFrame(Time);

			FMovieSceneEvent Value;
			FKeyHandle NewHandle = ChannelData.UpdateOrAddKey(Time, Value);
			if (NewHandle != FKeyHandle::Invalid())
			{
				if (OutKeyHandles)
				{
					(*OutKeyHandles)[Index] = NewHandle;
				}
			}
		}
	}
}

bool FEventChannelCurveModel::Evaluate(double Time, double& OutValue) const
{
	// Events don't evaluate into a valid value
	OutValue = 0.0;
	return false;
}

void FEventChannelCurveModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();

		TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();

		for (FKeyHandle Handle : InKeys)
		{
			int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				ChannelData.RemoveKey(KeyIndex);
			}
		}
	}
}

void FEventChannelCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	// Event Channels don't draw any lines so there's no need to fill out the Interpolating Points array.
}

void FEventChannelCurveModel::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times  = ChannelData.GetTimes();

		const FFrameNumber StartFrame = MinTime <= MIN_int32 ? MIN_int32 : (MinTime * TickResolution).CeilToFrame();
		const FFrameNumber EndFrame   = MaxTime >= MAX_int32 ? MAX_int32 : (MaxTime * TickResolution).FloorToFrame();

		const int32 StartingIndex = Algo::LowerBound(Times, StartFrame);
		const int32 EndingIndex   = Algo::UpperBound(Times, EndFrame);

		for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
		{
			// Event Channels don't have Values associated with them so we ignore the Min/Max value and always return the key.
			OutKeyHandles.Add(ChannelData.GetHandle(KeyIndex));
		}
	}
}

void FEventChannelCurveModel::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	OutDrawInfo.Brush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
	OutDrawInfo.ScreenSize = FVector2D(10, 10);
}

void FEventChannelCurveModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times  = ChannelData.GetTimes();
		TArrayView<const FMovieSceneEvent> Values = ChannelData.GetValues();

		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				OutKeyPositions[Index].InputValue  = Times[KeyIndex] / TickResolution;
				// Events have no values so we just output zero.
				OutKeyPositions[Index].OutputValue = 0.0;
			}
		}
	}
}

void FEventChannelCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		Section->MarkAsChanged();

		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneEvent> ChannelData = Channel->GetData();
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				FFrameNumber NewTime = (InKeyPositions[Index].InputValue * TickResolution).FloorToFrame();

				const bool bRemoveDuplicates = ChangeType == EPropertyChangeType::ValueSet;
				KeyIndex = ChannelData.MoveKey(KeyIndex, NewTime, bRemoveDuplicates);
				Section->ExpandToFrame(NewTime);
			}
		}
	}
}

void FEventChannelCurveModel::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
}

void FEventChannelCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
}

void FEventChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	// Event Channels have no Pre/Post Extrapolation
	OutCurveAttributes.SetPreExtrapolation(RCCE_None);
	OutCurveAttributes.SetPostExtrapolation(RCCE_None);
}

void FEventChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
}

void FEventChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
}

void FEventChannelCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		if (Times.Num() == 0)
		{
			MinTime = 0.f;
			MaxTime = 0.f;
		}
		else
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			double ToTime = TickResolution.AsInterval();
			MinTime = static_cast<double> (Times[0].Value) * ToTime;
			MaxTime = static_cast<double>(Times[Times.Num() - 1].Value) * ToTime;
		}
	}
}

void FEventChannelCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	// Event Tracks have no value, thus their value range is just zero.
	MinValue = MaxValue = 0.0;
}

int32 FEventChannelCurveModel::GetNumKeys() const
{
	FMovieSceneEventChannel* Channel = ChannelHandle.Get();

	if (Channel)
	{
		return Channel->GetData().GetTimes().Num();
	}

	return 0;
}
