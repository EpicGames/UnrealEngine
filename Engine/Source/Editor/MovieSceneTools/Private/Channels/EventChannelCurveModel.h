// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneEventChannel.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Misc/OptionalFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FCurveAttributes;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyPosition;
struct FMovieSceneEventChannel;

class FEventChannelCurveModel : public FCurveModel
{
public:
	static ECurveEditorViewID EventView;

	FEventChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneEventChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);

	virtual const void* GetCurve() const override;

	virtual void Modify() override;

	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override;
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;

	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override;
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;

	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;

	virtual void GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const override;
	virtual void SetCurveAttributes(const FCurveAttributes& InCurveAttributes) override;
	virtual void GetTimeRange(double& MinTime, double& MaxTime) const override;
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	virtual int32 GetNumKeys() const override;
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override {}
	virtual bool Evaluate(double ProspectiveTime, double& OutValue) const override;
	virtual void AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override;

	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;

private:

	TMovieSceneChannelHandle<FMovieSceneEventChannel> ChannelHandle;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TWeakPtr<ISequencer> WeakSequencer;
};