// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/FloatChannelCurveModel.h"

#include "Channels/CurveModelHelpers.h"
#include "Channels/FloatChannelKeyProxy.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class ISequencer;
class UObject;

/**
 * Buffered curve implementation for a float channel curve model, stores a copy of the float channel in order to draw itself.
 */
class FFloatChannelBufferedCurveModel : public IBufferedCurveModel
{
public:
	/** Create a copy of the float channel while keeping the reference to the section */
	FFloatChannelBufferedCurveModel(const FMovieSceneFloatChannel* InMovieSceneFloatChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
		TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
		: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
		, Channel(*InMovieSceneFloatChannel)
		, WeakSection(InWeakSection)
	{}

	virtual void DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override
	{
		UMovieSceneSection* Section = WeakSection.Get();

		if (Section && Section->GetTypedOuter<UMovieScene>())
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

			const double StartTimeSeconds = InScreenSpace.GetInputMin();
			const double EndTimeSeconds = InScreenSpace.GetInputMax();
			const double TimeThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerInput());
			const double ValueThreshold = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerOutput());

			Channel.PopulateCurvePoints(StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, TickResolution, OutInterpolatingPoints);
		}
	}

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		return UE::MovieSceneTools::CurveHelpers::Evaluate(InTime, OutValue, Channel, WeakSection);
	}

private:
	FMovieSceneFloatChannel Channel;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};

FFloatChannelCurveModel::FFloatChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneFloatChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
	: FBezierChannelCurveModel<FMovieSceneFloatChannel, FMovieSceneFloatValue, float>(InChannel, OwningSection, InWeakSequencer)
{
}

void FFloatChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		UFloatChannelKeyProxy* NewProxy = NewObject<UFloatChannelKeyProxy>(GetTransientPackage(), NAME_None);

		NewProxy->Initialize(InKeyHandles[Index], GetChannelHandle(), this->GetOwningObjectOrOuter<UMovieSceneSection>());
		OutObjects[Index] = NewProxy;
	}
}

TUniquePtr<IBufferedCurveModel> FFloatChannelCurveModel::CreateBufferedCurveCopy() const
{
	FMovieSceneFloatChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		TArray<FKeyHandle> TargetKeyHandles;
		TMovieSceneChannelData<FMovieSceneFloatValue> ChannelData = Channel->GetData();

		TRange<FFrameNumber> TotalRange = ChannelData.GetTotalRange();
		ChannelData.GetKeys(TotalRange, nullptr, &TargetKeyHandles);

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(GetNumKeys());
		TArray<FKeyAttributes> KeyAttributes;
		KeyAttributes.SetNumUninitialized(GetNumKeys());
		GetKeyPositions(TargetKeyHandles, KeyPositions);
		GetKeyAttributes(TargetKeyHandles, KeyAttributes);

		double ValueMin = 0.f, ValueMax = 1.f;
		GetValueRange(ValueMin, ValueMax);

		return MakeUnique<FFloatChannelBufferedCurveModel>(Channel, this->GetOwningObjectOrOuter<UMovieSceneSection>(), MoveTemp(KeyPositions), MoveTemp(KeyAttributes), GetLongDisplayName().ToString(), ValueMin, ValueMax);
	}
	return nullptr;
}

