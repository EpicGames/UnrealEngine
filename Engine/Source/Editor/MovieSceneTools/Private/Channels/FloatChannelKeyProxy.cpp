// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/FloatChannelKeyProxy.h"
#include "Channels/MovieSceneChannelHandle.h"

#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatChannelKeyProxy)

struct FPropertyChangedEvent;

void UFloatChannelKeyProxy::Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneFloatChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSection> InWeakSection)
{
	KeyHandle     = InKeyHandle;
	ChannelHandle = InChannelHandle;
	WeakSection   = InWeakSection;
}


void UFloatChannelKeyProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FMovieSceneFloatValue NewValue = Value;

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		NewValue.Value = -NewValue.Value;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnProxyValueChanged(ChannelHandle, WeakSection.Get(), KeyHandle, NewValue, Time);
}

void UFloatChannelKeyProxy::UpdateValuesFromRawData()
{
	RefreshCurrentValue(ChannelHandle, KeyHandle, Value, Time);

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		Value.Value = -Value.Value;
	}
}
