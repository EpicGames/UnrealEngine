// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/IntegerChannelCurveModel.h"

#include "Algo/BinarySearch.h"
#include "Channels/CurveModelHelpers.h"
#include "Channels/IntegerChannelKeyProxy.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "IBufferedCurveModel.h"
#include "Internationalization/Text.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class UObject;

void DrawCurveImpl(const FMovieSceneIntegerChannel* Channel, const FCurveEditorScreenSpace& InScreenSpace, FFrameRate DisplayRate, FFrameRate TickResolution, TArray<TTuple<double, double>>& OutInterpolatingPoints)
{
	TMovieSceneChannelData<const int32> ChannelData = Channel->GetData();
	TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
	TArrayView<const int32> Values = ChannelData.GetValues();

	const bool bInterpolateLinearKeys = Channel->bInterpolateLinearKeys;
	const double StartTimeSeconds = InScreenSpace.GetInputMin();
	const double EndTimeSeconds = InScreenSpace.GetInputMax();

	const FFrameNumber StartFrame = (StartTimeSeconds * TickResolution).FloorToFrame();
	const FFrameNumber EndFrame = (EndTimeSeconds * TickResolution).CeilToFrame();

	const int32 StartingIndex = Algo::UpperBound(Times, StartFrame);
	const int32 EndingIndex = Algo::LowerBound(Times, EndFrame);

	// Add the lower bound of the visible space
	const bool bValidRange = StartingIndex < EndingIndex;
	//if we aren't just doing the default constant then we need to sample
	const bool PreNotConstant = (Channel->PreInfinityExtrap != RCCE_None && Channel->PreInfinityExtrap != RCCE_Constant);
	const bool PostNotConstant = (Channel->PostInfinityExtrap != RCCE_None && Channel->PostInfinityExtrap != RCCE_Constant);
	if (bValidRange && (PreNotConstant || PostNotConstant))
	{
		const FFrameNumber StartTimeInDisplay = FFrameRate::TransformTime(FFrameTime(StartFrame), TickResolution, DisplayRate).FloorToFrame();
		const FFrameNumber EndTimeInDisplay = FFrameRate::TransformTime(FFrameTime(EndFrame), TickResolution, DisplayRate).CeilToFrame();

		double Value = 0.0;
		TOptional<double> PreviousValue;
		for (FFrameNumber DisplayFrameNumber = StartTimeInDisplay; DisplayFrameNumber <= EndTimeInDisplay; ++DisplayFrameNumber)
		{
			FFrameTime TickFrameTime = FFrameRate::TransformTime(FFrameTime(DisplayFrameNumber), DisplayRate, TickResolution);
			Channel->EvaluateInterp(TickFrameTime, Value);
			if (!bInterpolateLinearKeys && (PreviousValue.IsSet() && PreviousValue.GetValue() != Value))
			{
				OutInterpolatingPoints.Add(MakeTuple(TickFrameTime / TickResolution, PreviousValue.GetValue()));
			}
			OutInterpolatingPoints.Add(MakeTuple(TickFrameTime / TickResolution, Value));
			PreviousValue = Value;
		}
	}
	else
	{
		if (bValidRange)
		{
			OutInterpolatingPoints.Add(MakeTuple(StartFrame / TickResolution, (double)Values[StartingIndex]));
		}

		TOptional<double> PreviousValue;
		for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
		{
			double Value = (double)Values[KeyIndex];
			if (!bInterpolateLinearKeys && (PreviousValue.IsSet() && PreviousValue.GetValue() != Value))
			{
				OutInterpolatingPoints.Add(MakeTuple(Times[KeyIndex] / TickResolution, PreviousValue.GetValue()));
			}

			OutInterpolatingPoints.Add(MakeTuple(Times[KeyIndex] / TickResolution, Value));
			PreviousValue = Value;
		}

		// Add the upper bound of the visible space
		if (bValidRange)
		{
			OutInterpolatingPoints.Add(MakeTuple(EndFrame / TickResolution, (double)Values[EndingIndex - 1]));
		}
	}
}

/**
 * Buffered curve implementation for a integer channel curve model, stores a copy of the integer channel in order to draw itself.
 */
class FIntegerChannelBufferedCurveModel : public IBufferedCurveModel
{
public:
	/** Create a copy of the float channel while keeping the reference to the section */
	FIntegerChannelBufferedCurveModel(const FMovieSceneIntegerChannel* InMovieSceneIntegerChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
		TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax)
		: IBufferedCurveModel(MoveTemp(InKeyPositions), MoveTemp(InKeyAttributes), InLongDisplayName, InValueMin, InValueMax)
		, Channel(*InMovieSceneIntegerChannel)
		, WeakSection(InWeakSection)
	{}

	virtual void DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override
	{
		UMovieSceneSection* Section = WeakSection.Get();

		if (Section)
		{
			FFrameRate DisplayRate = Section->GetTypedOuter<UMovieScene>()->GetDisplayRate();
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

			DrawCurveImpl(&Channel, InScreenSpace, DisplayRate, TickResolution, OutInterpolatingPoints);
		}
	}

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		return UE::MovieSceneTools::CurveHelpers::Evaluate(InTime, OutValue, Channel, WeakSection);
	}

private:
	FMovieSceneIntegerChannel Channel;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};

FIntegerChannelCurveModel::FIntegerChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneIntegerChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
	: FChannelCurveModel<FMovieSceneIntegerChannel, int32, int32>(InChannel, OwningSection, InWeakSequencer)
{
}

void FIntegerChannelCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const
{
	UMovieSceneSection* Section = this->GetOwningObjectOrOuter<UMovieSceneSection>();
	FMovieSceneIntegerChannel* Channel = GetChannelHandle().Get();

	if (Section && Channel && Channel->bInterpolateLinearKeys)
	{
		FFrameRate DisplayRate = Section->GetTypedOuter<UMovieScene>()->GetDisplayRate();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		DrawCurveImpl(Channel, InScreenSpace, DisplayRate, TickResolution, OutInterpolatingPoints);
	}
	else
	{
		FChannelCurveModel<FMovieSceneIntegerChannel, int32, int32>::DrawCurve(CurveEditor, InScreenSpace, OutInterpolatingPoints);
	}
}

void FIntegerChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
	for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
	{
		UIntegerChannelKeyProxy* NewProxy = NewObject<UIntegerChannelKeyProxy>(GetTransientPackage(), NAME_None);

		NewProxy->Initialize(InKeyHandles[Index], GetChannelHandle(), this->GetOwningObjectOrOuter<UMovieSceneSection>());
		OutObjects[Index] = NewProxy;
	}
}

TUniquePtr<IBufferedCurveModel> FIntegerChannelCurveModel::CreateBufferedCurveCopy() const
{
	FMovieSceneIntegerChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		TArray<FKeyHandle> TargetKeyHandles;
		TMovieSceneChannelData<int32> ChannelData = Channel->GetData();

		TRange<FFrameNumber> TotalRange = ChannelData.GetTotalRange();
		ChannelData.GetKeys(TotalRange, nullptr, &TargetKeyHandles);

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(GetNumKeys());
		TArray<FKeyAttributes> KeyAttributes;
		KeyAttributes.SetNumUninitialized(GetNumKeys());
		GetKeyPositions(TargetKeyHandles, KeyPositions);
		GetKeyAttributes(TargetKeyHandles, KeyAttributes);

		double ValueMin = 0.f, ValueMax = 1.f;
		GetValueRange(ValueMin, ValueMax);

		return MakeUnique<FIntegerChannelBufferedCurveModel>(Channel, this->GetOwningObjectOrOuter<UMovieSceneSection>(), MoveTemp(KeyPositions), MoveTemp(KeyAttributes), GetLongDisplayName().ToString(), ValueMin, ValueMax);
	}
	return nullptr;
}

void FIntegerChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	FMovieSceneIntegerChannel* Channel = GetChannelHandle().Get();
	if (Channel)
	{
		OutCurveAttributes.SetPreExtrapolation(Channel->PreInfinityExtrap);
		OutCurveAttributes.SetPostExtrapolation(Channel->PostInfinityExtrap);
	}
}

void FIntegerChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
	FMovieSceneIntegerChannel* Channel = GetChannelHandle().Get();
	UMovieSceneSection* Section = this->GetOwningObjectOrOuter<UMovieSceneSection>();
	if (Channel && Section && !IsReadOnly())
	{
		Section->MarkAsChanged();

		if (InCurveAttributes.HasPreExtrapolation())
		{
			Channel->PreInfinityExtrap = InCurveAttributes.GetPreExtrapolation();
		}

		if (InCurveAttributes.HasPostExtrapolation())
		{
			Channel->PostInfinityExtrap = InCurveAttributes.GetPostExtrapolation();
		}

		CurveModifiedDelegate.Broadcast();
	}
}

double FIntegerChannelCurveModel::GetKeyValue(TArrayView<const int32> Values, int32 Index) const
{
	return (double)Values[Index];
}

void FIntegerChannelCurveModel::SetKeyValue(int32 Index, double KeyValue) const
{
	FMovieSceneIntegerChannel* Channel = GetChannelHandle().Get();

	if (Channel)
	{
		TMovieSceneChannelData<int32> ChannelData = Channel->GetData();
		ChannelData.GetValues()[Index] = (int32)KeyValue;
	}
}
