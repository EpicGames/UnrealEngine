// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/IntegerChannelKeyProxy.h"

#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneChannelHandle.h"

#include "HAL/PlatformCrt.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntegerChannelKeyProxy)

struct FPropertyChangedEvent;

void UIntegerChannelKeyProxy::Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneIntegerChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSection> InWeakSection)
{
	KeyHandle     = InKeyHandle;
	ChannelHandle = InChannelHandle;
	WeakSection   = InWeakSection;
}

void UIntegerChannelKeyProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	int32 NewValue = Value;

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		NewValue = -NewValue;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnProxyValueChanged(ChannelHandle, WeakSection.Get(), KeyHandle, NewValue, Time);
}

void UIntegerChannelKeyProxy::UpdateValuesFromRawData()
{
	RefreshCurrentValue(ChannelHandle, KeyHandle, Value, Time);

	if (ChannelHandle.GetMetaData() != nullptr && ChannelHandle.GetMetaData()->bInvertValue)
	{
		Value = -Value;
	}
}
