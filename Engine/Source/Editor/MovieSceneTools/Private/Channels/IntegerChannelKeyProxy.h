// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "CoreTypes.h"
#include "CurveEditorKeyProxy.h"
#include "Curves/KeyHandle.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneKeyProxy.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "IntegerChannelKeyProxy.generated.h"

class UMovieSceneSection;
struct FPropertyChangedEvent;

UCLASS()
class UIntegerChannelKeyProxy : public UObject, public ICurveEditorKeyProxy, public IMovieSceneKeyProxy
{
public:
	GENERATED_BODY()

	/**
	 * Initialize this key proxy object by caching the underlying key object, and retrieving the time/value each tick
	 */
	void Initialize(FKeyHandle InKeyHandle, TMovieSceneChannelHandle<FMovieSceneIntegerChannel> InChannelHandle, TWeakObjectPtr<UMovieSceneSection> InWeakSection);

private:

	/** Apply this class's properties to the underlying key */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Update this class's properties from the underlying key */
	virtual void UpdateValuesFromRawData() override;

private:

	/** User-facing time of the key, applied to the actual key on PostEditChange, and updated every tick */
	UPROPERTY(EditAnywhere, Category="Key")
	FFrameNumber Time;

	/** User-facing value of the key, applied to the actual key on PostEditChange, and updated every tick */
	UPROPERTY(EditAnywhere, Category="Key", meta=(ShowOnlyInnerProperties))
	int32 Value;

private:

	/** Cached key handle that this key proxy relates to */
	FKeyHandle KeyHandle;
	/** Cached channel in which the key resides */
	TMovieSceneChannelHandle<FMovieSceneIntegerChannel> ChannelHandle;
	/** Cached section in which the channel resides */
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};