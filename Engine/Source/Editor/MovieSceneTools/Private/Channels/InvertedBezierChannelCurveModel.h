// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cache/MovieSceneCachedCurve.h"
#include "Cache/MovieSceneCurveCachePool.h"
#include "Cache/MovieSceneUpdateCachedCurveData.h"
#include "Channels/BezierChannelCurveModel.h"
#include "Channels/DoubleChannelCurveModel.h"
#include "Channels/FloatChannelCurveModel.h"
#include "CurveEditorCurveDrawParamsHandle.h"
#include "InvertedCurveModel.h"
#include "MovieScene.h"

namespace UE::MovieSceneTools
{
	template <typename TBase> 
	class TInvertedBezierChannelCurveModel 
		: public UE::CurveEditor::TInvertedCurveModel<TBase>
	{
	public:
		template <typename... TArg>
		explicit TInvertedBezierChannelCurveModel(TArg&&... Arg) 
			: UE::CurveEditor::TInvertedCurveModel<TBase>(Forward<TArg>(Arg)...)
		{}

		virtual UE::CurveEditor::ICurveEditorCurveCachePool* DrawCurveToCachePool(const TSharedRef<FCurveEditor>& CurveEditor, const UE::CurveEditor::FCurveDrawParamsHandle& CurveDrawParamsHandle, const FCurveEditorScreenSpace& ScreenSpace) override
		{
			using ChannelType = std::remove_pointer_t<decltype(this->GetChannelHandle().Get())>;
			ChannelType* Channel = this->GetChannelHandle().Get();
			UMovieSceneSection* Section = this->template GetOwningObjectOrOuter<UMovieSceneSection>();

			if (!this->CachedCurve.IsValid())
			{
				this->CachedCurve = MakeShared<UE::MovieSceneTools::FMovieSceneCachedCurve<ChannelType>>(CurveDrawParamsHandle.GetID());
				this->CachedCurve->Initialize(CurveEditor);
			}

			if (Channel && Section && Section->GetTypedOuter<UMovieScene>())
			{
				const FMovieSceneChannelMetaData* MetaData = this->GetChannelHandle().GetMetaData();
				const bool bInvertInterpolatingPointsY = MetaData ? MetaData->bInvertValue : false;

				const FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

				UE::MovieSceneTools::FMovieSceneUpdateCachedCurveData<ChannelType> UpdateData(
					*CurveEditor,
					*this,
					*Channel,
					ScreenSpace,
					TickResolution,
					bInvertInterpolatingPointsY);

				this->CachedCurve->UpdateCachedCurve(UpdateData, CurveDrawParamsHandle);
			}

			return &UE::MovieSceneTools::FMovieSceneCurveCachePool::Get();
		}
	};
}
