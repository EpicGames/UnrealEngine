// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/PerlinNoiseChannelDetailsCustomization.h"
#include "DetailLayoutBuilder.h"

TSharedRef<IDetailCustomization> FMovieSceneFloatPerlinNoiseChannelDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneFloatPerlinNoiseChannelDetailsCustomization());
}

FName FMovieSceneFloatPerlinNoiseChannelDetailsCustomization::GetPerlinNoiseChannelParamsPropertyName()
{
	static const FString ChannelProperty = GET_MEMBER_NAME_STRING_CHECKED(UMovieSceneFloatPerlinNoiseChannelContainer, PerlinNoiseChannel);
	static const FString ParamsProperty = GET_MEMBER_NAME_STRING_CHECKED(FMovieSceneFloatPerlinNoiseChannel, PerlinNoiseParams);
	static FName FullProperty(ChannelProperty + TEXT(".") + ParamsProperty);
	return FullProperty;
}

void FMovieSceneFloatPerlinNoiseChannelDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName PropertyName = GetPerlinNoiseChannelParamsPropertyName();
	TSharedRef<IPropertyHandle> ParamsProperty = DetailBuilder.GetProperty(PropertyName, UMovieSceneFloatPerlinNoiseChannelContainer::StaticClass());
	IDetailPropertyRow& Row = DetailBuilder.AddPropertyToCategory(ParamsProperty);	
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneFloatPerlinNoiseChannelContainer, PerlinNoiseChannel));
}

TSharedRef<IDetailCustomization> FMovieSceneDoublePerlinNoiseChannelDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneDoublePerlinNoiseChannelDetailsCustomization());
}

FName FMovieSceneDoublePerlinNoiseChannelDetailsCustomization::GetPerlinNoiseChannelParamsPropertyName()
{
	static const FString ChannelProperty = GET_MEMBER_NAME_STRING_CHECKED(UMovieSceneDoublePerlinNoiseChannelContainer, PerlinNoiseChannel);
	static const FString ParamsProperty = GET_MEMBER_NAME_STRING_CHECKED(FMovieSceneDoublePerlinNoiseChannel, PerlinNoiseParams);
	static FName FullProperty(ChannelProperty + TEXT(".") + ParamsProperty);
	return FullProperty;
}

void FMovieSceneDoublePerlinNoiseChannelDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName PropertyName = GetPerlinNoiseChannelParamsPropertyName();
	TSharedRef<IPropertyHandle> ParamsProperty = DetailBuilder.GetProperty(PropertyName, UMovieSceneDoublePerlinNoiseChannelContainer::StaticClass());
	IDetailPropertyRow& Row = DetailBuilder.AddPropertyToCategory(ParamsProperty);	
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneDoublePerlinNoiseChannelContainer, PerlinNoiseChannel));
}

