// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/PerlinNoiseChannelInterface.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneDoublePerlinNoiseChannel.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannel.h"
#include "Editor.h"
#include "IStructureDetailsView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "PerlinNoiseChannelInterface"

FPerlinNoiseChannelSectionMenuExtension::FPerlinNoiseChannelSectionMenuExtension(TArrayView<const FMovieSceneChannelHandle> InChannelHandles, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections)
	: ChannelHandles(InChannelHandles)
	, WeakSections(InWeakSections)
{
	Initialize();
}

TSharedPtr<ISidebarChannelExtension> FPerlinNoiseChannelSectionMenuExtension::ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu)
{
	if (ChannelHandles.Num() > 1)
	{
		if (bInSubMenu)
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("PerlinNoiseChannelsMenu", "Perlin Noise Channels"),
				LOCTEXT("PerlinNoiseChannelsMenuToolTip", "Edit parameters for Perlin Noise channels"),
				FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InnerMenuBuilder)
					{
						BuildChannelsMenu(InnerMenuBuilder);
					})
			);
		}
		else
		{
			BuildChannelsMenu(MenuBuilder);
		}
	}
	else if (ChannelHandles.Num() == 1)
	{
		if (bInSubMenu)
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("PerlinNoiseChannelsMenu", "Perlin Noise Channels"),
				LOCTEXT("PerlinNoiseChannelsMenuToolTip", "Edit parameters for Perlin Noise channels"),
				FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InnerMenuBuilder)
					{
						BuildParametersMenu(InnerMenuBuilder, 0);
					})
			);
		}
		else
		{
			BuildParametersMenu(MenuBuilder, 0);
		}
	}

	return AsShared();
}

void FPerlinNoiseChannelSectionMenuExtension::Initialize()
{
	// Figure out which channels belong to which section by building the index indirections.
	// Also, create the notify hooks. Normal channels need to modify the section, but overriden channels
	// need to modify their override channel container.
	TArray<FMovieSceneChannelProxy*> ChannelProxies;
	for (const TWeakObjectPtr<UMovieSceneSection>& Section : WeakSections)
	{
		ChannelProxies.Add(&Section->GetChannelProxy());
	}

	for (const FMovieSceneChannelHandle& ChannelHandle : ChannelHandles)
	{
		int32 SectionIndex = ChannelProxies.Find(ChannelHandle.GetChannelProxy());
		ChannelHandleSectionIndexes.Add(SectionIndex);

		TWeakObjectPtr<UMovieSceneSection> WeakSection = WeakSections[SectionIndex];
		if (!WeakSection.IsValid())
		{
			continue;
		}

		UObject* ObjectToModify = WeakSection.Get();

		IMovieSceneChannelOverrideProvider* ChannelOverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(WeakSection);
		UMovieSceneSectionChannelOverrideRegistry* ChannelOverrideRegistry = ChannelOverrideProvider ?
			ChannelOverrideProvider->GetChannelOverrideRegistry(false) : nullptr;
		if (ChannelOverrideRegistry)
		{
			const FName ChannelName = ChannelHandle.GetMetaData()->Name;
			UMovieSceneChannelOverrideContainer* ChannelOverrideContainer = ChannelOverrideRegistry->GetChannel(ChannelName);
			if (ChannelOverrideContainer)
			{
				ensureMsgf(ChannelOverrideContainer->GetChannel() == ChannelHandle.Get(),
						TEXT("Mismatched channel override!"));
				ObjectToModify = ChannelOverrideContainer;
			}
		}

		NotifyHooks.Add(FChannelNotifyHook(ObjectToModify));
	}
}

void FPerlinNoiseChannelSectionMenuExtension::BuildChannelsMenu(FMenuBuilder& MenuBuilder)
{
	const bool bMultipleSections = WeakSections.Num() > 1;
	TSharedRef<FPerlinNoiseChannelSectionMenuExtension> SharedThis = this->AsShared();

	for (int32 Index = 0; Index < ChannelHandles.Num(); ++Index)
	{
		const int32 SectionIndex(ChannelHandleSectionIndexes[Index]);
		const FMovieSceneChannelHandle ChannelHandle(ChannelHandles[Index]);
		const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

		FText DisplayText;
		if (MetaData)
		{
			if (!MetaData->Group.IsEmpty())
			{
				DisplayText = FText::Format(LOCTEXT("PerlinNoiseChannelDisplayText","{0}.{1}"), MetaData->Group, MetaData->DisplayText);
			}
			else
			{
				DisplayText = MetaData->DisplayText;
			}
		}

		if (bMultipleSections)
		{
			MenuBuilder.AddSubMenu(
				FText::Format(LOCTEXT("PerlinNoiseChannelAndSectionSelectMenu", "Section{0}.{1}"), SectionIndex + 1, DisplayText),
				LOCTEXT("PerlinNoiseChannelAndSectionSelectMenuToolTip", "Edit parameters for this Perlin Noise channel"),
				FNewMenuDelegate::CreateLambda([this, Index](FMenuBuilder& InnerMenuBuilder)
					{
						BuildParametersMenu(InnerMenuBuilder, Index);
					})
			);
		}
		else
		{
			MenuBuilder.AddSubMenu(
				DisplayText,
				LOCTEXT("PerlinNoiseChannelSelectMenuToolTip", "Edit parameters for this Perlin Noise channel"),
				FNewMenuDelegate::CreateLambda([this, Index](FMenuBuilder& InnerMenuBuilder)
					{
						BuildParametersMenu(InnerMenuBuilder, Index);
					})
			);
		}
	}
}

void FPerlinNoiseChannelSectionMenuExtension::BuildParametersMenu(FMenuBuilder& MenuBuilder, int32 ChannelHandleIndex)
{
	if (!ensure(ChannelHandles.IsValidIndex(ChannelHandleIndex)))
	{
		return;
	}

	FPerlinNoiseParams* PerlinNoiseParams = nullptr;
	FMovieSceneChannelHandle ChannelHandle(ChannelHandles[ChannelHandleIndex]);
	if (ChannelHandle.GetChannelTypeName() == FMovieSceneFloatPerlinNoiseChannel::StaticStruct()->GetFName())
	{
		FMovieSceneFloatPerlinNoiseChannel* FloatChannel = ChannelHandle.Cast<FMovieSceneFloatPerlinNoiseChannel>().Get();
		PerlinNoiseParams = &FloatChannel->PerlinNoiseParams;
	}
	else if (ChannelHandle.GetChannelTypeName() == FMovieSceneDoublePerlinNoiseChannel::StaticStruct()->GetFName())
	{
		FMovieSceneDoublePerlinNoiseChannel* DoubleChannel = ChannelHandle.Cast<FMovieSceneDoublePerlinNoiseChannel>().Get();
		PerlinNoiseParams = &DoubleChannel->PerlinNoiseParams;
	}
	else
	{
		ensureMsgf(false, TEXT("Unknown perlin noise channel type: %s"), *ChannelHandle.GetChannelTypeName().ToString());
		return;
	}

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowScrollBar = false;
	// The notify hook is owned by us. We will live as long as the menu is active, so as long as the NotifyHooks array isn't
	// modified, the address of the hooks should be valid.
	DetailsViewArgs.NotifyHook = &NotifyHooks[ChannelHandleIndex];

	FStructureDetailsViewArgs StructureDetailsViewArgs;
	StructureDetailsViewArgs.bShowObjects = true;
	StructureDetailsViewArgs.bShowAssets = true;
	StructureDetailsViewArgs.bShowClasses = true;
	StructureDetailsViewArgs.bShowInterfaces = true;

	TSharedRef<FStructOnScope> StructData = MakeShareable(new FStructOnScope(FPerlinNoiseParams::StaticStruct(), (uint8*)PerlinNoiseParams));

	TSharedRef<IStructureDetailsView> DetailsView = EditModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, StructData);

	MenuBuilder.AddWidget(DetailsView->GetWidget().ToSharedRef(), FText(), true, false);
}

void FPerlinNoiseChannelSectionMenuExtension::FChannelNotifyHook::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	GEditor->BeginTransaction(FText::Format(LOCTEXT("EditProperty", "Edit {0}"), PropertyAboutToChange->GetDisplayNameText()));

	WeakObjectToModify->Modify();
}

void FPerlinNoiseChannelSectionMenuExtension::FChannelNotifyHook::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	GEditor->EndTransaction();
}

#undef LOCTEXT_NAMESPACE
