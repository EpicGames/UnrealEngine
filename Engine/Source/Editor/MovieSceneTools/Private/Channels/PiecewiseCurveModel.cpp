// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/PiecewiseCurveModel.h"
#include "Channels/MovieScenePiecewiseCurve.h"
#include "CurveEditorScreenSpace.h"

namespace UE::MovieScene
{


void RefineCurvePoints(const FPiecewiseCurve* InCurve, FFrameRate InFrameRate, double TimeThreshold, double ValueThreshold, TArray<TTuple<double, double>>& InterpolatingPoints)
{
	const float InterpTimes[] = { 0.25f, 0.5f, 0.6f };

	for (int32 Index = 0; Index < InterpolatingPoints.Num() - 1; ++Index)
	{
		TTuple<double, double> Lower = InterpolatingPoints[Index];
		TTuple<double, double> Upper = InterpolatingPoints[Index + 1];

		if ((Upper.Get<0>() - Lower.Get<0>()) >= TimeThreshold)
		{
			bool bSegmentIsLinear = true;

			TTuple<double, double> Evaluated[UE_ARRAY_COUNT(InterpTimes)] = { TTuple<double, double>(0, 0) };

			for (int32 InterpIndex = 0; InterpIndex < UE_ARRAY_COUNT(InterpTimes); ++InterpIndex)
			{
				double& EvalTime  = Evaluated[InterpIndex].Get<0>();

				EvalTime = FMath::Lerp(Lower.Get<0>(), Upper.Get<0>(), InterpTimes[InterpIndex]);

				double Value = 0.0;
				InCurve->Evaluate(EvalTime * InFrameRate, Value);

				const double LinearValue = FMath::Lerp(Lower.Get<1>(), Upper.Get<1>(), InterpTimes[InterpIndex]);
				if (bSegmentIsLinear)
				{
					bSegmentIsLinear = FMath::IsNearlyEqual(Value, LinearValue, ValueThreshold);
				}

				Evaluated[InterpIndex].Get<1>() = Value;
			}

			if (!bSegmentIsLinear)
			{
				// Add the point
				InterpolatingPoints.Insert(Evaluated, UE_ARRAY_COUNT(Evaluated), Index+1);
				--Index;
			}
		}
	}
}

void FPiecewiseCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	using namespace Interpolation;

	const FPiecewiseCurve* Curve = CurveAttribute.Get();
	const FFrameRate       FrameRate = FrameRateAttribute.Get();
	if (Curve && Curve->Values.Num() > 0)
	{
		for (const FCachedInterpolation& Piece : Curve->Values)
		{
			if (Piece.GetRange().Start.Value != MIN_int32)
			{
				MinTime = Piece.GetRange().Start / FrameRate;
				break;
			}
		}

		for (int32 Index = Curve->Values.Num()-1; Index >= 0; --Index)
		{
			const FCachedInterpolation& Piece = Curve->Values[Index];

			if (Piece.GetRange().End.Value != MAX_int32)
			{
				MinTime = Piece.GetRange().End / FrameRate;
				break;
			}
		}
	}
}

void FPiecewiseCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	using namespace Interpolation;

	const FPiecewiseCurve* Curve = CurveAttribute.Get();

	FInterpolationExtents Extents;

	if (Curve)
	{
		for (const FCachedInterpolation& Piece : Curve->Values)
		{
			if (Piece.GetRange().Start.Value != MIN_int32 && Piece.GetRange().End.Value != MAX_int32)
			{
				Extents.Combine(Piece.ComputeExtents());
			}
		}
	}

	if (Extents.IsValid())
	{
		MinValue = Extents.MinValue;
		MaxValue = Extents.MaxValue;
	}
}

bool FPiecewiseCurveModel::Evaluate(double InTime, double& OutValue) const
{
	if (const FPiecewiseCurve* Curve = CurveAttribute.Get())
	{
		FFrameRate FrameRate = FrameRateAttribute.Get();
		return Curve->Evaluate(InTime * FrameRate, OutValue);
	}

	return false;
}

void FPiecewiseCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	using namespace Interpolation;

	const FPiecewiseCurve* Curve     = CurveAttribute.Get();
	const FFrameRate       FrameRate = FrameRateAttribute.Get();

	if (Curve)
	{
		const double StartTimeSeconds = InScreenSpace.GetInputMin();
		const double EndTimeSeconds   = InScreenSpace.GetInputMax();
		const double TimeThreshold    = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerInput());
		const double ValueThreshold   = FMath::Max(0.0001, 1.0 / InScreenSpace.PixelsPerOutput());


		const FFrameNumber StartFrame = (StartTimeSeconds * FrameRate).FloorToFrame();
		const FFrameNumber EndFrame   = (EndTimeSeconds   * FrameRate).CeilToFrame();

		const int32 StartingIndex = Algo::UpperBoundBy(Curve->Values, StartFrame, [](const FCachedInterpolation& In){ return In.GetRange().Start; });
		const int32 EndingIndex   = Algo::LowerBoundBy(Curve->Values, EndFrame,   [](const FCachedInterpolation& In){ return In.GetRange().Start; });

		// Add the lower bound of the visible space
		double EvaluatedValue = 0.0;
		if (Curve->Evaluate(StartFrame, EvaluatedValue))
		{
			InterpolatingPoints.Add(MakeTuple(StartFrame / FrameRate, EvaluatedValue));
		}

		// Add all pieces in-between
		for (int32 PieceIndex = StartingIndex; PieceIndex < EndingIndex; ++PieceIndex)
		{
			FCachedInterpolation Interp = Curve->Values[PieceIndex];

			if (Interp.GetRange().Start >= StartFrame && Interp.Evaluate(Interp.GetRange().Start, EvaluatedValue))
			{
				if (InterpolatingPoints.Num() > 0 && InterpolatingPoints.Last().Get<0>() != Interp.GetRange().Start / FrameRate)
				{
					InterpolatingPoints.Add(MakeTuple(Interp.GetRange().Start / FrameRate, EvaluatedValue));
				}
			}
			if (Interp.GetRange().End <= EndFrame && Interp.Evaluate(Interp.GetRange().End, EvaluatedValue))
			{
				if (InterpolatingPoints.Num() > 0 && InterpolatingPoints.Last().Get<0>() != Interp.GetRange().End / FrameRate)
				{
					InterpolatingPoints.Add(MakeTuple(Interp.GetRange().End / FrameRate, EvaluatedValue));
				}
			}
		}

		// Add the upper bound of the visible space
		if (Curve->Evaluate(EndFrame, EvaluatedValue))
		{
			InterpolatingPoints.Add(MakeTuple(EndFrame / FrameRate, EvaluatedValue));
		}

		int32 OldSize = InterpolatingPoints.Num();
		do
		{
			OldSize = InterpolatingPoints.Num();
			RefineCurvePoints(Curve, FrameRate, TimeThreshold, ValueThreshold, InterpolatingPoints);
		}
		while(OldSize != InterpolatingPoints.Num());
	}
}

} // namespace UE::MovieScene