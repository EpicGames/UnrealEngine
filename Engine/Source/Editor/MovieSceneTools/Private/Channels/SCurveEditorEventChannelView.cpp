// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/SCurveEditorEventChannelView.h"

#include "Containers/SortedMap.h"
#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveModel.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/TransformCalculus2D.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "SCurveEditorView.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FSlateBrush;

float SCurveEditorEventChannelView::TrackHeight = 24.f;

void SCurveEditorEventChannelView::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	bFixedOutputBounds = true;
	bAllowModelViewTransforms = false;
	OutputMin = -0.5;
	OutputMax =  0.5;
	SortBias = 25;

	SInteractiveCurveEditorView::Construct(InArgs, InCurveEditor);
}

FVector2D SCurveEditorEventChannelView::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(100.f, TrackHeight * (CurveInfoByID.Num()));
}

void SCurveEditorEventChannelView::GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{}

void SCurveEditorEventChannelView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	double Count = 0.0;
	for (auto It = CurveInfoByID.CreateIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (ensureAlways(Curve))
		{
			It->Value.ViewToCurveTransform = FTransform2d(FVector2D(0.f, Count));
		}

		Count += 1.0;
	}

	OutputMin = OutputMax - FMath::Max(Count, 1e-10);
	SInteractiveCurveEditorView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SCurveEditorEventChannelView::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawLabels(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawGridLines(CurveEditor.ToSharedRef(), AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawCurves(CurveEditor.ToSharedRef(), AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
	}
}

void SCurveEditorEventChannelView::DrawLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

	// Draw some text telling the user how to get retime anchors.
	const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");

	// We have to measure the string so we can draw it centered on the window.
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D LocalSize = AllottedGeometry.GetLocalSize();

	const FCurveEditorScreenSpaceV ViewSpace = GetViewSpace();

	for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		const int32 CurveIndex = static_cast<int32>(It->Value.ViewToCurveTransform.GetTranslation().Y);

		const FCurveEditorScreenSpaceV CurveSpace = ViewSpace.ToCurveSpace(It->Value.ViewToCurveTransform);
		const float LaneTop = CurveSpace.ValueToScreen(0.0) - TrackHeight*.5f;

		// Draw the curve color as the background. Event curves set their track color as the curve color.
		FLinearColor CurveColor = Curve->GetColor();

		// Alpha blend the zebra tint
		if (CurveIndex%2)
		{
			static FLinearColor ZebraTint = FLinearColor::White.CopyWithNewOpacity(0.01f);
			if (CurveColor == FLinearColor::White)
			{
				CurveColor = ZebraTint;
			}
			else
			{
				CurveColor = CurveColor * (1.f - ZebraTint.A) + ZebraTint * ZebraTint.A;
			}
		}

		if (CurveColor != FLinearColor::White)
		{
			const FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(
				FVector2D(AllottedGeometry.GetLocalSize().X, TrackHeight),
				FSlateLayoutTransform(FVector2D(0.f, LaneTop))
			);

			FSlateDrawElement::MakeBox(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Background, BoxGeometry, WhiteBrush, DrawEffects, CurveColor);
		}

		// Draw the curve label
		{
			const FText Label = Curve->GetLongDisplayName();

			const FVector2D TextSize = FontMeasure->Measure(Label, FontInfo);
			const FVector2D Position(LocalSize.X - TextSize.X - CurveViewConstants::CurveLabelOffsetX, LaneTop + (TrackHeight - TextSize.Y)*.5f);

			const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(
				FSlateLayoutTransform(Position)
			);

			FSlateDrawElement::MakeText(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Labels, LabelGeometry, Label, FontInfo, DrawEffects, FLinearColor::White);
		}
	}
}