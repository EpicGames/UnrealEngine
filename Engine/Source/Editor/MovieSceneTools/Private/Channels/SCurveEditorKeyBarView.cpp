// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/SCurveEditorKeyBarView.h"

#include "Containers/SortedMap.h"
#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "KeyBarCurveModel.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/TransformCalculus2D.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderer.h"
#include "SCurveEditorView.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"

class FCurveModel;
class FMenuBuilder;
class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FSlateBrush;

float SCurveEditorKeyBarView::TrackHeight = 24.f;

void SCurveEditorKeyBarView::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
{
	bFixedOutputBounds = true;
	bAllowModelViewTransforms = false;

	OutputMin = -0.5;
	OutputMax =  0.5;
	SortBias = 25;

	SInteractiveCurveEditorView::Construct(InArgs, InCurveEditor);
}

FVector2D SCurveEditorKeyBarView::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(100.f, TrackHeight * (CurveInfoByID.Num()));
}

void SCurveEditorKeyBarView::GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const
{}

void SCurveEditorKeyBarView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	double Count = 0.0;
	for (auto It = CurveInfoByID.CreateIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (ensureAlways(Curve))
		{
			It->Value.ViewToCurveTransform = FTransform2d(FVector2D(0.f, Count));
		}

		Count += 1.0;
	}

	OutputMin = OutputMax - FMath::Max(Count, 1e-10);
	SInteractiveCurveEditorView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SCurveEditorKeyBarView::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawLabels(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawGridLines(CurveEditor.ToSharedRef(), AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
		DrawCurves(CurveEditor.ToSharedRef(), AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
	}
}

void SCurveEditorKeyBarView::DrawLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

	// Draw some text telling the user how to get retime anchors.
	const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");

	// We have to measure the string so we can draw it centered on the window.
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D LocalSize(AllottedGeometry.GetLocalSize().X, TrackHeight);

	double InputMin = 0, InputMax = 1;
	GetInputBounds(InputMin, InputMax);//in Sequencer Seconds

	const FCurveEditorScreenSpace ViewSpace = GetViewSpace();

	for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(It.Key());
		if (!ensureAlways(Curve))
		{
			continue;
		}

		const int32 CurveIndex = static_cast<int32>(It->Value.ViewToCurveTransform.GetTranslation().Y);

		const FCurveEditorScreenSpace CurveSpace = GetCurveSpace(It.Key());
		const float LaneTop = CurveSpace.ValueToScreen(0.0) - TrackHeight*.5f;

		// Draw the curve label and background
		FKeyBarCurveModel* KeyBarCurveModel = static_cast<FKeyBarCurveModel*>(Curve);
		if (KeyBarCurveModel)
		{
			TArray<FKeyBarCurveModel::FBarRange> Ranges = KeyBarCurveModel->FindRanges();
			int32 LastLabelPos = -1;
			for (int32 Index = 0;Index < Ranges.Num(); ++Index)
			{
				const FKeyBarCurveModel::FBarRange& Range = Ranges[Index];

				double LowerSeconds = Range.Range.HasLowerBound() ? Range.Range.GetLowerBoundValue() : InputMin;
				double UpperSeconds = Range.Range.HasUpperBound() ? Range.Range.GetUpperBoundValue() : InputMax;
				
				if (UpperSeconds == LowerSeconds)
				{
					UpperSeconds = InputMax;
				}

				const bool bOutsideUpper = (Index != Ranges.Num() - 1) && UpperSeconds < InputMin;
				if (LowerSeconds > InputMax || bOutsideUpper) //out of range
				{
					continue;
				}
				
				FLinearColor CurveColor = Range.Color;

				// Alpha blend the zebra tint
				if (CurveIndex % 2)
				{
					static FLinearColor ZebraTint = FLinearColor::White.CopyWithNewOpacity(0.01f);
					if (CurveColor == FLinearColor::White)
					{
						CurveColor = ZebraTint;
					}
					else
					{
						CurveColor = CurveColor * (1.f - ZebraTint.A) + ZebraTint * ZebraTint.A;
					}
				}

				// draw background box
				if (CurveColor != FLinearColor::White)
				{
					FPaintGeometry BoxGeometry;
					if (Range.bRangeIsInfinite)
					{
						const double LowerSecondsForBox = Index == 0 ? InputMin : FMath::Max<double>(InputMin, LowerSeconds);
						const double BoxPos = CurveSpace.SecondsToScreen(LowerSecondsForBox);
						BoxGeometry = AllottedGeometry.ToPaintGeometry(
							LocalSize,
							FSlateLayoutTransform(FVector2D(BoxPos, LaneTop))
						);
					}
					else
					{
						const double LowerSecondsForBox = FMath::Max<double>(InputMin, LowerSeconds);
						const double UpperSecondsForBox = FMath::Min<double>(InputMax, UpperSeconds);
						const double BoxPos = CurveSpace.SecondsToScreen(LowerSecondsForBox);
						const double BoxEnd = CurveSpace.SecondsToScreen(UpperSecondsForBox);
						const FVector2D OurLocalSize(BoxEnd - BoxPos, LocalSize.Y);

						BoxGeometry = AllottedGeometry.ToPaintGeometry(
							OurLocalSize,
							FSlateLayoutTransform(FVector2D(BoxPos, LaneTop))
						);
					}
				
					FSlateDrawElement::MakeBox(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Background, BoxGeometry, WhiteBrush, DrawEffects, CurveColor);
				}

				// draw label
				const double LowerSecondsForLabel = (InputMin > LowerSeconds) ? InputMin : LowerSeconds;
				double LabelPos = CurveSpace.SecondsToScreen(LowerSecondsForLabel) + 10;

				const FText Label = FText::FromName(Range.Name);
				const FVector2D TextSize = FontMeasure->Measure(Label, FontInfo);
				if (Index > 0)
				{
					LabelPos = (LabelPos < LastLabelPos) ? LastLabelPos + 5 : LabelPos;
				}
				LastLabelPos = LabelPos + TextSize.X + 15;
				const FVector2D Position(LabelPos, LaneTop + (TrackHeight - TextSize.Y) * .5f);

				const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(
					FSlateLayoutTransform(Position)
				);

				FSlateDrawElement::MakeText(OutDrawElements, BaseLayerId + CurveViewConstants::ELayerOffset::Labels, LabelGeometry, Label, FontInfo, DrawEffects, FLinearColor::White);

			}
		}
	}
}

void SCurveEditorKeyBarView::BuildContextMenu(FMenuBuilder & MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint, TOptional<FCurveModelID> HoveredCurveID)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}
	int32 NumSelectedKeys = CurveEditor->GetSelection().Count();

	FCurveModel* Curve = HoveredCurveID.IsSet() ? CurveEditor->FindCurve(HoveredCurveID.GetValue()) : nullptr;
	FKeyBarCurveModel* KeyBarCurveModel = static_cast<FKeyBarCurveModel*>(Curve);
	if (KeyBarCurveModel)
	{
		KeyBarCurveModel->BuildContextMenu(*CurveEditor,MenuBuilder, ClickedPoint);
	}
}




