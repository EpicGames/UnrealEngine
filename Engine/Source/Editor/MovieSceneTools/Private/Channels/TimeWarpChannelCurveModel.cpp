// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/TimeWarpChannelCurveModel.h"
#include "Channels/PiecewiseCurveModel.h"
#include "Variants/MovieScenePlayRateCurve.h"
#include "CurveEditor.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"

FTimeWarpChannelCurveModel::FTimeWarpChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer)
	: FDoubleChannelCurveModel(InChannel, InOwningSection, InOwningObject, InWeakSequencer)
{
}

UObject* FTimeWarpChannelCurveModel::GetOwningObjectOrOuter(UClass* Class) const
{
	if (UObject* Result = FDoubleChannelCurveModel::GetOwningObjectOrOuter(Class))
	{
		return Result;
	}
	// The time warp getter is sequence-outered, so a UMovieSceneSection isn't in the outer
	// chain. Fall back to our directly-stored section when the caller is looking for one
	// (or any base of UMovieSceneSection).
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		if (Class && Section->IsA(Class))
		{
			return Section;
		}
	}
	return nullptr;
}


void FTimeWarpChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
	FCurveAttributes FinalAttributes = InCurveAttributes;

	const FMovieSceneTimeWarpChannel* Channel = static_cast<const FMovieSceneTimeWarpChannel*>(GetChannelHandle().Get());

	// Disallow certain extrapolation modes for play rate curves.
	if (Channel && Channel->Domain == UE::MovieScene::ETimeWarpChannelDomain::PlayRate)
	{
		FCurveAttributes ExistingAttributes;
		GetCurveAttributes(ExistingAttributes);

		if (InCurveAttributes.HasPreExtrapolation())
		{
			switch (InCurveAttributes.GetPreExtrapolation())
			{
				case RCCE_Cycle:
				case RCCE_CycleWithOffset:
				case RCCE_Oscillate:
					if (ExistingAttributes.HasPreExtrapolation())
					{
						FinalAttributes.SetPreExtrapolation(ExistingAttributes.GetPreExtrapolation());
					}
					else
					{
						FinalAttributes.SetPreExtrapolation(RCCE_None);
					}
					break;
				default:
					break;
			}
		}


		if (InCurveAttributes.HasPostExtrapolation())
		{
			switch (InCurveAttributes.GetPostExtrapolation())
			{
				case RCCE_Cycle:
				case RCCE_CycleWithOffset:
				case RCCE_Oscillate:
					if (ExistingAttributes.HasPostExtrapolation())
					{
						FinalAttributes.SetPostExtrapolation(ExistingAttributes.GetPostExtrapolation());
					}
					else
					{
						FinalAttributes.SetPostExtrapolation(RCCE_None);
					}
					break;
				default:
					break;
			}
		}
	}

	FDoubleChannelCurveModel::SetCurveAttributes(FinalAttributes);
}


void FTimeWarpChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	FDoubleChannelCurveModel::GetCurveAttributes(OutCurveAttributes);
}


void FTimeWarpChannelCurveModel::AllocateAxes(FCurveEditor* InCurveEditor, TSharedPtr<FCurveEditorAxis>& OutHorizontalAxis, TSharedPtr<FCurveEditorAxis>& OutVerticalAxis) const
{
	FMovieSceneTimeWarpChannel* Channel = static_cast<FMovieSceneTimeWarpChannel*>(GetChannelHandle().Get());

	if (Channel && Channel->Domain == UE::MovieScene::ETimeWarpChannelDomain::Time)
	{
		OutVerticalAxis = InCurveEditor->FindAxis("FocusedSequenceTime");
	}
}


void FTimeWarpChannelCurveModel::MakeChildCurves(TArray<TUniquePtr<FCurveModel>>& OutChildCurves) const
{
	using namespace UE::MovieScene;

	const FMovieSceneChannelMetaData* MetaData = GetChannelHandle().GetMetaData();
	if (!MetaData)
	{
		return;
	}

	UMovieScenePlayRateCurve* Owner = Cast<UMovieScenePlayRateCurve>(MetaData->WeakOwningObject.Get());
	if (!Owner)
	{
		return;
	}

	struct FTimeDomainPiecewiseCurveModel : FPiecewiseCurveModel
	{
		void AllocateAxes(FCurveEditor* InCurveEditor, TSharedPtr<FCurveEditorAxis>& OutHorizontalAxis, TSharedPtr<FCurveEditorAxis>& OutVerticalAxis) const
		{
			OutVerticalAxis = InCurveEditor->FindAxis("FocusedSequenceTime");
		}
	};

	TUniquePtr<FTimeDomainPiecewiseCurveModel> Model = MakeUnique<FTimeDomainPiecewiseCurveModel>();

	UMovieSceneSequence* Sequence = Owner->GetTypedOuter<UMovieSceneSequence>();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	Model->CurveAttribute = MakeAttributeLambda([Owner]{ return &Owner->GetTimeWarpCurve(); });
	Model->FrameRateAttribute = MovieScene ? MovieScene->GetTickResolution() : FFrameRate();
	if (MetaData->bRelativeToSection)
	{
		Model->CurveTransformAttribute = MakeAttributeLambda([this]{ return this->GetCurveTransform(); });
	}
	Model->SetColor(FLinearColor::White);
	Model->SetThickness(2.f);
	Model->SetDashLength(5.f);

	OutChildCurves.Add(MoveTemp(Model));
}