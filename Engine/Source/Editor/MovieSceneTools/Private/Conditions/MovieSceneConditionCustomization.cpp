// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneConditionCustomization.h"
#include "Conditions/MovieSceneCondition.h"

#include "PropertyHandle.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Images/SImage.h"
#include "MovieSceneSequence.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AssetToolsModule.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionCustomization.h"
#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieScene.h"
#include "ClassViewerFilter.h"
#include "ScopedTransaction.h"
#include "ISequencer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneConditionCustomization)

#define LOCTEXT_NAMESPACE "MovieSceneConditionCustomization"

class FConditionClassFilter : public IClassViewerFilter
{
public:
	
	TWeakObjectPtr<UMovieScene> MovieScene;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if (InClass && InClass->IsChildOf(UMovieSceneCondition::StaticClass()))
		{
			// Don't show the director blueprint condition here, as we call it out separately
			if (InClass == UMovieSceneDirectorBlueprintCondition::StaticClass())
			{
				return false;
			}

			if (MovieScene.IsValid())
			{
				return MovieScene->IsConditionClassAllowed(InClass);
			}
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InBlueprint, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		const UClass* NativeParent = InBlueprint->GetNativeParent();
		if (NativeParent && NativeParent->IsChildOf(UMovieSceneCondition::StaticClass()))
		{
			if (MovieScene.IsValid())
			{
				return MovieScene->IsConditionClassAllowed(NativeParent);
			}
		}
		return false;
	}
};

TSharedRef<IPropertyTypeCustomization> FMovieSceneConditionCustomization::MakeInstance()
{
	TSharedRef<FMovieSceneConditionCustomization> Instance = MakeShared<FMovieSceneConditionCustomization>();
	return Instance;
}

TSharedRef<IPropertyTypeCustomization> FMovieSceneConditionCustomization::MakeInstance(TWeakObjectPtr<UMovieSceneSequence> InMovieSceneSequence, const TWeakPtr<ISequencer> Sequencer)
{
	TSharedRef<FMovieSceneConditionCustomization> Instance = MakeShared<FMovieSceneConditionCustomization>();
	Instance->Sequence = InMovieSceneSequence;
	Instance->Sequencer = Sequencer;
	return Instance;
}

void FMovieSceneConditionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ConditionContainerPropertyHandle = InPropertyHandle;

	if (!Sequencer.IsValid())
	{
		ConditionContainerPropertyHandle->MarkHiddenByCustomization();
		return;
	}

	ConditionPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneConditionContainer, Condition));

	if (!Sequence.IsValid())
	{
		Sequence = GetCommonSequence();
	}

	if (!Track.IsValid())
	{
		Track = GetCommonTrack();
	}

	TStrongObjectPtr<UMovieSceneSequence> SequencePtr = Sequence.Pin();
	TStrongObjectPtr<UMovieSceneTrack> TrackPtr = Track.Pin();
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();

	// If conditions not allowed, hide condition property functionality
	if (!SequencePtr.IsValid()
	|| !SequencePtr->GetMovieScene()->IsConditionClassAllowed(UMovieSceneCondition::StaticClass())
	|| (TrackPtr.IsValid() && SequencerPtr.IsValid() && !SequencerPtr->TrackSupportsConditions(TrackPtr.Get())))
	{
		ConditionContainerPropertyHandle->MarkHiddenByCustomization();
		return;
	}
	
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	HeaderRow
	.NameContent()
	[
		ConditionPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.FillWidth(0.5)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ComboButton, SComboButton)
				.OnGetMenuContent(this, &FMovieSceneConditionCustomization::GenerateConditionPicker)
				.ContentPadding(0.0f)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(this, &FMovieSceneConditionCustomization::GetDisplayValueIcon)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &FMovieSceneConditionCustomization::GetDisplayValueAsString)
					]
				]
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeUseSelectedButton(FSimpleDelegate::CreateSP(this, &FMovieSceneConditionCustomization::OnUseSelected),
				LOCTEXT("UseSelectedConditionClass", "Use Selected Condition Class in Content Browser"),
				TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMovieSceneConditionCustomization::CanUseSelectedAsset)))
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FMovieSceneConditionCustomization::OnBrowseTo),
				LOCTEXT("BrowseToConditionClass", "Browse To Condition Class in Content Browser"), 
				TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMovieSceneConditionCustomization::CanBrowseToAsset)))
			]
	];
}

void FMovieSceneConditionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!Sequencer.IsValid())
	{
		return;
	}

	// If conditions not allowed, hide condition property functionality
	if (!Sequence.IsValid() || !Sequence->GetMovieScene()->IsConditionClassAllowed(UMovieSceneCondition::StaticClass()) || (Track.IsValid() && !Track->SupportsConditions()))
	{
		return;
	}

	// Create new properties in the parent layout rather than adding a single item to a single category
	IDetailLayoutBuilder& LayoutBuilder = ChildBuilder.GetParentCategory().GetParentLayout();

	//IDetailCategoryBuilder& NewCategory = LayoutBuilder.EditCategory(TEXT("Condition"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	// Hold onto a reference to the details view to prevent it from being destroyed immediately when the menu goes away. 
	DetailsView = LayoutBuilder.GetDetailsViewSharedPtr();

	// Customize and display the inner children of the Condition property itself as the children here.

	uint32 NumChildren;
	ConditionPropertyHandle->GetNumChildren(NumChildren);

	// This should be the object itself
	if (NumChildren == 1)
	{
		TSharedRef<IPropertyHandle> ObjectHandle = ConditionPropertyHandle->GetChildHandle(0).ToSharedRef();
		TArray<void*> ConditionRawArray;
		ObjectHandle->AccessRawData(ConditionRawArray);
		if (ConditionRawArray.Num() > 0)
		{
			UMovieSceneCondition* Condition = reinterpret_cast<UMovieSceneCondition*>(ConditionRawArray[0]);
			{
				TArray<UObject*> ObjectArray;
				ObjectArray.Add(Condition);
				IDetailPropertyRow* ExternalRow = ChildBuilder.AddExternalObjects(ObjectArray, FAddPropertyParams().HideRootObjectNode(true).AllowChildren(true));
			}
		}
	}
}

FText FMovieSceneConditionCustomization::GetDisplayValueAsString() const
{
	UObject* CurrentValue = NULL;
	FPropertyAccess::Result Result = ConditionPropertyHandle->GetValue(CurrentValue);
	if (Result == FPropertyAccess::Success && CurrentValue != NULL)
	{
		return CurrentValue->GetClass()->GetDisplayNameText();
	}
	else
	{
		return LOCTEXT("ConditionNone", "None");
	}
}

const FSlateBrush* FMovieSceneConditionCustomization::GetDisplayValueIcon() const
{
	UObject* CurrentValue = nullptr;
	FPropertyAccess::Result Result = ConditionPropertyHandle->GetValue(CurrentValue);
	if (Result == FPropertyAccess::Success && CurrentValue != nullptr)
	{
		return FSlateIconFinder::FindIconBrushForClass(CurrentValue->GetClass());
	}

	return nullptr;
}


void FMovieSceneConditionCustomization::FillConditionClassSubMenu(FMenuBuilder& MenuBuilder)
{
	if (UMovieScene* MovieScene = Sequence.IsValid() ? Sequence->GetMovieScene() : nullptr)
	{
		// Not quite the right thing to do, but we don't have a generic way of checking whether blueprint graphs are enabled.
		// We make the assumption that if Director Blueprints conditions aren't allowed, then neither is creating a new condition blueprint class.
		if (MovieScene->IsConditionClassAllowed(UMovieSceneDirectorBlueprintCondition::StaticClass()))
		{
			// Create a new Condition Class
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ConditionAddNewBlueprintCondition", "Create new Condition Blueprint Class"),
				LOCTEXT("ConditionAddNewBlueprintConditionTooltip", "Creates a new condition blueprint asset"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSPLambda(this, [SharedThis=StaticCastSharedRef<FMovieSceneConditionCustomization>(AsShared())]()
				{
					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

					if (SharedThis->Sequence.IsValid())
					{
						FString NewConditionPath = SharedThis->Sequence->GetPathName();
						FString NewConditionName = SharedThis->Sequence->GetName() + TEXT("_Condition");
						AssetToolsModule.Get().CreateUniqueAssetName(NewConditionPath + TEXT("/") + NewConditionName, TEXT(""), NewConditionPath, NewConditionName);

						const FScopedTransaction Transaction(LOCTEXT("CreateConditionAsset", "Create Condition Asset"));
						UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(LOCTEXT("CreateNewConditionClass", "Create New Condition Class"), UMovieSceneCondition::StaticClass(), NewConditionName);

						if (Blueprint != NULL && Blueprint->GeneratedClass)
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);

							// Implement the EvaluateCondition function

							UFunction* OverrideFunc = FindUField<UFunction>(UMovieSceneCondition::StaticClass(), GET_FUNCTION_NAME_CHECKED(UMovieSceneCondition, BP_EvaluateCondition));
							check(OverrideFunc);
							Blueprint->Modify();
							// Implement the function graph
							UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, TEXT("BP_EvaluateCondition"), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
							FBlueprintEditorUtils::AddFunctionGraph(Blueprint, NewGraph, /*bIsUserCreated=*/ false, UMovieSceneCondition::StaticClass());
							NewGraph->Modify();
							FKismetEditorUtilities::CompileBlueprint(Blueprint);
							// Set the property to the newly created class
							PropertyCustomizationHelpers::CreateNewInstanceOfEditInlineObjectClass(SharedThis->ConditionPropertyHandle.ToSharedRef(), Blueprint->GeneratedClass);
							SharedThis->PropertyUtilities->ForceRefresh();
							FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewGraph);
						}
					}
				}
			)));
		}
	}

	MenuBuilder.BeginSection(TEXT("ChooseConditionClass"), LOCTEXT("ChooseConditionClass", "Choose Condition Class"));
	{
		TSharedPtr<FConditionClassFilter> ConditionClassFilter = nullptr;
		if (UMovieScene* MovieScene = Sequence.IsValid() ? Sequence->GetMovieScene() : nullptr)
		{ 
			ConditionClassFilter = MakeShared<FConditionClassFilter>();
			ConditionClassFilter->MovieScene = MovieScene;
		}

		MenuBuilder.AddWidget(PropertyCustomizationHelpers::MakeEditInlineObjectClassPicker(ConditionPropertyHandle.ToSharedRef(), FOnClassPicked::CreateSPLambda(this, [SharedThis = StaticCastSharedRef<FMovieSceneConditionCustomization>(AsShared())](UClass* Class)
			{
				FSlateApplication::Get().DismissMenuByWidget(SharedThis->OpenMenuWidget.ToSharedRef());
				SharedThis->PropertyUtilities->ForceRefresh();
			}), ConditionClassFilter), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();
}

void FMovieSceneConditionCustomization::FillDirectorBlueprintConditionSubMenu(FMenuBuilder& MenuBuilder)
{
	if (Sequence.IsValid())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateEndpoint_Text", "Create New Condition Endpoint"),
			LOCTEXT("CreateEndpoint_Tooltip", "Creates a new condition endpoint in this sequence's blueprint."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateEventBinding"),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [SharedThis = StaticCastSharedRef<FMovieSceneConditionCustomization>(AsShared())]()
				{
					UMovieSceneSequence* ThisSequence = SharedThis->Sequence.Get();
					if (ThisSequence)
					{
						const FScopedTransaction Transaction(LOCTEXT("CreateNewConditionEndpoint", "Create New Condition Endpoint"));

						ThisSequence->Modify();
						SharedThis->ConditionPropertyHandle->NotifyPreChange();
						// Create a new director blueprint condition and set it in the details view. Use 'interactive change' so we don't early fire the property finished changing event and reset the details view mid-change
						PropertyCustomizationHelpers::CreateNewInstanceOfEditInlineObjectClass(SharedThis->ConditionPropertyHandle.ToSharedRef(), UMovieSceneDirectorBlueprintCondition::StaticClass(), EPropertyValueSetFlags::InteractiveChange);
						TSharedPtr<IPropertyHandle> DirectorBlueprintConditionHandle = SharedThis->ConditionPropertyHandle->GetChildHandle(TEXT("DirectorBlueprintConditionData"));
						TSharedPtr<FMovieSceneDirectorBlueprintConditionCustomization> BlueprintConditionCustomization = FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(ThisSequence->GetMovieScene(), DirectorBlueprintConditionHandle, SharedThis->PropertyUtilities);
						BlueprintConditionCustomization->CreateEndpoint();
						SharedThis->ConditionPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
						SharedThis->PropertyUtilities->NotifyFinishedChangingProperties(FPropertyChangedEvent(SharedThis->ConditionPropertyHandle->GetProperty()));
						// Extra end transaction because we use 'Interactive Change' in the CreateNewInstance call
						GEditor->EndTransaction();
						SharedThis->PropertyUtilities->ForceRefresh();
					}
				}
			)
		));

		MenuBuilder.AddSubMenu(
			LOCTEXT("CreateQuickBinding_Text", "Quick Bind"),
			LOCTEXT("CreateQuickBinding_Tooltip", "Shows a list of functions in this sequence's blueprint that can be used for conditions."),
			FNewMenuDelegate::CreateSP(this, &FMovieSceneConditionCustomization::PopulateQuickBindSubMenu),
			false /* bInOpenSubMenuOnClick */,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateQuickBinding"),
			false /* bInShouldWindowAfterMenuSelection */
		);
	}
}


void FMovieSceneConditionCustomization::PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FMovieSceneDirectorBlueprintConditionCustomization> BlueprintConditionCustomization = FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(Sequence->GetMovieScene(), nullptr, PropertyUtilities);

	if (BlueprintConditionCustomization.IsValid())
	{
		BlueprintConditionCustomization->PopulateQuickBindSubMenu(MenuBuilder, 
		Sequence.Get(),
		FOnQuickBindActionSelected::CreateSPLambda(this, [BlueprintConditionCustomization, SharedThis = StaticCastSharedRef<FMovieSceneConditionCustomization>(AsShared())]
		(
			const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, 
			ESelectInfo::Type InSelectionType, 
			UBlueprint* Blueprint, 
			FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition
		)
		{
			if (!SelectedAction.IsEmpty())
			{
				const FScopedTransaction Transaction(LOCTEXT("SetConditionEndpoint", "Set Condition Endpoint"));
				
				// Create a new director blueprint condition and set it in the details view. Use 'interactive change' so we don't early fire the property finished changing event and reset the details view mid-change
				PropertyCustomizationHelpers::CreateNewInstanceOfEditInlineObjectClass(SharedThis->ConditionPropertyHandle.ToSharedRef(), UMovieSceneDirectorBlueprintCondition::StaticClass(), EPropertyValueSetFlags::InteractiveChange);
				TSharedPtr<IPropertyHandle> DirectorBlueprintConditionHandle = SharedThis->ConditionPropertyHandle->GetChildHandle(TEXT("DirectorBlueprintConditionData"));
				BlueprintConditionCustomization->SetPropertyHandle(DirectorBlueprintConditionHandle);
				BlueprintConditionCustomization->HandleQuickBindActionSelected(SelectedAction, InSelectionType, Blueprint, EndpointDefinition);
				// Extra end transaction because we use 'Interactive Change' in the CreateNewInstance call
				GEditor->EndTransaction();
				SharedThis->PropertyUtilities->ForceRefresh();
			}
		}));
	}
}

void FMovieSceneConditionCustomization::OnUseSelected()
{
	// Load selected assets
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		UBlueprint* SelectedBlueprint = Cast<UBlueprint>(AssetData.GetAsset());

		if (SelectedBlueprint)
		{
			if (SelectedBlueprint->GeneratedClass && SelectedBlueprint->GeneratedClass->IsChildOf<UMovieSceneCondition>())
			{
				const FScopedTransaction Transaction(LOCTEXT("SetConditionClass", "Set Condition Class"));

				Sequence->Modify();
				ConditionPropertyHandle->NotifyPreChange();
				PropertyCustomizationHelpers::CreateNewInstanceOfEditInlineObjectClass(ConditionPropertyHandle.ToSharedRef(), SelectedBlueprint->GeneratedClass, EPropertyValueSetFlags::InteractiveChange);
				ConditionPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				PropertyUtilities->NotifyFinishedChangingProperties(FPropertyChangedEvent(ConditionPropertyHandle->GetProperty()));
				// Extra end transaction because we use 'Interactive Change' in the CreateNewInstance call
				GEditor->EndTransaction();
				PropertyUtilities->ForceRefresh();
				return;
			}
		}
	}
}

bool FMovieSceneConditionCustomization::CanUseSelectedAsset() const
{
	// Load selected assets
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		UBlueprint* SelectedBlueprint = Cast<UBlueprint>(AssetData.GetAsset());

		if (SelectedBlueprint)
		{
			if (SelectedBlueprint->GeneratedClass && SelectedBlueprint->GeneratedClass->IsChildOf<UMovieSceneCondition>())
			{
				return true;
			}
		}
	}

	return false;
}

void FMovieSceneConditionCustomization::OnBrowseTo()
{
	UObject* CurrentValue = NULL;
	FPropertyAccess::Result Result = ConditionPropertyHandle->GetValue(CurrentValue);
	if (Result == FPropertyAccess::Success && CurrentValue != NULL)
	{
		UClass* CurrentClass = CurrentValue->GetClass();
		if (CurrentClass)
		{
			if (TObjectPtr<UObject> Blueprint = CurrentClass->ClassGeneratedBy)
			{
				TArray< UObject* > Objects;
				Objects.Add(Blueprint.Get());
				GEditor->SyncBrowserToObjects(Objects);
			}
		}
	}
}

bool FMovieSceneConditionCustomization::CanBrowseToAsset() const
{
	UObject* CurrentValue = NULL;
	FPropertyAccess::Result Result = ConditionPropertyHandle->GetValue(CurrentValue);
	if (Result == FPropertyAccess::Success && CurrentValue != NULL)
	{
		UClass* CurrentClass = CurrentValue->GetClass();
		if (CurrentClass)
		{
			if (TObjectPtr<UObject> Blueprint = CurrentClass->ClassGeneratedBy)
			{
				return true;
			}
		}
	}
	return false;
}

TSharedRef<SWidget> FMovieSceneConditionCustomization::GenerateConditionPicker()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

	// None option
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ConditionNone", "None"),
		LOCTEXT("ConditionNoneTooltip", "No Condition"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSPLambda(this, [SharedThis = StaticCastSharedRef<FMovieSceneConditionCustomization>(AsShared())]() 
		{ 
			SharedThis->ConditionPropertyHandle->ResetToDefault();
			FSlateApplication::Get().DismissMenuByWidget(SharedThis->OpenMenuWidget.ToSharedRef());
		}))
	);

	// Option to choose or create a new condition class
	MenuBuilder.AddSubMenu(
		LOCTEXT("ConditionClass", "Condition Class..."),
		LOCTEXT("ConditionClassTooltip", "Select an existing condition class, or create a new blueprint condition class"),
		FNewMenuDelegate::CreateSP(this, &FMovieSceneConditionCustomization::FillConditionClassSubMenu)
	);

	if (UMovieScene* MovieScene = Sequence.IsValid() ? Sequence->GetMovieScene() : nullptr)
	{
		if (MovieScene->IsConditionClassAllowed(UMovieSceneDirectorBlueprintCondition::StaticClass()))
		{
			// Option to use a director blueprint condition and create or quick bind to an endpoint
			MenuBuilder.AddSubMenu(
				LOCTEXT("ConditionDirectorBlueprint", "Director Blueprint Condition..."),
				LOCTEXT("ConditionDirectorBlueprintTooltip", "Use a director blueprint function as a condition"),
				FNewMenuDelegate::CreateSP(this, &FMovieSceneConditionCustomization::FillDirectorBlueprintConditionSubMenu)
			);
		}
	}



	OpenMenuWidget = MenuBuilder.MakeWidget().ToSharedPtr();
	return OpenMenuWidget.ToSharedRef();
}

UMovieSceneSequence* FMovieSceneConditionCustomization::GetCommonSequence() const
{
	TArray<UObject*> EditObjects;
	ConditionContainerPropertyHandle->GetOuterObjects(EditObjects);

	UMovieSceneSequence* CommonSequence = nullptr;

	for (UObject* Obj : EditObjects)
	{
		UMovieSceneSequence* ThisSequence = Obj ? Obj->GetTypedOuter<UMovieSceneSequence>() : nullptr;
		if (CommonSequence && CommonSequence != ThisSequence)
		{
			return nullptr;
		}

		CommonSequence = ThisSequence;
	}
	return CommonSequence;
}

UMovieSceneTrack* FMovieSceneConditionCustomization::GetCommonTrack() const
{
	TArray<UObject*> EditObjects;
	ConditionContainerPropertyHandle->GetOuterObjects(EditObjects);

	UMovieSceneTrack* CommonTrack = nullptr;

	for (UObject* Obj : EditObjects)
	{
		UMovieSceneTrack* ThisTrack = Cast<UMovieSceneTrack>(Obj);
		if (!ThisTrack)
		{
			ThisTrack = Obj ? Obj->GetTypedOuter<UMovieSceneTrack>() : nullptr;
		}

		if (!ThisTrack)
		{
			// Special case
			if (UMovieSceneTrackRowMetadataHelper* TrackRowHelper = Cast<UMovieSceneTrackRowMetadataHelper>(Obj))
			{
				ThisTrack = TrackRowHelper->OwnerTrack.Get();
			}
		}

		if (CommonTrack && CommonTrack != ThisTrack)
		{
			return nullptr;
		}

		CommonTrack = ThisTrack;
	}
	return CommonTrack;
}

#undef LOCTEXT_NAMESPACE

