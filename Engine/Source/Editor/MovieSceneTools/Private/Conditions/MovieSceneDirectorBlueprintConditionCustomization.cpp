// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneDirectorBlueprintConditionCustomization.h"

#include "MovieScene.h"
#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionUtils.h"
#include "MovieSceneSequence.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "EdGraphSchema_K2.h"
#include "IPropertyUtilities.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MovieSceneDirectorBlueprintConditionCustomization"

TSharedRef<IPropertyTypeCustomization> FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance()
{
	TSharedRef<FMovieSceneDirectorBlueprintConditionCustomization> Instance = MakeShared<FMovieSceneDirectorBlueprintConditionCustomization>();
	return Instance;
}

TSharedRef<IPropertyTypeCustomization> FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(UMovieScene* InMovieScene)
{
	TSharedRef<FMovieSceneDirectorBlueprintConditionCustomization> Instance = MakeShared<FMovieSceneDirectorBlueprintConditionCustomization>();
	Instance->EditedMovieScene = InMovieScene;
	return Instance;
}

TSharedRef<FMovieSceneDirectorBlueprintConditionCustomization> FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(UMovieScene* InMovieScene, TSharedPtr<IPropertyHandle> InPropertyHandle, TSharedPtr<IPropertyUtilities> InPropertyUtilities)
{
	TSharedRef<FMovieSceneDirectorBlueprintConditionCustomization> Instance = MakeShared<FMovieSceneDirectorBlueprintConditionCustomization>();
	Instance->EditedMovieScene = InMovieScene;
	Instance->SetPropertyHandle(InPropertyHandle);
	Instance->PropertyUtilities = InPropertyUtilities;
	return Instance;
}

void FMovieSceneDirectorBlueprintConditionCustomization::GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const
{
	const FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData);
	for (const TPair<FName, FMovieSceneDirectorBlueprintConditionPayloadVariable>& Pair : DirectorBlueprintConditionData->PayloadVariables)
	{
		OutPayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
	}
}

bool FMovieSceneDirectorBlueprintConditionCustomization::SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewVariableValue)
{
	UMovieScene* MovieScene = Cast<UMovieScene>(EditObject);
	FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData);
	if (!MovieScene || !DirectorBlueprintConditionData)
	{
		return false;
	}

	MovieScene->Modify();

	if (!NewVariableValue.Value.IsEmpty())
	{
		FMovieSceneDirectorBlueprintConditionPayloadVariable* PayloadVariable = DirectorBlueprintConditionData->PayloadVariables.Find(FieldName);
		if (!PayloadVariable)
		{
			PayloadVariable = &DirectorBlueprintConditionData->PayloadVariables.Add(FieldName);
		}

		PayloadVariable->Value = NewVariableValue.Value;
		PayloadVariable->ObjectValue = NewVariableValue.ObjectValue;
	}
	else
	{
		DirectorBlueprintConditionData->PayloadVariables.Remove(FieldName);
	}

	return true;
}

UK2Node* FMovieSceneDirectorBlueprintConditionCustomization::FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const
{
	FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData);
	if (UK2Node* Node = Cast<UK2Node>(DirectorBlueprintConditionData->WeakEndpoint.Get()))
	{
		return Node;
	}
	return nullptr;
}

void FMovieSceneDirectorBlueprintConditionCustomization::GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutWellKnownParameters) const
{
	FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData);
	OutWellKnownParameters.Add(DirectorBlueprintConditionData->ConditionContextPinName);
}

void FMovieSceneDirectorBlueprintConditionCustomization::GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const
{
	FWellKnownParameterCandidates ConditionContextCandidates;
	ConditionContextCandidates.Metadata.PickerLabel = LOCTEXT("ConditionContextParamsPin_Label", "Pass Condition Context To");
	ConditionContextCandidates.Metadata.PickerTooltip = LOCTEXT("ContextContextParamsPin_Tooltip", "Specifies a pin to pass the condition context through when the condition is evaluated.");

	for (UEdGraphPin* Pin : Endpoint->Pins)
	{
		// Parameter pins are outputs on the function entry node.
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		// Pin of type FMovieSceneConditionContext is eligible for passing the params.
		if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
			Pin->PinType.PinSubCategoryObject == FMovieSceneConditionContext::StaticStruct())
		{
			ConditionContextCandidates.CandidatePinNames.Add(Pin->GetFName());
		}
	}

	OutCandidates.Add(ConditionContextCandidates);
}

bool FMovieSceneDirectorBlueprintConditionCustomization::SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName)
{
	FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData);
	switch (ParameterIndex)
	{
	case 0:
		DirectorBlueprintConditionData->ConditionContextPinName = BoundPinName;
		return true;
	}
	return false;
}

FMovieSceneDirectorBlueprintEndpointDefinition FMovieSceneDirectorBlueprintConditionCustomization::GenerateEndpointDefinition(UMovieSceneSequence* Sequence)
{
	FMovieSceneDirectorBlueprintEndpointDefinition Definition;
	Definition.EndpointType = EMovieSceneDirectorBlueprintEndpointType::Function;

	// We use a dummy utility class to get a function signature that takes the condition context parameter and returns a bool
	static const FName SampleDirectorBlueprintConditionFuncName("SampleDirectorBlueprintCondition");
	UClass* EndpointUtilClass = UMovieSceneDirectorBlueprintConditionEndpointUtil::StaticClass();
	Definition.EndpointSignature = EndpointUtilClass->FindFunctionByName(SampleDirectorBlueprintConditionFuncName);
	check(Definition.EndpointSignature);

	Definition.EndpointName = "EvaluateCondition";

	return Definition;
}

void FMovieSceneDirectorBlueprintConditionCustomization::OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	MovieScene->Modify();

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		ensureMsgf(
			Cast<UMovieScene>(EditObjects[Index]) == MovieScene,
			TEXT("Editing director blueprint condition endpoint for a different sequence"));

		FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData[Index]);

		// Default call in editor to true
		if (UK2Node_FunctionEntry* CallFunction = Cast<UK2Node_FunctionEntry>(NewEndpoint))
		{
			CallFunction->MetaData.bCallInEditor = true;
		}

		FMovieSceneDirectorBlueprintConditionUtils::SetEndpoint(MovieScene, DirectorBlueprintConditionData, NewEndpoint);

		// If we have a candidate for the condition context pin, set it automatically
		UK2Node* CommonEndpoint = GetCommonEndpoint();
		if (CommonEndpoint)
		{
			TArray<FWellKnownParameterCandidates> WellKnownParameterCandidates;
			GetWellKnownParameterCandidates(CommonEndpoint, WellKnownParameterCandidates);
			for(int32 ParameterIndex = 0; ParameterIndex < WellKnownParameterCandidates.Num(); ++ParameterIndex)
			{
				const FWellKnownParameterCandidates& Candidate = WellKnownParameterCandidates[ParameterIndex];
				if (Candidate.CandidatePinNames.Num() > 0)
				{
					// Pick the first one
					SetWellKnownParameterPinName(EditObjects[Index], RawData[Index], ParameterIndex, Candidate.CandidatePinNames[0]);
				}
			}
		}
	}

	FMovieSceneDirectorBlueprintConditionUtils::EnsureBlueprintExtensionCreated(Sequence, Blueprint);
}

void FMovieSceneDirectorBlueprintConditionCustomization::OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	check(EditObjects.Num() == RawData.Num());

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		UMovieScene* MovieScene = Cast<UMovieScene>(EditObjects[Index]);
		FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData = static_cast<FMovieSceneDirectorBlueprintConditionData*>(RawData[Index]);

		FMovieSceneDirectorBlueprintConditionUtils::SetEndpoint(MovieScene, DirectorBlueprintConditionData, NewEndpoint);

		// If we have a candidate for the condition context pin, set it automatically
		UK2Node* CommonEndpoint = GetCommonEndpoint();
		if (CommonEndpoint)
		{
			TArray<FWellKnownParameterCandidates> WellKnownParameterCandidates;
			GetWellKnownParameterCandidates(CommonEndpoint, WellKnownParameterCandidates);
			for (int32 ParameterIndex = 0; ParameterIndex < WellKnownParameterCandidates.Num(); ++ParameterIndex)
			{
				const FWellKnownParameterCandidates& Candidate = WellKnownParameterCandidates[ParameterIndex];
				if (Candidate.CandidatePinNames.Num() > 0)
				{
					// Pick the first one
					SetWellKnownParameterPinName(EditObjects[Index], RawData[Index], ParameterIndex, Candidate.CandidatePinNames[0]);
				}
			}
		}

		FMovieSceneDirectorBlueprintConditionUtils::EnsureBlueprintExtensionCreated(Sequence, Blueprint);
	}
}

void FMovieSceneDirectorBlueprintConditionCustomization::GetEditObjects(TArray<UObject*>& OutObjects) const
{
	OutObjects.Add(EditedMovieScene);
}

void FMovieSceneDirectorBlueprintConditionCustomization::OnCollectQuickBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder)
{
	CollectConditionBindActions(Blueprint, MenuBuilder, false);
}

void FMovieSceneDirectorBlueprintConditionCustomization::CollectConditionBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder, bool bIsRebinding)
{
	// We don't show the resolver library endpoints for rebinding, because we should only rebind
	// to other function graphs of the director blueprint.
	if (bIsRebinding)
	{
		return;
	}

	// We want the ability to create CallFunction nodes for any static method that we think can be used
	// as a condition function.
	FBlueprintActionFilter MenuFilter(FBlueprintActionFilter::BPFILTER_RejectGlobalFields | FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);
	MenuFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
	MenuFilter.Context.Blueprints.Add(Blueprint);

	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* CurrentClass = *ClassIt;
		FBlueprintActionFilter::Add(MenuFilter.TargetClasses, CurrentClass);
	}

	auto RejectAnyIncompatibleReturnValues = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		const UFunction* Function = BlueprintAction.GetAssociatedFunction();
		const FBoolProperty* FunctionReturnProperty = CastField<FBoolProperty>(Function->GetReturnProperty());
		return FunctionReturnProperty == nullptr;
	};

	MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateLambda(RejectAnyIncompatibleReturnValues));

	MenuBuilder.AddMenuSection(MenuFilter, LOCTEXT("DirectorBlueprintConditionCustomization", "Condition Library"), 0);
}

#undef LOCTEXT_NAMESPACE

