// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneDirectorBlueprintConditionExtension.h"

#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionUtils.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDirectorBlueprintConditionExtension)

class UEdGraphNode;

void UMovieSceneDirectorBlueprintConditionExtension::BindTo(TWeakObjectPtr<UMovieSceneSequence> InMovieSceneSequence)
{
	WeakMovieSceneSequences.AddUnique(InMovieSceneSequence);
}

void UMovieSceneDirectorBlueprintConditionExtension::PostLoad()
{
	WeakMovieSceneSequences.Remove(nullptr);
	Super::PostLoad();
}

void UMovieSceneDirectorBlueprintConditionExtension::HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint)
{
	for (TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence : WeakMovieSceneSequences)
	{
		if (UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get())
		{
			UBlueprint::ForceLoad(MovieSceneSequence);
			if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
			{
				UBlueprint::ForceLoad(MovieScene);
			}
		}
	}
}

void UMovieSceneDirectorBlueprintConditionExtension::HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext)
{
	for (TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence : WeakMovieSceneSequences)
	{
		UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get();
		HandleGenerateFunctionGraphs(CompilerContext, MovieSceneSequence);
	}
}

void UMovieSceneDirectorBlueprintConditionExtension::HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext, UMovieSceneSequence* MovieSceneSequence)
{
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	ensureMsgf(!MovieScene->HasAnyFlags(RF_NeedLoad), TEXT("Attempting to generate entry point functions before a movie scene has been loaded"));

	// Generate a function graph for each endpoint used by the bound sequence. This function graph is simply a call
	// to the endpoint function with the payload variables set as the call parameters.
	auto GenerateFunctionGraphs =
		[CompilerContext]
	(FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintConditionData)
	{
		UEdGraphNode* Endpoint = Cast<UK2Node>(DirectorBlueprintConditionData.WeakEndpoint.Get());
		if (Endpoint)
		{
			// Set up the endpoint call, with our payload variables.
			FMovieSceneDirectorBlueprintEndpointCall EndpointCall;
			EndpointCall.Endpoint = Endpoint;
			if (!DirectorBlueprintConditionData.ConditionContextPinName.IsNone())
			{
				EndpointCall.ExposedPinNames.Add(DirectorBlueprintConditionData.ConditionContextPinName);
			}
			for (auto& Pair : DirectorBlueprintConditionData.PayloadVariables)
			{
				EndpointCall.PayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
			}

			// Create the endpoint call, and clean-up stale payload variables.
			FMovieSceneDirectorBlueprintEntrypointResult EntrypointResult = FMovieSceneDirectorBlueprintUtils::GenerateEntryPoint(EndpointCall, CompilerContext);
			DirectorBlueprintConditionData.CompiledFunctionName = EntrypointResult.CompiledFunctionName;
			EntrypointResult.CleanUpStalePayloadVariables(DirectorBlueprintConditionData.PayloadVariables);
		}
	};
	FMovieSceneDirectorBlueprintConditionUtils::IterateDirectorBlueprintConditions(MovieScene, GenerateFunctionGraphs);

	// This callback sets the generated function calls back onto the director blueprint conditions in the sequence,
	// and keeps a pointer to any needed arguments on this function, so that we can pass special values later
	// at runtime.
	TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence(MovieSceneSequence);
	auto OnFunctionListGenerated = [WeakMovieSceneSequence](FKismetCompilerContext* CompilerContext)
	{
		UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get();
		UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
		if (!ensureMsgf(MovieSceneSequence, TEXT("A movie scene was garbage-collected while its director blueprint was being compiled!")))
		{
			return;
		}

		UBlueprint* Blueprint = CompilerContext->Blueprint;
		if (!Blueprint->GeneratedClass)
		{
			return;
		}

		TArray<FMovieSceneDirectorBlueprintConditionData*> DirectorBlueprintConditionDatas;
		FMovieSceneDirectorBlueprintConditionUtils::GatherDirectorBlueprintConditions(MovieScene, DirectorBlueprintConditionDatas);
		for (FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData : DirectorBlueprintConditionDatas)
		{
			// Set the pointer to the resolver function to invoke.
			if (DirectorBlueprintConditionData->CompiledFunctionName != NAME_None)
			{
				DirectorBlueprintConditionData->Function = Blueprint->GeneratedClass->FindFunctionByName(DirectorBlueprintConditionData->CompiledFunctionName);
			}
			else
			{
				DirectorBlueprintConditionData->Function = nullptr;
			}

			DirectorBlueprintConditionData->CompiledFunctionName = NAME_None;

			// Set the pointer to the resolve parameter field, if any.
			if (DirectorBlueprintConditionData->Function && !DirectorBlueprintConditionData->ConditionContextPinName.IsNone())
			{
				FProperty* ConditionContextProp = DirectorBlueprintConditionData->Function->FindPropertyByName(DirectorBlueprintConditionData->ConditionContextPinName);
				DirectorBlueprintConditionData->ConditionContextProperty = ConditionContextProp;
			}
			else
			{
				DirectorBlueprintConditionData->ConditionContextProperty = nullptr;
			}
		}

		if (!Blueprint->bIsRegeneratingOnLoad)
		{
			MovieScene->MarkAsChanged();
			MovieScene->MarkPackageDirty();
		}
	};

	CompilerContext->OnFunctionListCompiled().AddLambda(OnFunctionListGenerated);
}

