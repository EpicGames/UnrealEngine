// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintExtension.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneDirectorBlueprintConditionExtension.generated.h"

class FKismetCompilerContext;
class UBlueprint;
class UMovieSceneSequence;
class UObject;
struct FMovieSceneCustomizeDirectorBlueprintEndpointCallParams;

/**
 * An extension for a sequence's director blueprint to compile director blueprint condition endpoints.
 */
UCLASS()
class UMovieSceneDirectorBlueprintConditionExtension : public UBlueprintExtension
{
public:

	GENERATED_BODY()

	/**
	 * Bind this extension to the given sequence. This may be called multiple times, but the extension
	 * must only be bound to one sequence.
	 */
	void BindTo(TWeakObjectPtr<UMovieSceneSequence> InMovieSceneSequence);

private:

	virtual void PostLoad() override;

	virtual void HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint) override final;
	virtual void HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext) override final;

	void HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext, UMovieSceneSequence* MovieSceneSequence);

private:

	/** List of movie scenes that are bound to the director blueprint */
	UPROPERTY()
	TArray<TWeakObjectPtr<UMovieSceneSequence>> WeakMovieSceneSequences;
};

