// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneDirectorBlueprintConditionUtils.h"

#include "K2Node.h"
#include "K2Node_FunctionEntry.h"
#include "MovieSceneSequence.h"
#include "MovieSceneDirectorBlueprintConditionExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDirectorBlueprintConditionUtils)

void FMovieSceneDirectorBlueprintConditionUtils::SetEndpoint(UMovieScene* MovieScene, FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData, UK2Node* NewEndpoint)
{
	UK2Node* ExistingEndpoint = CastChecked<UK2Node>(DirectorBlueprintConditionData->WeakEndpoint.Get(), ECastCheckedType::NullAllowed);
	if (ExistingEndpoint)
	{
		ExistingEndpoint->OnUserDefinedPinRenamed().RemoveAll(MovieScene);
	}

	if (NewEndpoint)
	{
		checkf(
			NewEndpoint->IsA<UK2Node_FunctionEntry>(),
			TEXT("Only functions are supported as dynamic binding endpoints"));

		NewEndpoint->OnUserDefinedPinRenamed().AddUObject(MovieScene, &UMovieScene::OnDirectorBlueprintConditionUserDefinedPinRenamed);
		DirectorBlueprintConditionData->WeakEndpoint = NewEndpoint;
	}
	else
	{
		DirectorBlueprintConditionData->WeakEndpoint = nullptr;
	}
}

void FMovieSceneDirectorBlueprintConditionUtils::EnsureBlueprintExtensionCreated(UMovieSceneSequence* MovieSceneSequence, UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	check(MovieSceneSequence);

	for (const TObjectPtr<UBlueprintExtension>& Extension : Blueprint->GetExtensions())
	{
		UMovieSceneDirectorBlueprintConditionExtension* DirectorBlueprintConditionExtension = Cast<UMovieSceneDirectorBlueprintConditionExtension>(Extension);
		if (DirectorBlueprintConditionExtension)
		{
			DirectorBlueprintConditionExtension->BindTo(MovieSceneSequence);
			return;
		}
	}

	UMovieSceneDirectorBlueprintConditionExtension* DirectorBlueprintConditionExtension = NewObject<UMovieSceneDirectorBlueprintConditionExtension>(Blueprint);
	DirectorBlueprintConditionExtension->BindTo(MovieSceneSequence);
	Blueprint->AddExtension(DirectorBlueprintConditionExtension);
}

