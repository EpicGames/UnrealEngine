// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieScenePlatformConditionCustomization.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "Conditions/MovieScenePlatformCondition.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Textures/SlateIcon.h"
#include "SCheckBoxList.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
 
#define LOCTEXT_NAMESPACE "MovieSceneDynamicBindingCustomization"

TSharedRef<IDetailCustomization> FMovieScenePlatformConditionCustomization::MakeInstance()
{
	return MakeShareable(new FMovieScenePlatformConditionCustomization);
}

void FMovieScenePlatformConditionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ValidPlatformsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieScenePlatformCondition, ValidPlatforms), UMovieScenePlatformCondition::StaticClass());
	ValidPlatformsPropertyHandle->MarkHiddenByCustomization();
	IDetailCategoryBuilder& PlatformsCategory = DetailBuilder.EditCategory(TEXT("Valid Platforms"));

	const TArray<const FDataDrivenPlatformInfo*>& PlatformInfos = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);

	auto GetComboButtonText = [this, SharedThis = StaticCastSharedRef<FMovieScenePlatformConditionCustomization>(AsShared())]() -> FText
	{
		TArray<FName> CurrentValidPlatformNames = GetCurrentValidPlatformNames();
		TArray<FText> CurrentValidPlatforms;
		CurrentValidPlatforms.Reserve(CurrentValidPlatformNames.Num());

		for (const FName& PlatformName : CurrentValidPlatformNames)
		{
			CurrentValidPlatforms.Add(FText::FromName(PlatformName));
		}
		if (CurrentValidPlatforms.Num() > 3)
		{
			CurrentValidPlatforms.SetNum(3);
			CurrentValidPlatforms.Add(FText::FromString("..."));
		}

		return FText::Join(FText::FromString(", "), CurrentValidPlatforms);
	};

	PlatformsCategory.AddCustomRow(LOCTEXT("ValidPlatforms", "Valid Platforms"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ValidPlatforms", "Valid Platforms"))
			.ToolTipText(LOCTEXT("ValidPlatformsTooltip", "Which platforms will pass the condition"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.ButtonContent()
				[
					SNew(STextBlock)
					.Text_Lambda(GetComboButtonText)
				]
			.OnGetMenuContent_Lambda([&DetailBuilder, this, SharedThis = StaticCastSharedRef<FMovieScenePlatformConditionCustomization>(AsShared()), PlatformInfos]()
				{
					TArray<FName> CurrentValidPlatformNames = GetCurrentValidPlatformNames();

					SharedThis->CheckBoxList = SNew(SCheckBoxList)
						.OnItemCheckStateChanged(this, &FMovieScenePlatformConditionCustomization::OnPlatformCheckChanged)
						.IncludeGlobalCheckBoxInHeaderRow(false);

					for (int32 i = 0; i < PlatformInfos.Num(); ++i)
					{
						TSharedRef< SWidget > IconWidget = SNullWidget::NullWidget;
						FSlateIcon SlateIcon(FAppStyle::GetAppStyleSetName(), PlatformInfos[i]->GetIconStyleName(EPlatformIconSize::Normal));
						const FSlateBrush* IconBrush = SlateIcon.GetIcon();
						if (IconBrush->GetResourceName() != NAME_None)
						{
							IconWidget = SNew(SImage)
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								.Image(IconBrush);
						}
						const float MenuIconSize = FAppStyle::Get().GetFloat(FAppStyle::GetAppStyleSetName(), ".MenuIconSize", 16.f);

						SharedThis->CheckBoxList->AddItem(
							SNew(SHorizontalBox)
								// Whatever we have in the icon area goes first
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(FMargin(2, 0, 6, 0))
								[
									SNew(SBox)
										.Visibility(IconWidget != SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed)
										.WidthOverride(MenuIconSize + 2)
										.HeightOverride(MenuIconSize)
										.HAlign(HAlign_Center)
										.VAlign(VAlign_Center)
										[
											SNew(SBox)
												.WidthOverride(MenuIconSize)
												.HeightOverride(MenuIconSize)
												[
													IconWidget
												]
										]
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(FMargin(2, 0, 6, 0))
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), ISlateStyle::Join("Menu", ".Label"))
										.Text(FText::FromName(PlatformInfos[i]->IniPlatformName))
								],
							CurrentValidPlatformNames.Contains(PlatformInfos[i]->IniPlatformName));
					}
					return SharedThis->CheckBoxList.ToSharedRef();
				})
			.OnMenuOpenChanged_Lambda([this, SharedThis = StaticCastSharedRef<FMovieScenePlatformConditionCustomization>(AsShared())](const bool IsOpen)
				{
					if (!IsOpen)
					{
						ValidPlatformsPropertyHandle->NotifyFinishedChangingProperties();
					}
				})
		]; 
}

TArray<FName> FMovieScenePlatformConditionCustomization::GetCurrentValidPlatformNames()
{
	TArray<FName> Names;

	TArray<void*> RawData;
	ValidPlatformsPropertyHandle->AccessRawData(RawData);

	if (RawData.Num() > 0)
	{
		if (TArray<FName>* CurrentValidPlatformNamesPtr = reinterpret_cast<TArray<FName>*>(RawData[0]))
		{
			Names = *CurrentValidPlatformNamesPtr;
		}
	}
	return Names;
};

void FMovieScenePlatformConditionCustomization::OnPlatformCheckChanged(int32 Index)
{
	if (CheckBoxList.IsValid())
	{
		TArray<FName> CurrentValidPlatformNames = GetCurrentValidPlatformNames(); 
		const TArray<const FDataDrivenPlatformInfo*>& PlatformInfos = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);

		TArray<FName> NewValidPlatformNames = CurrentValidPlatformNames;

		if (Index == -1)
		{
			for (int32 PlatformIndex = 0; PlatformIndex < PlatformInfos.Num(); PlatformIndex++)
			{
				if (CheckBoxList->IsItemChecked(PlatformIndex))
				{
					NewValidPlatformNames.AddUnique(PlatformInfos[PlatformIndex]->IniPlatformName);
				}
				else
				{
					NewValidPlatformNames.Remove(PlatformInfos[PlatformIndex]->IniPlatformName);
				}
			}
		}
		else
		{
			if (CheckBoxList->IsItemChecked(Index))
			{
				NewValidPlatformNames.Add(PlatformInfos[Index]->IniPlatformName);
			}
			else
			{
				NewValidPlatformNames.Remove(PlatformInfos[Index]->IniPlatformName);
			}
		}

		TArray<void*> RawData;
		ValidPlatformsPropertyHandle->AccessRawData(RawData);
		if (RawData.Num() == 1)
		{
			if (TArray<FName>* CurrentValidPlatformNamesPtr = reinterpret_cast<TArray<FName>*>(RawData[0]))
			{
				ValidPlatformsPropertyHandle->NotifyPreChange();
				*CurrentValidPlatformNamesPtr = NewValidPlatformNames;
				ValidPlatformsPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE