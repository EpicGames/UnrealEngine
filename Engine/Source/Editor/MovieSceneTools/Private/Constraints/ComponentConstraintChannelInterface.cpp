// Copyright Epic Games, Inc. All Rights Reserved.


#include "ComponentConstraintChannelInterface.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneToolHelpers.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformConstraint.h"
#include "Constraints/MovieSceneConstraintChannelHelper.inl"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Sections/MovieSceneConstrainedSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "ConstraintsManager.h"
#include "ScopedTransaction.h"

UMovieSceneSection* FComponentConstraintChannelInterface::GetHandleSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	const UTransformableComponentHandle* ComponentHandle = static_cast<const UTransformableComponentHandle*>(InHandle);
	static constexpr bool bConstraintSection = false;
	return GetComponentSection(ComponentHandle, InSequencer, bConstraintSection);
}

UMovieSceneSection* FComponentConstraintChannelInterface::GetHandleConstraintSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	const UTransformableComponentHandle* ComponentHandle = static_cast<const UTransformableComponentHandle*>(InHandle);
	static constexpr bool bConstraintSection = true;
	return GetComponentSection(ComponentHandle, InSequencer, bConstraintSection);
}

UWorld* FComponentConstraintChannelInterface::GetHandleWorld(UTransformableHandle* InHandle)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	const UTransformableComponentHandle* ComponentHandle = static_cast<const UTransformableComponentHandle*>(InHandle);
	const AActor* Actor = ComponentHandle->Component->GetOwner();
	return Actor ? Actor->GetWorld() : nullptr;
}

bool FComponentConstraintChannelInterface::SmartConstraintKey(
	UTickableTransformConstraint* InConstraint,
	const TOptional<bool>& InOptActive,
	const TRange<FFrameNumber>& InTimeRange,
	const TSharedPtr<ISequencer>& InSequencer)
{
	if (!IsValid(InConstraint->ChildTRSHandle) || !InConstraint->ChildTRSHandle->IsValid() || !InSequencer.IsValid())
	{
		return false;
	}
	
	const UTransformableComponentHandle* ComponentHandle = static_cast<UTransformableComponentHandle*>(InConstraint->ChildTRSHandle);

	UMovieScene3DTransformSection* ConstraintSection = GetComponentSection(ComponentHandle, InSequencer, true);
	UMovieScene3DTransformSection* TransformSection = GetComponentSection(ComponentHandle, InSequencer, false);
	if ((!ConstraintSection) || (!TransformSection))
	{
		return false;
	}

	// TODO handle range
	const bool bInfiniteInput = !InTimeRange.HasLowerBound();
	const TOptional<FFrameNumber> Time = bInfiniteInput ? TOptional<FFrameNumber>() : InTimeRange.GetLowerBoundValue();

	FScopedTransaction Transaction(NSLOCTEXT("Constraints", "KeyConstraintKey", "Key Constraint Key"));
	TransformSection->Modify();
	ConstraintSection->Modify();

	// set constraint as dynamic
	InConstraint->bDynamicOffset = true;
	
	// add the channel
	const bool bHadConstraintChannel = ConstraintSection->HasConstraintChannel(InConstraint->ConstraintID);
	if (!bHadConstraintChannel)
	{
		//check if static if so we need to delete it from world, will get added later again
		if (UConstraintsManager* Manager = InConstraint->GetTypedOuter<UConstraintsManager>())
		{
			Manager->RemoveStaticConstraint(InConstraint);
		}
		ConstraintSection->AddConstraintChannel(InConstraint);
		InSequencer->RecreateCurveEditor();
	}

	// add key if needed
	if (FConstraintAndActiveChannel* Channel = ConstraintSection->GetConstraintChannel(InConstraint->ConstraintID))
	{
		if (bInfiniteInput)
		{
			ensure(Channel->ActiveChannel.GetNumKeys() == 0);
		}

		bool ActiveValueToBeSet = bInfiniteInput;
		//add key if we can and make sure the key we are setting is what we want
		if (bInfiniteInput || (CanAddKey(Channel->ActiveChannel, *Time, ActiveValueToBeSet) && (!InOptActive.IsSet() || InOptActive.GetValue() == ActiveValueToBeSet)))
		{
			const bool bNeedsCompensation = InConstraint->NeedsCompensation();

			//new for compensation
			TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			TGuardValue<bool> RemoveConstraintGuard(FConstraintsManagerController::bDoNotRemoveConstraint, true);

			// store the frames to compensate
			const TArrayView<FMovieSceneDoubleChannel*> Channels = ComponentHandle->GetDoubleChannels(TransformSection);
			TArray<FFrameNumber> FramesToCompensate;
			if (bNeedsCompensation)
			{
				FMovieSceneConstraintChannelHelper::GetFramesToCompensate<FMovieSceneDoubleChannel>(Channel->ActiveChannel, ActiveValueToBeSet, Time, Channels, FramesToCompensate);
			}
			else
			{
				if (!bInfiniteInput)
				{
					FramesToCompensate.Add(*Time);
				}
			}

			const bool bIsActiveByDefault = Channel->ActiveChannel.GetDefault() && *Channel->ActiveChannel.GetDefault();
			const bool bWasInfinite = bIsActiveByDefault && Channel->ActiveChannel.GetNumKeys() == 0;
			
			const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
			const FFrameTime CurrentTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);
			const FFrameNumber CurrentFrame = CurrentTime.GetFrame();
			
			TOptional<FFrameNumber> OptCurrentTime;
			bool bPropagateRelative = true;
			if (!bHadConstraintChannel && ActiveValueToBeSet)
			{
				if (!bInfiniteInput && !bWasInfinite)
				{
					// check if the first active key will be added before the current frame
					// if that's the case then the current frame must be added and the compensation evaluator notified
					Tie(OptCurrentTime, bPropagateRelative) =
						IsActiveKeyAddedBeforeCurrentTime(CurrentFrame, *Time, Channels, FramesToCompensate);
				}
				else if (bInfiniteInput && !FramesToCompensate.IsEmpty())
				{
					// make sure to add the current frame first if the child has transforms to compensate
					FramesToCompensate.Remove(CurrentFrame);
					FramesToCompensate.Insert(CurrentFrame, 0);
					OptCurrentTime = CurrentFrame;
					bPropagateRelative = false;
				}
			}

			FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, FramesToCompensate);
			EvalParams.bToActive = ActiveValueToBeSet;
			EvalParams.OptCurrentFrame = OptCurrentTime;
			EvalParams.bPropagateRelative = bPropagateRelative;

			if (bWasInfinite)
			{
				EvalParams.bToActive = false;
			}
			else if (bIsActiveByDefault && Time)
			{
				const TMovieSceneChannelData<const bool> ChannelData = Channel->ActiveChannel.GetData();
				const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
				const TArrayView<const bool> Values = ChannelData.GetValues();
				
				const bool bInfiniteAtLeft = *Time < Times[0] && !Values[0];
				EvalParams.bInfiniteAtLeft = bInfiniteAtLeft;
			}
			
			if (bHadConstraintChannel && ActiveValueToBeSet && !bWasInfinite)
			{
				const TMovieSceneChannelData<const bool> ChannelData = Channel->ActiveChannel.GetData();
				if (ChannelData.GetTimes().IsEmpty() && !bInfiniteInput)
				{
					// this means that the channel already existed but there was no existing key
					// e.g., when the user deletes keys directly in sequencer without deleting the constraint
					EvalParams.bKeepCurrent = !FramesToCompensate.IsEmpty() && FramesToCompensate[0] == CurrentTime.GetFrame();
				}
			}
			
			// store child and space transforms for these frames
			FCompensationEvaluator Evaluator(InConstraint);
			Evaluator.ComputeLocalTransforms(GetHandleWorld(InConstraint->ChildTRSHandle), EvalParams);
			TArray<FTransform>& ChildLocals = Evaluator.ChildLocals;

			// store tangents at this time
			TArray<FMovieSceneTangentData> Tangents;
			static constexpr int32 ChannelIndex = 0, NumChannels = 9;
			
			if (bNeedsCompensation && !bInfiniteInput)
			{
				EvaluateTangentAtThisTime<FMovieSceneDoubleChannel>(ChannelIndex, NumChannels, TransformSection, *Time, Tangents);
			}

			const EMovieSceneTransformChannel ChannelsToKey = InConstraint->GetChannelsToKey();

			// add child's transform key at Time-1 to keep animation
			if (bNeedsCompensation && !bInfiniteInput)
			{
				const FFrameNumber TimeMinusOne(*Time - 1);

				TArray<FTransform> TransformMinusOne({ChildLocals[0]} );
				if (ConstraintSection != TransformSection)
				{
					RecomposeTransforms(InSequencer, ComponentHandle->Component.Get(), TransformSection, { TimeMinusOne },TransformMinusOne);
				}
				MovieSceneToolHelpers::AddTransformKeys(TransformSection, { TimeMinusOne }, TransformMinusOne, ChannelsToKey);

				SetTangentsAtThisTime<FMovieSceneDoubleChannel>(ChannelIndex, NumChannels, TransformSection, TimeMinusOne, Tangents);
			}

			// add active key
			if (bInfiniteInput)
			{
				constexpr bool bActiveByDefault = true;
				Channel->ActiveChannel.SetDefault(bActiveByDefault);
			}
			else
			{
				TMovieSceneChannelData<bool> ChannelData = Channel->ActiveChannel.GetData();

				const TArrayView<const FFrameNumber> FrameNumbers = ChannelData.GetTimes();
				const bool bRemoveInfinite = bIsActiveByDefault && ActiveValueToBeSet && (FrameNumbers.IsEmpty() || *Time < FrameNumbers[0]); 
				
				ChannelData.UpdateOrAddKey(*Time, ActiveValueToBeSet);

				if (bRemoveInfinite)
				{
					Channel->ActiveChannel.RemoveDefault();
				}
			}

			// compensate
			{
				// we need to remove the first transforms as we store NumFrames+1 transforms
				if (!ChildLocals.IsEmpty())
				{
					ChildLocals.RemoveAt(0);
				}

				if (ConstraintSection != TransformSection)
				{
					RecomposeTransforms(InSequencer, ComponentHandle->Component.Get(), TransformSection, FramesToCompensate, ChildLocals);
				}

				// add keys
				MovieSceneToolHelpers::AddTransformKeys(TransformSection, FramesToCompensate, ChildLocals, ChannelsToKey);

				// set tangents at Time
				if (bNeedsCompensation && !bInfiniteInput)
				{
					SetTangentsAtThisTime<FMovieSceneDoubleChannel>(ChannelIndex, NumChannels, TransformSection, *Time, Tangents);
				}
			}

			// cleanup next keys that have the same value as they are useless
			if (!bInfiniteInput)
			{
				TArray<FFrameNumber> TimesRemoved;
				CleanDuplicates(Channel->ActiveChannel, TimesRemoved);

				// as compensation has been disabled earlier (FMovieSceneConstraintChannelHelper::bDoNotCompensate set to true),
				// previous transform compensation keys must be explicitly removed as HandleConstraintKeyDeleted won't do it
				for (const FFrameNumber& TimeToRemove: TimesRemoved)
				{
					const FFrameNumber TimeMinusOne(TimeToRemove - 1);
					FMovieSceneConstraintChannelHelper::DeleteTransformKeys(Channels, TimeMinusOne);
				}
			}
			
			// evaluate the constraint, this is needed so the global transform will be set up on the component 
			//Todo do we need to evaluate all constraints?
			InConstraint->SetActive(true); //will be false
			InConstraint->Evaluate();

			//need to fire this event so the transform values set by the constraint propragate to the transform section
			//first turn off autokey though
			EAutoChangeMode AutoChangeMode = InSequencer->GetAutoChangeMode();
			if (AutoChangeMode == EAutoChangeMode::AutoKey || AutoChangeMode == EAutoChangeMode::All)
			{
				InSequencer->SetAutoChangeMode(EAutoChangeMode::None);
			};

			AActor* Actor = ComponentHandle->Component->GetOwner();
			FProperty* TransformProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(TransformProperty);
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(Actor, PropertyChain);
			FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, PropertyChangedEvent);
			InSequencer->RequestEvaluate();

			if (AutoChangeMode == EAutoChangeMode::AutoKey || AutoChangeMode == EAutoChangeMode::All)
			{
				InSequencer->SetAutoChangeMode(AutoChangeMode);
			}
			return true;
		}
	}
	return false;
}

void FComponentConstraintChannelInterface::AddHandleTransformKeys(
	const TSharedPtr<ISequencer>& InSequencer,
	const UTransformableHandle* InHandle,
	const TArray<FFrameNumber>& InFrames,
	const TArray<FTransform>& InTransforms,
	const EMovieSceneTransformChannel& InChannels)
{
	ensure(InTransforms.Num());
	
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return;
	}

	const UTransformableComponentHandle* Handle = static_cast<const UTransformableComponentHandle*>(InHandle);
	AActor* Actor = Handle->Component->GetOwner();
	if (!Actor)
	{
		return;
	}

	const FGuid Guid = InSequencer->GetHandleToObject(Actor, true);
	if (!Guid.IsValid())
	{
		return;
	}

	UMovieScene3DTransformSection* TransformSection = GetComponentSection(Handle, InSequencer, false);
	if (!TransformSection)
	{
		return;
	}

	const UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	
	const UMovieScene3DTransformSection* ConstraintSection = GetComponentSection(Handle, InSequencer, true);
	if (ConstraintSection && TransformSection != ConstraintSection)
	{
		TArray<FTransform> Transforms(InTransforms);
		RecomposeTransforms(InSequencer, Handle->Component.Get(), TransformSection, InFrames, Transforms);
		Handle->AddTransformKeys(InFrames, Transforms, InChannels, MovieScene->GetTickResolution(), TransformSection, true);
	}
	else
	{
		Handle->AddTransformKeys(InFrames, InTransforms, InChannels, MovieScene->GetTickResolution(), TransformSection, true);	
	}
}

UMovieScene3DTransformSection* FComponentConstraintChannelInterface::GetComponentSection(
	const UTransformableComponentHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer, const bool bIsConstraint)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	AActor* Actor = InHandle->Component->GetOwner();
	if (!Actor)
	{
		return nullptr;
	}

	const FGuid Guid = InSequencer->GetHandleToObject(Actor, true);
	if (!Guid.IsValid())
	{
		return nullptr;
	}


	const FTransform DefaultTransform = InHandle->GetLocalTransform();
	if (!bIsConstraint)
	{
		return MovieSceneToolHelpers::GetTransformSection(InSequencer.Get(), Guid, DefaultTransform);
	}

	const UMovieSceneSequence* MovieSceneSequence = InSequencer ? InSequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return nullptr;
	}

	UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(Guid);
	if (!TransformTrack)
	{
		MovieScene->Modify();
		TransformTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(Guid);
	}
	TransformTrack->Modify();

	static constexpr FFrameNumber Frame0;
	bool bSectionAdded = false;

	const TArray<UMovieSceneSection*>& AllSections = TransformTrack->GetAllSections();
	UMovieScene3DTransformSection* TransformSection = AllSections.IsEmpty() ?
		Cast<UMovieScene3DTransformSection>(TransformTrack->FindOrAddSection(Frame0, bSectionAdded)) :
		Cast<UMovieScene3DTransformSection>(AllSections[0]);
	if (!TransformSection)
	{
		return nullptr;
	}

	TransformSection->Modify();
	if (bSectionAdded)
	{
		TransformSection->SetRange(TRange<FFrameNumber>::All());

		const FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
		const TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
		};

		const FVector Location0 = DefaultTransform.GetLocation();
		const FRotator Rotation0 = DefaultTransform.GetRotation().Rotator();
		const FVector Scale3D0 = DefaultTransform.GetScale3D();
		const FTransform::FReal DefaultValues[] = { Location0.X, Location0.Y, Location0.Z,
													Rotation0.Roll, Rotation0.Pitch, Rotation0.Yaw,
													Scale3D0.X, Scale3D0.Y, Scale3D0.Z };
		for (int32 Index = 0; Index < 9; Index++)
		{
			if (FMovieSceneDoubleChannel* Channel = DoubleChannels[Index].Get())
			{
				Channel->SetDefault(DefaultValues[Index]);
			}
		}
	}

	return TransformSection;
}

void FComponentConstraintChannelInterface::RecomposeTransforms(
	const TSharedPtr<ISequencer>& InSequencer, USceneComponent* SceneComponent, UMovieSceneSection* Section,
	const TArray<FFrameNumber>& InFrames, TArray<FTransform>& InOutTransforms) const
{
	using namespace UE::MovieScene;

	FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = InSequencer->GetEvaluationTemplate();
	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if (!EntityLinker)
	{
		return;
	}

	UMovieScenePropertyInstantiatorSystem* System = EntityLinker->FindSystem<UMovieScenePropertyInstantiatorSystem>();
	if (!System)
	{
		return;
	}

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &EntityLinker->EntityManager);
	TArray<FMovieSceneEntityID> ImportedEntityIDs;
	EvaluationTemplate.FindEntitiesFromOwner(Section, InSequencer->GetFocusedTemplateID(), ImportedEntityIDs);
	if (ImportedEntityIDs.Num())
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

		UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const EMovieScenePlayerStatus::Type PlaybackStatus = InSequencer->GetPlaybackStatus();

		TArray<FMovieSceneEntityID> EntityIDs;
		FDecompositionQuery Query;
		Query.Object   = SceneComponent;
		Query.bConvertFromSourceEntityIDs = false;  // We already pass the children entity IDs
		FMovieSceneSequenceTransform RootToLocalTransform = InSequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

		// add keys
		for (int32 Index = 0; Index < InFrames.Num(); ++Index)
		{
			TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(InFrames[Index]);
			if (!RootTime)
			{
				continue;
			}

			const FMovieSceneEvaluationRange EvaluationRange = FMovieSceneEvaluationRange(RootTime.GetValue(), TickResolution);
			const FMovieSceneContext Context = FMovieSceneContext(EvaluationRange, PlaybackStatus).SetHasJumped(true);

			EvaluationTemplate.EvaluateSynchronousBlocking(Context);

			if (EntityIDs.IsEmpty())
			{
				// In order to check for the result channels later, we need to look up the children entities that are
				// bound to the given animated object. Imported entities generally don't have the result channels.
				FEntityTaskBuilder()
				.ReadEntityIDs()
				.Read(BuiltInComponents->ParentEntity)
				.Read(BuiltInComponents->BoundObject)
				.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
				.Iterate_PerEntity(
					&EntityLinker->EntityManager, 
					[SceneComponent, ImportedEntityIDs, &EntityIDs](FMovieSceneEntityID EntityID, FMovieSceneEntityID ParentEntityID, UObject* BoundObject)
					{
						if (SceneComponent == BoundObject && ImportedEntityIDs.Contains(ParentEntityID))
						{
							EntityIDs.Add(EntityID);
						}
					});
				Query.Entities = MakeArrayView(EntityIDs);
			}
			
			FTransform& Transform = InOutTransforms[Index];

			const FIntermediate3DTransform CurrentValue(Transform.GetTranslation(), Transform.GetRotation().Rotator(), Transform.GetScale3D());

			TRecompositionResult<FIntermediate3DTransform> TransformData = System->RecomposeBlendOperational(TrackComponents->ComponentTransform, Query, CurrentValue);

			double CurrentTransformChannels[9] = { 0, 0, 0, 0, 0, 0, 0, 0, 0 };
			EMovieSceneTransformChannel ChannelsObtained(EMovieSceneTransformChannel::None);
			check(EntityIDs.Num() == TransformData.Values.Num());
			for (int32 EntityIndex = 0; EntityIndex < TransformData.Values.Num(); ++EntityIndex)
			{
				const FMovieSceneEntityID& EntityID = EntityIDs[EntityIndex];
				
				const FIntermediate3DTransform& EntityTransformData = TransformData.Values[EntityIndex];
				FComponentMask EntityType = EntityLinker->EntityManager.GetEntityType(EntityID);
				for (int32 CompositeIndex = 0; CompositeIndex < 9; ++CompositeIndex)
				{
					EMovieSceneTransformChannel ChannelMask = (EMovieSceneTransformChannel)(1 << CompositeIndex);
					if (!EnumHasAnyFlags(ChannelsObtained, ChannelMask) && EntityType.Contains(BuiltInComponents->DoubleResult[CompositeIndex]))
					{
						EnumAddFlags(ChannelsObtained, (EMovieSceneTransformChannel)(1 << CompositeIndex));
						CurrentTransformChannels[CompositeIndex] = EntityTransformData[CompositeIndex];
					}
				}
			}

			Transform = FTransform(
				FRotator(CurrentTransformChannels[4], CurrentTransformChannels[5], CurrentTransformChannels[3]), // pitch yaw roll
				FVector(CurrentTransformChannels[0], CurrentTransformChannels[1], CurrentTransformChannels[2]),
				FVector(CurrentTransformChannels[6], CurrentTransformChannels[7], CurrentTransformChannels[8]));
		}
	}
}

void FComponentConstraintChannelInterface::UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld)
{
	ITransformConstraintChannelInterface::UnregisterTrack(InTrack, InWorld);
	
	UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(InTrack);
	if (!TransformTrack)
	{
		return;
	}

	const TArray<UMovieSceneSection*>& AllSections = TransformTrack->GetAllSections();
	UMovieScene3DTransformSection* Section =
		AllSections.IsEmpty() ? nullptr : Cast<UMovieScene3DTransformSection>(AllSections[0]);
	if (!Section)
	{
		return;
	}
	
	UnregisterConstraints(Section, InWorld);
}