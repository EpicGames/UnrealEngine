// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Constraints/TransformConstraintChannelInterface.h"

/**
* Interface that defines animatable capabilities for UTransformableComponentHandle
*/

class UTransformableComponentHandle;
class UMovieScene3DTransformSection;
class USceneComponent;

struct FComponentConstraintChannelInterface : ITransformConstraintChannelInterface
{
	virtual ~FComponentConstraintChannelInterface() = default;

	/** Get the UMovieScene3DTransformSection from the component. */
	virtual UMovieSceneSection* GetHandleSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) override;
	virtual UMovieSceneSection* GetHandleConstraintSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) override;

	/** Get the world from the component. */
	virtual UWorld* GetHandleWorld(UTransformableHandle* InHandle) override;
	
	/** Add an active/inactive key to the constraint channel if needed and does the transform compensation on the component's transform channels. */
	virtual bool SmartConstraintKey(
		UTickableTransformConstraint* InConstraint, const TOptional<bool>& InOptActive,
		const TRange<FFrameNumber>& InTimeRange, const TSharedPtr<ISequencer>& InSequencer) override;

	/** Add keys on the component's transform channels. */
	virtual void AddHandleTransformKeys(
		const TSharedPtr<ISequencer>& InSequencer,
		const UTransformableHandle* InHandle,
		const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InLocalTransforms,
		const EMovieSceneTransformChannel& InChannels) override;

protected:
	virtual void UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld) override;
	
private:
		static UMovieScene3DTransformSection* GetComponentSection(const UTransformableComponentHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer, const bool bInConstraint);
	
		void RecomposeTransforms(
			const TSharedPtr<ISequencer>& InSequencer,
			USceneComponent* SceneComponent, UMovieSceneSection* Section,
			const TArray<FFrameNumber>& InFrames, TArray<FTransform>& InOutTransforms) const;
};
