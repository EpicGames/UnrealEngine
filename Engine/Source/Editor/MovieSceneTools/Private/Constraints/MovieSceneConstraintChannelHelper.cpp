// Copyright Epic Games, Inc. All Rights Reserved.

#include "Constraints/MovieSceneConstraintChannelHelper.inl"

#include "ConstraintsEvaluation.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"

#include "LevelSequence.h"
#include "MovieSceneToolHelpers.h"

#include "Transform/TransformableHandle.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"
#include "Algo/Copy.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "LevelEditorViewport.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "Sections/MovieSceneConstrainedSection.h"
#include "MovieSceneToolsModule.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Transform/TransformableHandleUtils.h"
#include "Widgets/Notifications/SNotificationList.h"

/*
*
*  FCompensationEvaluator
*
*/

FCompensationEvaluator::FCompensationEvaluator(UTickableTransformConstraint* InConstraint)
	: Constraint(InConstraint)
	, Handle(InConstraint ? InConstraint->ChildTRSHandle : nullptr)
{}

void FCompensationEvaluator::ComputeLocalTransforms(UWorld* InWorld, const FEvalParameters& InEvalParams)
{
	using namespace UE::TransformConstraintUtil;
	
	if (!Constraint || !Handle || !InEvalParams.IsValid())
	{
		return;
	}

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray< ConstraintPtr > Constraints = GetHandleTransformConstraints(InWorld);
	if (Constraints.IsEmpty())
	{
		return;
	}
	
	const TArray<ConstraintPtr> ConstraintsMinusThis = Constraints.FilterByPredicate([this](const ConstraintPtr& InConstraint)
	{
		return InConstraint != Constraint;
	});

	// find last active constraint in the list that is different than the on we want to compensate for
	auto GetLastActiveConstraint = [ConstraintsMinusThis]()
	{
		// find last active constraint in the list that is different than the one we want to compensate for
		const int32 LastActiveIndex = GetLastActiveConstraintIndex(ConstraintsMinusThis);

		// if found, return its parent global transform
		return LastActiveIndex > INDEX_NONE ? Cast<UTickableTransformConstraint>(ConstraintsMinusThis[LastActiveIndex]) : nullptr;
	};
	
	bool bShouldPrepareParent = false;
	if (TransformableHandleUtils::SkipTicking())
	{
		USceneComponent* ChildComponent = TransformableHandleUtils::GetTarget<USceneComponent>(Handle);
		USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(Constraint->ParentTRSHandle);
		if (ParentComponent && ParentComponent != ChildComponent)
		{
			bShouldPrepareParent = [](const USceneComponent* Component)
			{
				while (Component)
				{
					if (Component->IsA<USkeletalMeshComponent>())
					{
						return true;
					}
					Component = Component->GetAttachParent();
				}
				return false;
			}(ParentComponent);
		}
	}
	
	ISequencer* Sequencer = InEvalParams.Sequencer;
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const EMovieScenePlayerStatus::Type PlaybackStatus = Sequencer->GetPlaybackStatus();

	const TConstArrayView<FFrameNumber> Frames(InEvalParams.Frames);
	const int32 NumFrames = Frames.Num();

	// store current state
	const FTransform CurrentLocal = Handle->GetLocalTransform();
	const FTransform CurrentGlobal = Handle->GetGlobalTransform();
	const FTransform CurrentSpaceGlobal = Constraint->GetParentGlobalTransform();
	const FTransform CurrentRelative = ComputeRelativeTransform(CurrentLocal, CurrentGlobal, CurrentSpaceGlobal, Constraint);

	// resize arrays to num frames + 1 as we also evaluate at InFrames[0]-1
	ChildLocals.SetNum(NumFrames + 1);
	ChildGlobals.SetNum(NumFrames + 1);
	SpaceGlobals.SetNum(NumFrames + 1);

	// avoid transacting when evaluating sequencer
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);

	const FMovieSceneToolsAnimationBakingWrapper BakingWrapper(MovieScene);
	BakingWrapper.Start();
	
	// get all constraints to evaluate
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> AllConstraints = Controller.GetAllConstraints(bSorted);

	FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

	const bool bCreatedBeforeCurrentFrame = InEvalParams.OptCurrentFrame.IsSet();
	const bool bPropagateRelative = InEvalParams.bPropagateRelative;
	
	for (int32 Index = 0; Index < NumFrames + 1; ++Index)
	{
		FFrameNumber FrameNumber = (Index == 0) ? Frames[0] - 1 : Frames[Index - 1];
		TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(FrameNumber);
		if (!RootTime)
		{
			continue;
		}

		FrameNumber = RootTime->GetFrame();

		// evaluate animation
		{
			const FMovieSceneEvaluationRange EvaluationRange = FMovieSceneEvaluationRange(FFrameTime(FrameNumber), TickResolution);
			const FMovieSceneContext Context = FMovieSceneContext(EvaluationRange, PlaybackStatus).SetHasJumped(true);

			BakingWrapper.PreEvaluate(FrameNumber);
		
			Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);

			if (bShouldPrepareParent)
			{
				constexpr bool bRecursive = true;
				USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(Constraint->ParentTRSHandle);
				TransformableHandleUtils::MarkComponentForEvaluation(ParentComponent, bRecursive);
			}
			
			// evaluate constraints
			UE::Anim::Constraints::EvaluateConstraints(AllConstraints);
		
			BakingWrapper.PostEvaluate(FrameNumber);
		}

		FTransform& ChildLocal = ChildLocals[Index];
		FTransform& ChildGlobal = ChildGlobals[Index];
		FTransform& SpaceGlobal = SpaceGlobals[Index];

		if (InEvalParams.bKeepCurrent && Index < 2)
		{
			ChildGlobal = CurrentGlobal;
		}
		else
		{
			// store child transforms        	
			ChildLocal = Handle->GetLocalTransform();
			Handle->PreEvaluate();
			ChildGlobal = Handle->GetGlobalTransform();
		}

		const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint();

		// store constraint/parent space global transform
		if (InEvalParams.bToActive)
		{
			// if activating the constraint, store last constraint or parent space at T[0]-1
			// and constraint space for all other times
			if (Index == 0)
			{
				if (bCreatedBeforeCurrentFrame)
				{
					const FFrameNumber CurrentFrame = *InEvalParams.OptCurrentFrame;
					if ((bPropagateRelative && FrameNumber < CurrentFrame) || FrameNumber == CurrentFrame)
					{
						SpaceGlobal = Constraint->GetParentGlobalTransform();
						Handle->SetGlobalTransform(CurrentRelative * SpaceGlobal);
						Handle->PreEvaluate();
						ChildLocal = Handle->GetLocalTransform();
						ChildGlobal = Handle->GetGlobalTransform();
						
						if (LastConstraint)
						{
							SpaceGlobal = LastConstraint->GetParentGlobalTransform();
							if (TOptional<FTransform> Relative = GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal))
							{
								ChildLocal = *Relative;
							}
						}
					}
				}
				else if (LastConstraint)
				{
					SpaceGlobal = LastConstraint->GetParentGlobalTransform();
					if (TOptional<FTransform> Relative = GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal))
					{
						ChildLocal = *Relative;
					}
				}
				else if (InEvalParams.bKeepCurrent)
				{
					ChildLocal = CurrentLocal;
				}
			}
			else
			{
				if (InEvalParams.bInfiniteAtLeft)
				{
					if (Index == 1)
					{
						SpaceGlobal = Constraint->GetParentGlobalTransform();
						ChildLocal = ComputeRelativeTransform(ChildLocal, ChildGlobal, SpaceGlobal, Constraint);
					}
					else
					{
						if (LastConstraint)
						{
							SpaceGlobal = LastConstraint->GetParentGlobalTransform();
							if (TOptional<FTransform> Relative = GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal))
							{
								ChildLocal = *Relative;
							}
						}
					}
				}
				else if (bCreatedBeforeCurrentFrame)
				{
					const FFrameNumber CurrentFrame = *InEvalParams.OptCurrentFrame;
					if ((bPropagateRelative && FrameNumber < CurrentFrame) || FrameNumber == CurrentFrame)
					{
					 	ChildLocal = CurrentRelative;
					}
					else
					{
						SpaceGlobal = Constraint->GetParentGlobalTransform();
						ChildLocal = ComputeRelativeTransform(ChildLocal, ChildGlobal, SpaceGlobal, Constraint);						
					}
				}
				else
				{
					SpaceGlobal = Constraint->GetParentGlobalTransform();
					ChildLocal = ComputeRelativeTransform(ChildLocal, ChildGlobal, SpaceGlobal, Constraint);
				}
			}
		}
		else
		{
			// if deactivating the constraint, store constraint space at T[0]-1
			// and last constraint or parent space for all other times
			if (Index == 0)
			{
				SpaceGlobal = Constraint->GetParentGlobalTransform();
				ChildLocal = ComputeRelativeTransform(ChildLocal, ChildGlobal, SpaceGlobal, Constraint);
			}
			else
			{
				if (LastConstraint)
				{
					SpaceGlobal = LastConstraint->GetParentGlobalTransform();
					if (TOptional<FTransform> Relative = GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal))
					{
						ChildLocal = *Relative;
					}
				}
			}
		}
	}
	
	BakingWrapper.Stop();
	
	//get back to where we are at, should also make sure things are active
	Sequencer->ForceEvaluate();
}

void FCompensationEvaluator::ComputeLocalTransformsForBaking(UWorld* InWorld, const FEvalParameters& InEvalParams)
{
	if (!InEvalParams.IsValid())
	{
		return;
	}

	if (!IsValid(Handle) || !Handle->IsValid())
	{
		return;
	}
	
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray< ConstraintPtr > Constraints = GetHandleTransformConstraints(InWorld);
	
	const TArray< ConstraintPtr > ConstraintsMinusThis =
		Constraints.FilterByPredicate([this](const ConstraintPtr& InConstraint)
		{
			return InConstraint != Constraint;
		});

	auto GetLastActiveConstraint = [ConstraintsMinusThis]()
	{
		// find last active constraint in the list that is different than the one we want to compensate for
		const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(ConstraintsMinusThis);

		// if found, return its parent global transform
		return LastActiveIndex > INDEX_NONE ? Cast<UTickableTransformConstraint>(ConstraintsMinusThis[LastActiveIndex]) : nullptr;
	};

	ISequencer* Sequencer = InEvalParams.Sequencer;
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const EMovieScenePlayerStatus::Type PlaybackStatus = Sequencer->GetPlaybackStatus();

	const TConstArrayView<FFrameNumber> Frames(InEvalParams.Frames);
	const int32 NumFrames = Frames.Num();

	ChildLocals.SetNum(NumFrames);
	ChildGlobals.SetNum(NumFrames);
	SpaceGlobals.SetNum(NumFrames);

	// get all constraints for evaluation
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> AllConstraints = Controller.GetAllConstraints(bSorted);

	// avoid transacting when evaluating sequencer
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);
	
	const FMovieSceneToolsAnimationBakingWrapper BakingWrapper(MovieScene);
	BakingWrapper.Start();
	
	FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

	{
		const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
		const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
	
		for (int32 Index = 0; Index < NumFrames; ++Index)
		{
			TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(Frames[Index]);
			if (!RootTime)
			{
				continue;
			}

			FFrameNumber FrameNumber = RootTime->GetFrame();

			// evaluate animation
			const FMovieSceneEvaluationRange EvaluationRange = FMovieSceneEvaluationRange(FFrameTime(FrameNumber), TickResolution);
			const FMovieSceneContext Context = FMovieSceneContext(EvaluationRange, PlaybackStatus).SetHasJumped(true);

			BakingWrapper.PreEvaluate(FrameNumber);

			Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);

			// evaluate constraints
			{
				const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
				UE::Anim::Constraints::EvaluateConstraints(AllConstraints);
			}

			BakingWrapper.PostEvaluate(FrameNumber);

			FTransform& ChildLocal = ChildLocals[Index];
			FTransform& ChildGlobal = ChildGlobals[Index];
			FTransform& SpaceGlobal = SpaceGlobals[Index];

			// store child transforms
			ChildLocal = Handle->GetLocalTransform();
			ChildGlobal = Handle->GetGlobalTransform();

			// store constraint/parent space global transform
			if (const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint())
			{
				SpaceGlobal = LastConstraint->GetParentGlobalTransform();
				TOptional<FTransform> Relative =
					UE::TransformConstraintUtil::GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal);
				if (Relative)
				{
					ChildLocal = *Relative;
				}
			}
		}

		BakingWrapper.Stop();
	}
	
	const bool bIsValidAfterBaking = IsValid(Handle) && Handle->IsValid();
	if (!bIsValidAfterBaking)
	{
		// the handle might not be valid after baking due to spawnables or baking out of the sequence boundaries
		// so force sequencer evaluation to make sure we're back to normal
		Sequencer->ForceEvaluate();
	}
}
void FCompensationEvaluator::ComputeLocalTransformsBeforeDeletion(UWorld* InWorld, const FEvalParameters& InEvalParams)
{
	if (!InEvalParams.Sequencer)
	{
		return;
	}

	const TConstArrayView<FFrameNumber> Frames(InEvalParams.Frames);
	const bool bFromCurrentFrame = Frames.IsEmpty() && InEvalParams.OptCurrentFrame.IsSet();
	if (Frames.IsEmpty() && !bFromCurrentFrame)
	{
		return;
	}

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray<ConstraintPtr> Constraints = GetHandleTransformConstraints(InWorld);
	const TArray<ConstraintPtr> ConstraintsMinusThis = Constraints.FilterByPredicate(
		[this](const ConstraintPtr& InConstraint)
		{
			return InConstraint.Get() != Constraint;
		});

	// find last active constraint in the list that is different than the on we want to compensate for
	auto GetLastActiveConstraint = [this, ConstraintsMinusThis]()
	{
		// find last active constraint in the list that is different than the on we want to compensate for
		const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(ConstraintsMinusThis);

		// if found, return its parent global transform
		return LastActiveIndex > INDEX_NONE ? Cast<UTickableTransformConstraint>(ConstraintsMinusThis[LastActiveIndex]) : nullptr;
	};

	if (bFromCurrentFrame)
	{
		// store child transforms        	
		FTransform& ChildLocal = ChildLocals.Emplace_GetRef(Handle->GetLocalTransform());
		FTransform& ChildGlobal = ChildGlobals.Emplace_GetRef(Handle->GetGlobalTransform());
		FTransform& SpaceGlobal = SpaceGlobals.Emplace_GetRef(FTransform::Identity);

		// store constraint/parent space global transform
		if (const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint())
		{
			SpaceGlobal = LastConstraint->GetParentGlobalTransform();
			TOptional<FTransform> Relative =
				UE::TransformConstraintUtil::GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal);
			if (Relative)
			{
				ChildLocal = *Relative;
			}
		}
		return;
	}

	// get all constraints for evaluation
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> AllConstraints = Controller.GetAllConstraints(bSorted);
	
	ISequencer* Sequencer = InEvalParams.Sequencer;
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const EMovieScenePlayerStatus::Type PlaybackStatus = Sequencer->GetPlaybackStatus();

	const int32 NumFrames = Frames.Num();

	ChildLocals.SetNum(NumFrames);
	ChildGlobals.SetNum(NumFrames);
	SpaceGlobals.SetNum(NumFrames);

	// avoid transacting when evaluating sequencer
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);
	
	const FMovieSceneToolsAnimationBakingWrapper BakingWrapper(MovieScene);
	BakingWrapper.Start();
	
	FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

	for (int32 Index = 0; Index < NumFrames; ++Index)
	{
		TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(Frames[Index]);
		if (!RootTime)
		{
			continue;
		}

		FFrameNumber FrameNumber = RootTime->GetFrame();

		// evaluate animation
		const FMovieSceneEvaluationRange EvaluationRange = FMovieSceneEvaluationRange(FFrameTime(FrameNumber), TickResolution);
		const FMovieSceneContext Context = FMovieSceneContext(EvaluationRange, PlaybackStatus).SetHasJumped(true);

		BakingWrapper.PreEvaluate(FrameNumber);
		
		Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);

		// evaluate constraints
		UE::Anim::Constraints::EvaluateConstraints(AllConstraints);
		
		BakingWrapper.PostEvaluate(FrameNumber);
		
		FTransform& ChildLocal = ChildLocals[Index];
		FTransform& ChildGlobal = ChildGlobals[Index];
		FTransform& SpaceGlobal = SpaceGlobals[Index];

		// store child transforms        	
		ChildLocal = Handle->GetLocalTransform();
		ChildGlobal = Handle->GetGlobalTransform();

		// store constraint/parent space global transform
		if (const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint())
		{
			SpaceGlobal = LastConstraint->GetParentGlobalTransform();
			TOptional<FTransform> Relative =
				UE::TransformConstraintUtil::GetConstraintsRelativeTransform(ConstraintsMinusThis, ChildLocal, ChildGlobal);
			if(Relative)
			{
				ChildLocal = *Relative;
			}
		}
	}

	BakingWrapper.Stop();
}

void FCompensationEvaluator::ComputeCompensation(UWorld* InWorld, const TSharedPtr<ISequencer>& InSequencer, const FFrameNumber& InTime)
{
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray<ConstraintPtr> Constraints = GetHandleTransformConstraints(InWorld);
	if (Constraints.IsEmpty())
	{
		return;
	}

	// find last active constraint in the list that is different than the on we want to compensate for
	auto GetLastActiveConstraint = [this, Constraints]()
	{
		// find last active constraint in the list that is different than the on we want to compensate for
		const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(Constraints);

		// if found, return its parent global transform
		return LastActiveIndex > INDEX_NONE ? Cast<UTickableTransformConstraint>(Constraints[LastActiveIndex]) : nullptr;
	};

	// get all constraints for evaluation
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> AllConstraints = Controller.GetAllConstraints(bSorted);

	// avoid transacting when evaluating sequencer
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);
	
	UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	
	const FMovieSceneToolsAnimationBakingWrapper BakingWrapper(MovieScene);
	BakingWrapper.Start();

	FMovieSceneInverseSequenceTransform LocalToRootTransform = InSequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

	auto EvaluateAt = [Handle = this->Handle, InSequencer, &AllConstraints, &BakingWrapper, &LocalToRootTransform](FFrameNumber InFrame)
	{
		TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(InFrame);
		if (!RootTime)
		{
			return;
		}

		InFrame = RootTime->GetFrame();

		UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const EMovieScenePlayerStatus::Type PlaybackStatus = InSequencer->GetPlaybackStatus();

		const FMovieSceneEvaluationRange EvaluationRange0 = FMovieSceneEvaluationRange(FFrameTime(InFrame), TickResolution);
		const FMovieSceneContext Context0 = FMovieSceneContext(EvaluationRange0, PlaybackStatus).SetHasJumped(true);

		BakingWrapper.PreEvaluate(InFrame);
			
		InSequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context0);

		UE::Anim::Constraints::EvaluateConstraints(AllConstraints);

		BakingWrapper.PostEvaluate(InFrame);

		if (Handle)
		{
			Handle->PreEvaluate();
		}
	};


	// allocate
	ChildLocals.SetNum(1);
	ChildGlobals.SetNum(1);
	SpaceGlobals.SetNum(1);

	
	// evaluate at InTime and store global
	EvaluateAt(InTime);
	ChildGlobals[0] = Handle->GetGlobalTransform();

	// evaluate at InTime-1 and store local
	EvaluateAt(InTime - 1);
	ChildLocals[0] = Handle->GetLocalTransform();

	BakingWrapper.Stop();
	
	// if constraint at T-1 then switch to its space
	if (const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint())
	{
		SpaceGlobals[0] = LastConstraint->GetParentGlobalTransform();
		TOptional<FTransform> Relative =
			UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, ChildLocals[0], ChildGlobals[0]);
		if(Relative)
		{
			ChildLocals[0] = *Relative;
		}
	}
	else // switch to parent space
	{
		const FTransform ChildLocal = ChildLocals[0];
		Handle->SetGlobalTransform(ChildGlobals[0]);
		Handle->PreEvaluate();
		ChildLocals[0] = Handle->GetLocalTransform();
		Handle->SetLocalTransform(ChildLocal);
		Handle->PreEvaluate();
	}
}

void FCompensationEvaluator::CacheTransforms(UWorld* InWorld, const FEvalParameters& InEvalParams)
{
	if (!InEvalParams.IsValid())
	{
		return;
	}

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;

	// get all constraints for evaluation
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> AllConstraints = Controller.GetAllConstraints(bSorted);

	ISequencer* Sequencer = InEvalParams.Sequencer;
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	const FFrameRate TickResolution = Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->GetTickResolution();
	const EMovieScenePlayerStatus::Type PlaybackStatus = Sequencer->GetPlaybackStatus();

	const TConstArrayView<FFrameNumber> Frames(InEvalParams.Frames);
	const int32 NumFrames = Frames.Num();

	ChildLocals.SetNum(NumFrames);
	ChildGlobals.SetNum(NumFrames);
	SpaceGlobals.SetNum(NumFrames);

	const FMovieSceneToolsAnimationBakingWrapper BakingWrapper(MovieScene);

	FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

	auto EvaluateAt = [&](FFrameNumber InFrame)
	{
		TOptional<FFrameTime> RootTime = LocalToRootTransform.TryTransformTime(InFrame);
		if (!RootTime)
		{
			return;
		}

		const FMovieSceneEvaluationRange EvaluationRange = FMovieSceneEvaluationRange(RootTime.GetValue(), TickResolution);
		const FMovieSceneContext Context = FMovieSceneContext(EvaluationRange, PlaybackStatus).SetHasJumped(true);

		BakingWrapper.PreEvaluate(InFrame);
		
		Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);

		// evaluate constraints
		UE::Anim::Constraints::EvaluateConstraints(AllConstraints);
		
		BakingWrapper.PostEvaluate(InFrame);
	};

	// avoid transacting when evaluating sequencer
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);
	
	BakingWrapper.Start();
	
	for (int32 Index = 0; Index < NumFrames; ++Index)
	{
		// evaluate animation
		EvaluateAt(Frames[Index]);

		// store transforms        	
		ChildLocals[Index] = Handle->GetLocalTransform();
		ChildGlobals[Index] = Handle->GetGlobalTransform();
		SpaceGlobals[Index] = Constraint->GetParentGlobalTransform();
	}
	
	BakingWrapper.Stop();
}

void FCompensationEvaluator::ComputeCurrentTransforms(UWorld* InWorld)
{
	ChildLocals = ChildGlobals = SpaceGlobals = {FTransform::Identity};

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray< ConstraintPtr > Constraints = GetHandleTransformConstraints(InWorld);
	if (Constraints.IsEmpty())
	{
		return;
	}

	for (const TWeakObjectPtr<UTickableConstraint>& InConstraint : Constraints)
	{
		if (InConstraint.IsValid())
		{
			InConstraint->Evaluate();
		}
	}

	ChildLocals[0] = Handle->GetLocalTransform();
	ChildGlobals[0] = Handle->GetGlobalTransform();
	
	auto GetLastActiveConstraint = [Constraints]()
	{
		// find last active constraint in the list that is different than the one we want to compensate for
		const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(Constraints);

		// if found, return its parent global transform
		return LastActiveIndex > INDEX_NONE ? Cast<UTickableTransformConstraint>(Constraints[LastActiveIndex]) : nullptr;
	};
	
	if (const UTickableTransformConstraint* LastConstraint = GetLastActiveConstraint())
	{
		SpaceGlobals[0] = LastConstraint->GetParentGlobalTransform();
		TOptional<FTransform> Relative =
			UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, ChildLocals[0], ChildGlobals[0]);
		if (Relative)
		{
			ChildLocals[0] = *Relative;
		}
	}
}

const TArray< TWeakObjectPtr<UTickableConstraint> > FCompensationEvaluator::GetHandleTransformConstraints(UWorld* InWorld) const
{
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	
	if (Handle)
	{
		// get sorted transform constraints
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
		static constexpr bool bSorted = true;
		const TArray< ConstraintPtr > Constraints = Controller.GetParentConstraints(Handle->GetHash(), bSorted);
		return Constraints.FilterByPredicate([](const ConstraintPtr& InConstraint)
		{
			return IsValid(InConstraint.Get()) && InConstraint.Get()->IsA<UTickableTransformConstraint>();  
		});
	}

	static const TArray< ConstraintPtr > DummyArray;
	return DummyArray;
}

bool FMovieSceneConstraintChannelHelper::bDoNotCompensate = false;


/*
*
*  Constraint Channel Helpers
*
*/
void FMovieSceneConstraintChannelHelper::HandleConstraintRemoved(
	UTickableConstraint* InConstraint,
	const FMovieSceneConstraintChannel* InConstraintChannel,
	const TSharedPtr<ISequencer>& InSequencer,
	UMovieSceneSection* InSection)
{
	UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraint);
	if (!Constraint || !Constraint->NeedsCompensation() || !InConstraintChannel || !InSection)
	{
		return;
	}
	
	InSection->Modify();
	TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);
	if (const UTransformableHandle* Handle = Constraint->ChildTRSHandle)
	{
		const TArrayView<const FFrameNumber> Times = InConstraintChannel->GetData().GetTimes();
		if (Times.IsEmpty() && !InConstraintChannel->IsInfinite())
		{
			return;
		}

		// get transform channels
		const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = Handle->GetFloatChannels(InSection);
		const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = Handle->GetDoubleChannels(InSection);

		// get frames after this time
		TArray<FFrameNumber> FramesToCompensate;
		if (FloatTransformChannels.Num() > 0)
		{
			GetFramesWithinActiveState(*InConstraintChannel, FloatTransformChannels, FramesToCompensate);
		}
		else
		{
			GetFramesWithinActiveState(*InConstraintChannel, DoubleTransformChannels, FramesToCompensate);
		}
		
		const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();		
		
		// do the compensation
		UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;

		FCompensationEvaluator Evaluator(Constraint);
		FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, FramesToCompensate);

		// update channels' default values if there are no frames to compensate
		if (FramesToCompensate.IsEmpty())
		{
			const FFrameTime CurrentTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);
			EvalParams.OptCurrentFrame = CurrentTime.GetFrame();
		}
		
		Evaluator.ComputeLocalTransformsBeforeDeletion(World, EvalParams);
		const TArray<FTransform>& ChildLocals = Evaluator.ChildLocals;

		const EMovieSceneTransformChannel ChannelsToKey = Constraint->GetChannelsToKey();

		Handle->AddTransformKeys(FramesToCompensate, ChildLocals, ChannelsToKey, TickResolution, InSection);

		// clean double keys
		if (IMovieSceneConstrainedSection* Section = Cast<IMovieSceneConstrainedSection>(InSection))
		{
			// get constraints acting on the same child that is different that InConstraint
			const TArray<FConstraintAndActiveChannel>& ConstraintChannels = Section->GetConstraintsChannels();
			const TArray<FConstraintAndActiveChannel> OtherConstraints = ConstraintChannels.FilterByPredicate([Constraint](const FConstraintAndActiveChannel& Channel)
			{
				if (const UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(Channel.GetConstraint()))
				{
					return TransformConstraint->GetTargetHash() == Constraint->GetTargetHash() && TransformConstraint != Constraint;
				}
				return false;
			});

			// disable extra compensation when removing keys
			if (OtherConstraints.IsEmpty())
			{
				// this was the only constraint so we can remove its double keys
				const EMovieSceneKeyInterpolation KeyType = InSequencer->GetKeyInterpolation();
				for (const FFrameNumber& Time: Times)
				{
					const FFrameNumber TimeMinusOne = Time - 1;
					if (FloatTransformChannels.Num() > 0)
					{
						DeleteTransformKeys(FloatTransformChannels, TimeMinusOne);
						//we also set the tangent at the break to the default type
						ChangeKeyInterpolation(FloatTransformChannels, Time, KeyType);
					}
					else if (DoubleTransformChannels.Num() > 0)
					{
						DeleteTransformKeys(DoubleTransformChannels, TimeMinusOne);
						//we also set the tangent at the break to the default type
						ChangeKeyInterpolation(DoubleTransformChannels, Time, KeyType);
					}
				}
			}
		}
	}
}

void FMovieSceneConstraintChannelHelper::HandleConstraintKeyDeleted(
	UTickableTransformConstraint* InConstraint,
	const FMovieSceneConstraintChannel* InConstraintChannel,
	const TSharedPtr<ISequencer>& InSequencer,
	UMovieSceneSection* InSection,
	const FFrameNumber& InTime)
{
	if (bDoNotCompensate == true || IsInGameThread() == false) //this may happen in an non game thread via a parallel for when we are deleting many channels
	{
		return;
	}
	
	if (!InConstraint || !InConstraint->NeedsCompensation())
	{
		return;
	}

	
	const FFrameNumber TimeMinusOne(InTime - 1);

	bool CurrentValue = false, PreviousValue = false;
	InConstraintChannel->Evaluate(TimeMinusOne, PreviousValue);
	InConstraintChannel->Evaluate(InTime, CurrentValue);

	if (CurrentValue == PreviousValue)
	{
		const int32 NumKeys = InConstraintChannel->GetNumKeys();
		if (NumKeys > 1)
		{
			return;
		}
	}

	TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);

	if (const UTransformableHandle* ControlHandle = InConstraint->ChildTRSHandle)
	{
		// get transform channels
		const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = ControlHandle->GetFloatChannels(InSection);
		const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = ControlHandle->GetDoubleChannels(InSection);

		// get frames after this time
		TArray<FFrameNumber> FramesToCompensate;
		if (FloatTransformChannels.Num() > 0)
		{
			GetFramesAfter(*InConstraintChannel, InTime, FloatTransformChannels, FramesToCompensate);
		}
		else
		{
			GetFramesAfter(*InConstraintChannel,  InTime, DoubleTransformChannels, FramesToCompensate);
		}
		// do the compensation
		UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
		FCompensationEvaluator Evaluator(InConstraint);
        FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, FramesToCompensate);
        EvalParams.bToActive = PreviousValue;
		Evaluator.ComputeLocalTransforms(World, EvalParams);
		TArray<FTransform>& ChildLocals = Evaluator.ChildLocals;
		//turn off constraint, if we delete the key it may not evaluate to false
		InConstraint->SetActive(false);

		if (ChildLocals.Num() < 2)
		{
			return;
		}
		ChildLocals.RemoveAt(0);

		const EMovieSceneTransformChannel ChannelsToKey = InConstraint->GetChannelsToKey();
		const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();

		ControlHandle->AddTransformKeys(FramesToCompensate,
			ChildLocals, ChannelsToKey, TickResolution, InSection);

		// now delete any extra TimeMinusOne
		if (FloatTransformChannels.Num() > 0)
		{
			FMovieSceneConstraintChannelHelper::DeleteTransformKeys(FloatTransformChannels, TimeMinusOne);
		}
		else
		{
			FMovieSceneConstraintChannelHelper::DeleteTransformKeys(DoubleTransformChannels, TimeMinusOne);
		}
	}
}

void FMovieSceneConstraintChannelHelper::HandleConstraintKeyMoved(
	const UTickableTransformConstraint* InConstraint,
	const FMovieSceneConstraintChannel* InConstraintChannel,
	UMovieSceneSection* InSection,
	const FFrameNumber& InCurrentFrame, const FFrameNumber& InNextFrame)
{
	const FFrameNumber Delta = InNextFrame - InCurrentFrame;
	if (Delta == 0)
	{
		return;
	}

	if (!InConstraint || !InConstraintChannel || !InSection)
	{
		return;
	}

	if (const UTransformableHandle* ControlHandle = InConstraint->ChildTRSHandle)
	{

		// get transform channels
		const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = ControlHandle->GetFloatChannels(InSection);
		const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = ControlHandle->GetDoubleChannels(InSection);

		// move them
		if (FloatTransformChannels.Num() > 0)
		{
			FMovieSceneConstraintChannelHelper::MoveTransformKeys(FloatTransformChannels, InCurrentFrame, InNextFrame);
		}
		else
		{
			FMovieSceneConstraintChannelHelper::MoveTransformKeys(DoubleTransformChannels, InCurrentFrame, InNextFrame);
		}
	}

}

namespace UE::Constraints::Private
{

void ShowSpawnableWarning(const FText& InNotification)
{
	// output log notification
	UE_LOGF(LogTemp, Warning, "%ls", *InNotification.ToString());

	// editor notification
	FNotificationInfo Info(InNotification);
	Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
	Info.ExpireDuration = 5.0f;

	FSlateNotificationManager::Get().AddNotification(Info);
}
	
static void ShowSpawnableDiscrepancyWarning(
	const TObjectPtr<UTransformableHandle>& InSpawnableHandle, const TObjectPtr<UTransformableHandle>& InNonSpawnableHandle)
{
	if (!IsValid(InSpawnableHandle) || !IsValid(InNonSpawnableHandle))
	{
		return;
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("Spawnable"), FText::FromString(InSpawnableHandle->GetLabel()));
	Args.Add(TEXT("NonSpawnable"), FText::FromString(InNonSpawnableHandle->GetLabel()));

	const FText Notification = FText::Format(
		NSLOCTEXT("FMovieSceneConstraintChannelHelper", "ChildParentSpawnableDiscrepancy",
		"Object '{Spawnable}' is spawnable whereas '{NonSpawnable}' is not.\n"
		"Consider making '{NonSpawnable}' spawnable to avoid a future undefined state of this constraint."), Args);

	ShowSpawnableWarning(Notification);
}

static void ShowNonSpawnableWarning(
	const TObjectPtr<UTransformableHandle>& InNonSpawnableHandle0, const TObjectPtr<UTransformableHandle>& InNonSpawnableHandle1)
{
	if (!IsValid(InNonSpawnableHandle0) || !IsValid(InNonSpawnableHandle1))
	{
		return;
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("NonSpawnable0"), FText::FromString(InNonSpawnableHandle0->GetLabel()));
	Args.Add(TEXT("NonSpawnable1"), FText::FromString(InNonSpawnableHandle1->GetLabel()));

	const FText Notification = FText::Format(
		NSLOCTEXT("FMovieSceneConstraintChannelHelper", "ChildParentNonSpawnable",
		"Objects '{NonSpawnable0}' and '{NonSpawnable1}' are not spawnable.\n"
		"Consider making them spawnable if you want them, as well as this constraint, to be fully integrated to your level sequence."), Args);

	ShowSpawnableWarning(Notification);
}
	
}

bool FMovieSceneConstraintChannelHelper::AddConstraintToSequencer(
	const TSharedPtr<ISequencer>& InSequencer,
	UTickableTransformConstraint* InConstraint,
	const FSequencerCreationOptions& InOptions)
{
	UMovieSceneSequence* MovieSceneSequence = InSequencer.IsValid() ? InSequencer->GetFocusedMovieSceneSequence() : nullptr;
	if (!MovieSceneSequence)
	{
		return false;
	}
	
	ITransformConstraintChannelInterface* Interface = InConstraint ? GetHandleInterface(InConstraint->ChildTRSHandle) : nullptr;
	if (!Interface)
	{
		return false;
	}

	const bool bIsChildSpawnable = IsHandleSpawnable(InSequencer, InConstraint->ChildTRSHandle);
	const bool bIsParentSpawnable = IsHandleSpawnable(InSequencer, InConstraint->ParentTRSHandle);

	// create bindings before smart keying so added to spawn copies
	CreateBindingIDForHandle(InSequencer, InConstraint->ChildTRSHandle);
	CreateBindingIDForHandle(InSequencer, InConstraint->ParentTRSHandle);

	// adding the child to sequencer can trigger that same function so the constraint might already be added 
	const bool IsOuterASection = !!Cast<IMovieSceneConstrainedSection>(InConstraint->GetOuter());
	if (IsOuterASection)
	{
		return true;
	}

	// notify of spawnable discrepancy 
	if (bIsChildSpawnable != bIsParentSpawnable)
	{
		const TObjectPtr<UTransformableHandle>& SpawnableHandle = bIsChildSpawnable ? InConstraint->ChildTRSHandle : InConstraint->ParentTRSHandle;
		const TObjectPtr<UTransformableHandle>& NonSpawnableHandle = bIsChildSpawnable ?  InConstraint->ParentTRSHandle : InConstraint->ChildTRSHandle;

		UE::Constraints::Private::ShowSpawnableDiscrepancyWarning(SpawnableHandle, NonSpawnableHandle);
	}
	else if (!bIsChildSpawnable && !bIsParentSpawnable)
	{
		const FMovieSceneObjectBindingID& ChildBindingID = InConstraint->ChildTRSHandle->ConstraintBindingID;
		const FMovieSceneObjectBindingID& ParentBindingID = InConstraint->ParentTRSHandle->ConstraintBindingID;
		if (ChildBindingID.IsValid() && ParentBindingID.IsValid())
		{
			UE::Constraints::Private::ShowNonSpawnableWarning(InConstraint->ChildTRSHandle, InConstraint->ParentTRSHandle);
		}
	}

	const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
	const FFrameTime FrameTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);

	TRange<FFrameNumber> Range;
	
	switch (InOptions.CreationTime)
	{
	case ECreationTime::CurrentFrame:
		Range = TRange<FFrameNumber>(FrameTime.GetFrame());
		break;
	case ECreationTime::FromStart:
	{
		const UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		if (ensure(MovieScene))
		{
			Range = TRange<FFrameNumber>(MovieScene->GetPlaybackRange().GetLowerBoundValue());
		}
		break;
	}
	case ECreationTime::Infinite:
	default:
		break;
	}

	return Interface->SmartConstraintKey(InConstraint, TOptional<bool>(), Range, InSequencer);
}

bool FMovieSceneConstraintChannelHelper::SmartConstraintKey(
	const TSharedPtr<ISequencer>& InSequencer,
	UTickableTransformConstraint* InConstraint, 
	const TOptional<bool>& InOptActive, 
	const TOptional<FFrameNumber>& InOptFrameTime)
{
	if (!InSequencer.IsValid() || !InSequencer->GetFocusedMovieSceneSequence())
	{
		return false;
	}
	
	ITransformConstraintChannelInterface* Interface = GetHandleInterface(InConstraint->ChildTRSHandle);
	if (!Interface)
	{
		return false;
	}
	
	FFrameNumber Time;
	if (InOptFrameTime.IsSet())
	{
		Time = InOptFrameTime.GetValue();
	}
	else
	{
		const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
		const FFrameTime FrameTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);
		Time = FrameTime.GetFrame();
	}

	//create bindings before smart keying so added to spawn copies
	CreateBindingIDForHandle(InSequencer, InConstraint->ChildTRSHandle);
	CreateBindingIDForHandle(InSequencer, InConstraint->ParentTRSHandle);

	const bool bSucceeded = Interface->SmartConstraintKey(InConstraint, InOptActive, TRange<FFrameNumber>(Time), InSequencer);

	return bSucceeded;
}

void FMovieSceneConstraintChannelHelper::Compensate(
	const TSharedPtr<ISequencer>& InSequencer,
	const UTickableTransformConstraint* InConstraint,
	const TOptional<FFrameNumber>& InOptTime,
	const bool bCompPreviousTick)
{
	if (!InSequencer.IsValid() || !InSequencer->GetFocusedMovieSceneSequence())
	{
		return;
	}

	const TObjectPtr<UTransformableHandle>& Handle = InConstraint->ChildTRSHandle;
	
	ITransformConstraintChannelInterface* Interface = GetHandleInterface(Handle);
	if (!Interface)
	{
		return;
	}

	IMovieSceneConstrainedSection* Section = Cast<IMovieSceneConstrainedSection>(Interface->GetHandleConstraintSection(Handle, InSequencer));
	const UWorld* World = Interface->GetHandleWorld(Handle);

	if (!Section || !IsValid(World))
	{
		return;
	}

	CompensateIfNeeded(InSequencer, Section, InOptTime, bCompPreviousTick, Handle->GetHash());
}

void FMovieSceneConstraintChannelHelper::CompensateIfNeeded(
	const TSharedPtr<ISequencer>& InSequencer,
	IMovieSceneConstrainedSection* ConstraintSection,
	const TOptional<FFrameNumber>& OptionalTime,
	const bool bCompPreviousTick,
	const int32 InChildHash)
{
	if (bDoNotCompensate)
	{
		return;
	}

	TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);

	// Frames to compensate
	TArray<FFrameNumber> OptionalTimeArray;
	if (OptionalTime.IsSet())
	{
		OptionalTimeArray.Add(OptionalTime.GetValue());
	}

	auto GetConstraintTimesToCompensate = [&OptionalTimeArray](const FConstraintAndActiveChannel& Channel)->TArrayView<const FFrameNumber>
	{
		if (OptionalTimeArray.IsEmpty())
		{
			return Channel.ActiveChannel.GetData().GetTimes();
		}
		return OptionalTimeArray;
	};

	// gather all transform constraints' channels for
	const TArray<FConstraintAndActiveChannel>& ConstraintChannels = ConstraintSection->GetConstraintsChannels();
	TArray<FConstraintAndActiveChannel> TransformConstraintsChannels;
	Algo::CopyIf(ConstraintChannels, TransformConstraintsChannels,
		[InChildHash](const FConstraintAndActiveChannel& InChannel)
		{
			if (!InChannel.GetConstraint().Get())
			{
				return false;
			}

			if ((InChildHash != INDEX_NONE) && (InChannel.GetConstraint()->GetTargetHash() != InChildHash))
			{
				return false;
			}

			const UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InChannel.GetConstraint().Get());
			//if no InChildHash specified(== INDEX_NONE) then do all!
			return Constraint && (InChildHash == INDEX_NONE || Constraint->GetTargetHash() == InChildHash) && Constraint->NeedsCompensation();
		}
	);

	// we only need to treat one single constraint per child as FCompensationEvaluator::ComputeCompensation will
	// compensate within the last active constraint's space
	using CompensationData = TPair< UTickableTransformConstraint*, TArray<FFrameNumber> >;
	TArray< CompensationData > ToCompensate;

	// store constraints and times where compensation is needed 
	for (const FConstraintAndActiveChannel& Channel : TransformConstraintsChannels)
	{
		const TArrayView<const FFrameNumber> FramesToCompensate = GetConstraintTimesToCompensate(Channel);
		for (const FFrameNumber& Time : FramesToCompensate)
		{
			const FFrameNumber TimeMinusOne(Time - 1);

			bool CurrentValue = false, PreviousValue = false;
			Channel.ActiveChannel.Evaluate(TimeMinusOne, PreviousValue);
			Channel.ActiveChannel.Evaluate(Time, CurrentValue);

			if (CurrentValue != PreviousValue) //if they are the same no need to do anything
			{
				UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(Channel.GetConstraint().Get());

				// is the child already in that array?
				int32 DataIndex = ToCompensate.IndexOfByPredicate([Constraint](const CompensationData& InData)
				{
					return InData.Key->GetTargetHash() == Constraint->GetTargetHash();
				});

				// if not, add the constraint
				if (DataIndex == INDEX_NONE)
				{
					DataIndex = ToCompensate.Emplace(Constraint, TArray<FFrameNumber>() );
				}

				// store the time it needs to be compensated at
				TArray<FFrameNumber>& Times = ToCompensate[DataIndex].Value;
				Times.AddUnique(Time);
			}
		}
	}

	// compensate
	bool bNeedsEvaluation = false;
	for (const CompensationData& Data: ToCompensate)
	{
		UTickableTransformConstraint* Constraint = Data.Key;
		const TObjectPtr<UTransformableHandle>& Handle = Constraint->ChildTRSHandle;
		if (ITransformConstraintChannelInterface* Interface = GetHandleInterface(Handle))
		{
			UWorld* World = Interface->GetHandleWorld(Handle);
			
			FCompensationEvaluator Evaluator(Constraint);
			const EMovieSceneTransformChannel ChannelsToKey = Constraint->GetChannelsToKey();
			for (const FFrameNumber& Time : Data.Value)
			{
				const FFrameNumber EvalTime = bCompPreviousTick ? Time : Time - 1;
				const FFrameNumber SetTime = bCompPreviousTick ? Time - 1 : Time;
				// compute transform to set
				// if switching from active to inactive then we must add a key at T-1 in the constraint space
				// if switching from inactive to active then we must add a key at T-1 in the previous constraint or parent space
				Evaluator.ComputeCompensation(World, InSequencer, EvalTime);
				const TArray<FTransform>& LocalTransforms = Evaluator.ChildLocals;

				Interface->AddHandleTransformKeys(InSequencer, Handle, { SetTime }, LocalTransforms, ChannelsToKey);

				bNeedsEvaluation = true;
			}
		}
	}

	if (bNeedsEvaluation)
	{
		InSequencer->ForceEvaluate();
	}
}
FConstraintSections FMovieSceneConstraintChannelHelper::GetConstraintSectionAndChannel(
	const UTickableTransformConstraint* InConstraint,
	const TSharedPtr<ISequencer>& InSequencer)
{
	FConstraintSections ReturnValue;

	if (InSequencer.IsValid() == false)
	{
		return ReturnValue;
	}
	const TObjectPtr<UTransformableHandle>& ChildHandle = InConstraint->ChildTRSHandle;

	const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();
	ReturnValue.Interface = InterfaceRegistry.FindConstraintChannelInterface(ChildHandle->GetClass());
	if (!ReturnValue.Interface)
	{
		return ReturnValue;
	}
	//get the section to be used later to delete the extra transform keys at the frame -1 times, abort if not there for some reason
	ReturnValue.ConstraintSection = ReturnValue.Interface->GetHandleConstraintSection(ChildHandle, InSequencer);
	ReturnValue.ChildTransformSection = ReturnValue.Interface->GetHandleSection(ChildHandle, InSequencer);

	ITransformConstraintChannelInterface* ParentInterface = InterfaceRegistry.FindConstraintChannelInterface(InConstraint->ParentTRSHandle->GetClass());
	if (ParentInterface)
	{
		ReturnValue.ParentTransformSection = ParentInterface->GetHandleSection(InConstraint->ParentTRSHandle, InSequencer);
	}
	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(ReturnValue.ConstraintSection);
	if (ConstrainedSection == nullptr )
	{
		return ReturnValue;
	}

	ReturnValue.ActiveChannel = ConstrainedSection->GetConstraintChannel(InConstraint->ConstraintID);
	return ReturnValue;
}

 void FMovieSceneConstraintChannelHelper::GetTransformFramesForConstraintHandles(
	const UTickableTransformConstraint* InConstraint,
	const TSharedPtr<ISequencer>& InSequencer,
	const FFrameNumber& StartFrame,
	const FFrameNumber& EndFrame,
	TArray<FFrameNumber>& OutFramesToBake)
{
	if ((InConstraint == nullptr) || (InConstraint->ChildTRSHandle == nullptr) || (InConstraint->ParentTRSHandle == nullptr))
	{
		return;
	}

	FConstraintSections ConstraintSections = FMovieSceneConstraintChannelHelper::GetConstraintSectionAndChannel(
		InConstraint, InSequencer);
	if (ConstraintSections.ChildTransformSection)
	{
		const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = InConstraint->ChildTRSHandle->GetFloatChannels(ConstraintSections.ChildTransformSection);
		TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
			FloatTransformChannels, StartFrame, EndFrame);
		//add transforms keys to bake
		{
			for (FFrameNumber& Frame : TransformFrameTimes)
			{
				OutFramesToBake.Add(Frame);
			}
		}
		const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = InConstraint->ChildTRSHandle->GetDoubleChannels(ConstraintSections.ChildTransformSection);
		TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
			DoubleTransformChannels, StartFrame, EndFrame);
		//add transforms keys to bake
		{
			for (FFrameNumber& Frame : TransformFrameTimes)
			{
				OutFramesToBake.Add(Frame);
			}
		}
	}
	if (ConstraintSections.ParentTransformSection)
	{
		const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = InConstraint->ParentTRSHandle->GetFloatChannels(ConstraintSections.ParentTransformSection);
		TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
			FloatTransformChannels, StartFrame, EndFrame);
		//add transforms keys to bake
		{
			for (FFrameNumber& Frame : TransformFrameTimes)
			{
				OutFramesToBake.Add(Frame);
			}
		}
		const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = InConstraint->ParentTRSHandle->GetDoubleChannels(ConstraintSections.ParentTransformSection);
		TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
			DoubleTransformChannels, StartFrame, EndFrame);
		//add transforms keys to bake
		{
			for (FFrameNumber& Frame : TransformFrameTimes)
			{
				OutFramesToBake.Add(Frame);
			}
		}
	}
}

ITransformConstraintChannelInterface* FMovieSceneConstraintChannelHelper::GetHandleInterface(const UTransformableHandle* InHandle)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();	
	return InterfaceRegistry.FindConstraintChannelInterface(InHandle->GetClass());
}

bool FMovieSceneConstraintChannelHelper::IsHandleSpawnable(const TSharedPtr<ISequencer>& InSequencer, const UTransformableHandle* InHandle)
{
	if (!InHandle || !InSequencer.IsValid())
	{
		return false;
	}
	
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InHandle->GetTarget().Get()))
	{
		if (AActor* Actor = SceneComponent->GetTypedOuter<AActor>())
		{
			const TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
			return Spawnable.IsSet();
		}
	}

	return false;
}

void FMovieSceneConstraintChannelHelper::CreateBindingIDForHandle(const TSharedPtr<ISequencer>& InSequencer, UTransformableHandle* InHandle)
{
	if (InHandle == nullptr || InSequencer.IsValid() == false)
	{
		return;
	}

	// make sure object is in sequencer or binding id will be empty we won't resolve the binding
	static constexpr bool bCreateHandleIfMissing = true;
	
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InHandle->GetTarget().Get()))
	{
		if (AActor* Actor = SceneComponent->GetTypedOuter<AActor>())
		{
			TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
			if (Spawnable.IsSet())
			{
				// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
				InHandle->ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(InSequencer->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID,
					*(InSequencer.Get()));
			}
			else
			{
				const FGuid Guid = InSequencer->GetHandleToObject(Actor, bCreateHandleIfMissing);
				InHandle->ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(Guid);
			}

			// in the context of actors with multiple scene components (such as BPs with multiple skeletal meshes, for example)
			// the ID must be the SceneComponent handle instead of the Actor handle.
			// this will also ensure that the binding for the component is created, if this has not yet been done.
			if (InHandle->ConstraintBindingID.IsValid() && SceneComponent != Actor->GetRootComponent())
			{
				const FGuid ComponentHandle = InSequencer->GetHandleToObject(SceneComponent, bCreateHandleIfMissing);
				if (ComponentHandle.IsValid())
				{
					InHandle->ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(ComponentHandle);
				}	
			}
		}
	}
}

void FMovieSceneConstraintChannelHelper::HandleConstraintPropertyChanged(
	UTickableTransformConstraint* InConstraint,
	const FMovieSceneConstraintChannel& InActiveChannel,
	const FPropertyChangedEvent& InPropertyChangedEvent,
	const TSharedPtr<ISequencer>& InSequencer,
	UMovieSceneSection* InSection)
{
	if (!InConstraint || !InSection || !InSequencer.IsValid())
	{
		return;
	}
	
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
	if (PropertyName == UTickableParentConstraint::GetScalingPropertyName())
	{
		return CompensateScale(Cast<UTickableParentConstraint>(InConstraint), InActiveChannel, InSequencer, InSection);
	}
	
	auto IsOffsetProperty = [](const FName InPropertyName)
	{
		return InPropertyName == GET_MEMBER_NAME_CHECKED(UTickableTranslationConstraint, OffsetTranslation) ||
			InPropertyName == GET_MEMBER_NAME_CHECKED(UTickableRotationConstraint, OffsetRotation) ||
			InPropertyName == GET_MEMBER_NAME_CHECKED(UTickableScaleConstraint, OffsetScale) ||
			InPropertyName == GET_MEMBER_NAME_CHECKED(UTickableParentConstraint, OffsetTransform);
	};
	
	if (IsOffsetProperty(PropertyName) || IsOffsetProperty(InPropertyChangedEvent.GetMemberPropertyName()))
	{
		return HandleOffsetChanged(InConstraint, InActiveChannel, InSequencer);
	}
}

void FMovieSceneConstraintChannelHelper::CompensateScale(
	UTickableParentConstraint* InParentConstraint,
	const FMovieSceneConstraintChannel& InActiveChannel,
	const TSharedPtr<ISequencer>& InSequencer,
	UMovieSceneSection* InSection)
{
	if (!InParentConstraint)
	{
		return;
	}

	TObjectPtr<UTransformableHandle> Handle = InParentConstraint->ChildTRSHandle;
	ITransformConstraintChannelInterface* Interface = GetHandleInterface(Handle);
	if (!Interface)
	{
		return;
	}

	const TArrayView<const FFrameNumber> Times = InActiveChannel.GetTimes();
	if (Times.IsEmpty())
	{
	    return;
	}

	// get transform channels
	const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = Handle->GetFloatChannels(InSection);
	const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = Handle->GetDoubleChannels(InSection);

	// get frames after this time
	TArray<FFrameNumber> ActiveTimes;
	if (!FloatTransformChannels.IsEmpty())
	{
	    GetFramesWithinActiveState(InActiveChannel, FloatTransformChannels, ActiveTimes);
	}
	else
	{
	    GetFramesWithinActiveState(InActiveChannel, DoubleTransformChannels, ActiveTimes);
	}

	if (ActiveTimes.IsEmpty())
	{
	    return;
	}

	const bool bRefScalingValue = InParentConstraint->IsScalingEnabled();
	
	// if scaling has been enabled (bRefScalingValue == true), it means that it was not before the property has changed so
	// the current scale channels values represent the local scale values of the handle
	// if scaling has been disabled (bRefScalingValue == false), it means that it was before the property has changed so
	// the current scale channels values represent the offset in the constraint space
	
	InParentConstraint->SetScaling(!bRefScalingValue);

	TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);
	InSection->Modify();
	
	FCompensationEvaluator Evaluator(InParentConstraint);
	const FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, ActiveTimes);
	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	Evaluator.CacheTransforms(World, EvalParams);

	TArray<FTransform>& ChildLocals = Evaluator.ChildLocals;
	const TArray<FTransform>& ChildGlobals = Evaluator.ChildGlobals;
	const TArray<FTransform>& SpaceGlobals = Evaluator.SpaceGlobals;

	const int32 NumTransforms = ChildLocals.Num();

	if (bRefScalingValue)
	{
	    // local scale values have to be switched to the constraint space to represent the offset
	    for (int Index = 0; Index < NumTransforms; Index++)
	    {
    		FTransform& ChildLocal = ChildLocals[Index];
    		const FTransform Offset = ChildGlobals[Index].GetRelativeTransform(SpaceGlobals[Index]);
    		ChildLocal.SetScale3D(Offset.GetScale3D());
	    }
	}
	// else ChildLocals already represents the data that needs to be keyed as it is the result of
	// the constraint evaluation so it just needs to be keyed

	// add keys
	Interface->AddHandleTransformKeys(InSequencer, Handle, ActiveTimes, ChildLocals, EMovieSceneTransformChannel::Scale);

	// reset scaling to reference value
	InParentConstraint->SetScaling(bRefScalingValue);
}

void FMovieSceneConstraintChannelHelper::HandleOffsetChanged(
	UTickableTransformConstraint* InConstraint,
	const FMovieSceneConstraintChannel& InActiveChannel,
	const TSharedPtr<ISequencer>& InSequencer)
{
	if (!InConstraint || !InSequencer.IsValid())
	{
		return;
	}
	
	TObjectPtr<UTransformableHandle> Handle = InConstraint->ChildTRSHandle;
	ITransformConstraintChannelInterface* Interface = GetHandleInterface(Handle);
	if (!Interface)
	{
		return;
	}

	const TArrayView<const FFrameNumber> Times = InActiveChannel.GetTimes();
	if (Times.IsEmpty() && !InActiveChannel.IsInfinite())
	{
		return;
	}

	const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
	const FFrameTime FrameTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);
	const FFrameNumber Time = FrameTime.GetFrame();
		
	bool bIsActive = false; InActiveChannel.Evaluate(Time, bIsActive);
	if (bIsActive)
	{
		const EMovieSceneTransformChannel Channels = InConstraint->GetChannelsToKey();
	
		// compute the current local value
		FCompensationEvaluator Evaluator(InConstraint);
		UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
		Evaluator.ComputeCurrentTransforms(World);

		// update key
		Interface->AddHandleTransformKeys(InSequencer, Handle, {Time}, {Evaluator.ChildLocals[0]}, Channels);
		
		// force evaluation so that new local values are evaluated before the constraint 
		InSequencer->ForceEvaluate();
	}
}
