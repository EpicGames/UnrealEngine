// Copyright Epic Games, Inc. All Rights Reserved.

#include "Constraints/TransformConstraintChannelInterface.h"

#include "ConstraintChannel.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "Sections/MovieSceneConstrainedSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"

ITransformConstraintChannelInterface::ITransformConstraintChannelInterface()
{
	Initialize();
}

ITransformConstraintChannelInterface::~ITransformConstraintChannelInterface()
{
	Shutdown();
}

void ITransformConstraintChannelInterface::Initialize()
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(
		FOnSequencerCreated::FDelegate::CreateRaw(this, &ITransformConstraintChannelInterface::OnSequencerCreated));
}

void ITransformConstraintChannelInterface::Shutdown()
{
	for (const TWeakPtr<ISequencer>& Sequencer : Sequencers)
	{
		if (Sequencer.IsValid())
		{
			Sequencer.Pin()->OnCloseEvent().RemoveAll(this);
		}
	}
	Sequencers.Reset();
	
	if (ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
	{
		if (SequencerCreatedHandle.IsValid())
		{
			SequencerModulePtr->UnregisterOnSequencerCreated(SequencerCreatedHandle);
		}
	}
	SequencerCreatedHandle.Reset();
}

void ITransformConstraintChannelInterface::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	Sequencers.Add(TWeakPtr<ISequencer>(InSequencer));
	InSequencer->OnCloseEvent().AddRaw(this, &ITransformConstraintChannelInterface::OnSequencerClosed);
}

//we changed this so that it will always return true
bool ITransformConstraintChannelInterface::CanAddKey(const FMovieSceneConstraintChannel& InActiveChannel, const FFrameNumber& InTime, bool& ActiveValue)
{
	const TMovieSceneChannelData<const bool> ChannelData = InActiveChannel.GetData();
	const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
	if (Times.IsEmpty())
	{
		ActiveValue = !InActiveChannel.IsInfinite();
		return true;
	}

	if (const TOptional<bool> OptDefault = InActiveChannel.GetDefault())
	{
		const TArrayView<const bool> Values = ChannelData.GetValues();
		if (InTime < Times[0] && !Values[0])
		{
			ActiveValue = OptDefault && *OptDefault;
			return true;
		}
	}
	
	InActiveChannel.Evaluate(InTime, ActiveValue);
	ActiveValue = !ActiveValue;
	
	return true;
}

void ITransformConstraintChannelInterface::CleanDuplicates(FMovieSceneConstraintChannel& InOutActiveChannel, TArray<FFrameNumber>& OutTimesRemoved)
{
	TMovieSceneChannelData<bool> ChannelData = InOutActiveChannel.GetData();
	const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
	const TArrayView<const bool> ActiveValues = ChannelData.GetValues();

	const int32 NumTimes = Times.Num();
	if (NumTimes == 0 || ActiveValues.Num() != NumTimes)
	{
		return;
	}

	if (NumTimes > 1)
	{
		for (int32 Index = NumTimes-1; Index >= 1; --Index)
		{
			// no need to keep consecutive values if they are similar
			if (ActiveValues[Index] == ActiveValues[Index-1])
			{
				OutTimesRemoved.Add(Times[Index]);
				ChannelData.RemoveKey(Index);
			}
		}
	}

	// no need to keep single inactive key
	const TOptional<bool> OptDefault = InOutActiveChannel.GetDefault();
	const bool bIsDefaultTrue = OptDefault && *OptDefault;
	if (!ActiveValues[0] && !bIsDefaultTrue)
	{
		OutTimesRemoved.Add(Times[0]);
		ChannelData.RemoveKey(0);
	}
}

void ITransformConstraintChannelInterface::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	UWorld* World = [InSequencer]() -> UWorld*
	{
		if (const TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = InSequencer->FindSharedPlaybackState())
		{
			if (UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext())
			{
				return PlaybackContext->GetWorld();	
			}
		}
		return nullptr;
	}();

	if (World)
	{
		UMovieSceneSequence* RootSceneSequence = InSequencer->GetRootMovieSceneSequence();
		if (const UMovieScene* RootMovieScene = RootSceneSequence ? RootSceneSequence->GetMovieScene() : nullptr)
		{
			UnregisterMovieScene(RootMovieScene, World);
		}
	}
	
	InSequencer->OnCloseEvent().RemoveAll(this);
	Sequencers.Remove(InSequencer);
}

void ITransformConstraintChannelInterface::UnregisterMovieScene(const UMovieScene* InMovieScene, UWorld* InWorld)
{
	if (!InMovieScene)
	{
		return;
	}
	
	for (UMovieSceneTrack* Track: InMovieScene->GetTracks())
	{
		UnregisterTrack(Track, InWorld);
	}
	
	for (const FMovieSceneBinding& Binding: InMovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track: Binding.GetTracks())
		{
			UnregisterTrack(Track, InWorld);	
		}
	}
}

void ITransformConstraintChannelInterface::UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld)
{
	if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(InTrack))
	{
		for (const UMovieSceneSection* Section: SubTrack->GetAllSections())
		{
			if (const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
			{
				const UMovieSceneSequence* SubSequence = SubSection->GetSequence();
				if (const UMovieScene* SubMovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr)
				{
					UnregisterMovieScene(SubMovieScene, InWorld);
				}
			}
		}
	}
}

void ITransformConstraintChannelInterface::UnregisterConstraints(IMovieSceneConstrainedSection* InSection, UWorld* InWorld)
{
	if (!InSection || !InWorld)
	{
		return;
	}

	const TArray<FConstraintAndActiveChannel> Constraints = InSection->GetConstraintsChannels();
	if (Constraints.IsEmpty())
	{
		return;
	}

	for (const FConstraintAndActiveChannel& ConstraintAndActiveChannel : Constraints)
	{
		if (UTickableConstraint* Constraint = ConstraintAndActiveChannel.GetConstraint())
		{
			FConstraintsManagerController::Get(InWorld).UnregisterConstraint(Constraint);
		}
	}		
}

FConstraintChannelInterfaceRegistry& FConstraintChannelInterfaceRegistry::Get()
{
	static FConstraintChannelInterfaceRegistry Singleton;
	return Singleton;
}

ITransformConstraintChannelInterface* FConstraintChannelInterfaceRegistry::FindConstraintChannelInterface(const UClass* InClass) const
{
	const TUniquePtr<ITransformConstraintChannelInterface>* Interface = HandleToInterfaceMap.Find(InClass);
	ensureMsgf(Interface, TEXT("No constraint channel interface found for class %s. Did you call RegisterConstraintChannelInterface<> for that class?"), *InClass->GetName());
	return Interface ? Interface->Get() : nullptr;
}