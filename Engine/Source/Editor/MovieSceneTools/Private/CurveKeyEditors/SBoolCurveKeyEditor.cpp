// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveKeyEditors/SBoolCurveKeyEditor.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "ScopedTransaction.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "BoolCurveKeyEditor"

void SBoolCurveKeyEditor::Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneBoolChannel, bool>& InKeyEditor)
{
	KeyEditor = InKeyEditor;

	ChildSlot
	[
		SNew(SCheckBox)
		.IsChecked(this, &SBoolCurveKeyEditor::IsChecked)
		.OnCheckStateChanged(this, &SBoolCurveKeyEditor::OnCheckStateChanged)
	];
}

ECheckBoxState SBoolCurveKeyEditor::IsChecked() const
{
	return KeyEditor.GetCurrentValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SBoolCurveKeyEditor::OnCheckStateChanged(ECheckBoxState NewCheckboxState)
{
	FScopedTransaction Transaction(LOCTEXT("SetBoolKey", "Set Bool Key Value"));

	const bool bNewValue = NewCheckboxState == ECheckBoxState::Checked;
	KeyEditor.SetValueWithNotify(bNewValue, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

#undef LOCTEXT_NAMESPACE
