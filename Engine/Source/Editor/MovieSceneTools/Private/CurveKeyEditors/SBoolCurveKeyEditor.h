// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneBoolChannel.h"
#include "HAL/Platform.h"
#include "CurveKeyEditors/SequencerKeyEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;

/**
 * A widget for editing a curve representing bool keys.
 */
class SBoolCurveKeyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBoolCurveKeyEditor) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneBoolChannel, bool>& InKeyEditor);

private:

	ECheckBoxState IsChecked() const;
	void OnCheckStateChanged(ECheckBoxState);

	TSequencerKeyEditor<FMovieSceneBoolChannel, bool> KeyEditor;
};
