// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveKeyEditors/SEnumKeyEditor.h"
#include "MovieSceneToolHelpers.h"
#include "ScopedTransaction.h"
#include "Channels/MovieSceneByteChannel.h"

#define LOCTEXT_NAMESPACE "EnumCurveKeyEditor"

void SEnumCurveKeyEditor::Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneByteChannel, uint8>& InKeyEditor, UEnum* Enum)
{
	KeyEditor = InKeyEditor;

	ChildSlot
	[
		MovieSceneToolHelpers::MakeEnumComboBox(
			Enum,
			TAttribute<int32>::Create(TAttribute<int32>::FGetter::CreateSP(this, &SEnumCurveKeyEditor::OnGetCurrentValueAsInt)),
			SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &SEnumCurveKeyEditor::OnChangeKey)
		)
	];
}

void SEnumCurveKeyEditor::OnChangeKey(int32 Selection, ESelectInfo::Type SelectionType)
{
	FScopedTransaction Transaction(LOCTEXT("SetKey", "Set Enum Key Value"));
	KeyEditor.SetValueWithNotify(Selection, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

#undef LOCTEXT_NAMESPACE
