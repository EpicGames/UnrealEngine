// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "CurveKeyEditors/SequencerKeyEditor.h"
#include "Channels/MovieSceneByteChannel.h"

class SEnumCurveKeyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEnumCurveKeyEditor) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneByteChannel, uint8>& InKeyEditor, UEnum* Enum);

private:

	int32 OnGetCurrentValueAsInt() const { return KeyEditor.GetCurrentValue(); }
	void OnChangeKey(int32 Selection, ESelectInfo::Type SelectionType);

	TSequencerKeyEditor<FMovieSceneByteChannel, uint8> KeyEditor;
};
