// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ScopedTransaction.h"
#include "Styling/SlateTypes.h"
#include "Editor.h"
#include "CurveKeyEditors/SequencerKeyEditor.h"
#include "Widgets/Input/SSpinBox.h"
#include "Styling/AppStyle.h"
#include "NumericPropertyParams.h"

#define LOCTEXT_NAMESPACE "NumericKeyEditor"

template<typename T>
struct SNonThrottledSpinBox : SSpinBox<T>
{
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = SSpinBox<T>::OnMouseButtonDown(MyGeometry, MouseEvent);
		if (Reply.IsEventHandled())
		{
			Reply.PreventThrottling();
		}
		return Reply;
	}

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		KeyEditor->BeginEditing(KeyEditor->GetCurrentTime());
		return SSpinBox<T>::OnFocusReceived(MyGeometry, InFocusEvent);
	}

	void SetKeyEditor(TSharedPtr<ISequencerKeyEditor<T>> InKeyEditor)
	{
		KeyEditor = InKeyEditor;
	}

private:

	TSharedPtr<ISequencerKeyEditor<T>> KeyEditor;
};

/**
 * A widget for editing a curve representing integer keys.
 */
template<typename NumericType>
class SNumericKeyEditorWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNumericKeyEditorWidget){}
	SLATE_END_ARGS()

	template<typename ChannelType>
	void Construct(const FArguments& InArgs, const TSequencerKeyEditor<ChannelType, NumericType>& InKeyEditor)
	{
		TSharedRef<TSequencerKeyEditorWrapper<ChannelType, NumericType>> KeyEditorWrapper =
			MakeShared<TSequencerKeyEditorWrapper<ChannelType, NumericType>>(InKeyEditor);

		Construct(InArgs, KeyEditorWrapper);
	}

	void Construct(const FArguments& InArgs, const TSharedRef<ISequencerKeyEditor<NumericType>>& InKeyEditor)
	{
		KeyEditor = InKeyEditor;

		const FProperty* Property = nullptr;
		ISequencer* Sequencer = InKeyEditor->GetSequencer();
		FTrackInstancePropertyBindings* PropertyBindings = InKeyEditor->GetPropertyBindings();
		if (Sequencer && PropertyBindings)
		{
			for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(InKeyEditor->GetObjectBindingID(), Sequencer->GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					Property = PropertyBindings->GetProperty(*Object);
					if (Property)
					{
						break;
					}
				}
			}
		}

		const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
		{
			return InKeyEditor->GetMetaData(Key);
		});

		TNumericPropertyParams<NumericType> NumericPropertyParams(Property, MetaDataGetter);

		ChildSlot
		[
			SAssignNew(SpinBox, SNonThrottledSpinBox<NumericType>)
			.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.MinValue(NumericPropertyParams.MinValue)
			.MaxValue(NumericPropertyParams.MaxValue)
			.TypeInterface(InKeyEditor->GetNumericTypeInterface())
			.MinSliderValue(NumericPropertyParams.MinSliderValue)
			.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
			.SliderExponent(NumericPropertyParams.SliderExponent)
			.Delta(NumericPropertyParams.Delta)
			// LinearDeltaSensitivity needs to be left unset if not provided, rather than being set to some default
			.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
			.WheelStep(NumericPropertyParams.WheelStep)
			.Value_Raw(KeyEditor.Get(), &ISequencerKeyEditor<NumericType>::GetCurrentValue)
			.OnValueChanged(this, &SNumericKeyEditorWidget::OnValueChanged)
			.OnValueCommitted(this, &SNumericKeyEditorWidget::OnValueCommitted)
			.OnBeginSliderMovement(this, &SNumericKeyEditorWidget::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &SNumericKeyEditorWidget::OnEndSliderMovement)
		];

		SpinBox->SetKeyEditor(KeyEditor);
	}

private:

	virtual FSlateColor GetForegroundColor() const override
	{
		if (KeyEditor->GetEditingKeySelection())
		{
			return FLinearColor::Yellow;
		}
		else
		{
			return FSlateColor::UseForeground();
		}
	}

	void OnBeginSliderMovement()
	{
		bSliding = true;
		GEditor->BeginTransaction(LOCTEXT("SetNumericKey", "Set Key Value"));
	}

	void OnEndSliderMovement(NumericType Value)
	{
		if (GEditor->IsTransactionActive())
		{
			KeyEditor->SetValue(Value);
			GEditor->EndTransaction();
		}
		bSliding = false;
	}

	void OnValueChanged(NumericType Value)
	{
		// Update the key editor only when spinning/sliding the value since the value hasn't been committed yet.
		// This should only happen in this case because OnValueChanged is called aggressively (ie. on focus changing)
		if (bSliding)
		{
			KeyEditor->SetValueWithNotify(Value, EMovieSceneDataChangeType::TrackValueChanged);
		}
	}

	void OnValueCommitted(NumericType Value, ETextCommit::Type CommitInfo)
	{
		if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus
#if !UE_BUILD_SHIPPING
			|| CommitInfo == ETextCommit::Default
#endif // !UE_BUILD_SHIPPING
			)
		{
			const FScopedTransaction Transaction( LOCTEXT("SetNumericKey", "Set Key Value") );
			KeyEditor->SetValueWithNotify(Value, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
		}

		KeyEditor->EndEditing();
	}

private:

	TSharedPtr<ISequencerKeyEditor<NumericType>> KeyEditor;
	TSharedPtr<SNonThrottledSpinBox<NumericType>> SpinBox;
	bool bSliding = false;
};

template<typename ChannelType, typename NumericType>
using SNumericKeyEditor = SNumericKeyEditorWidget<NumericType>;

#undef LOCTEXT_NAMESPACE
