// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveKeyEditors/SStringCurveKeyEditor.h"

#include "Containers/ArrayView.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "StringCurveKeyEditor"

void SStringCurveKeyEditor::Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneStringChannel, FString>& InKeyEditor)
{
	KeyEditor = InKeyEditor;

	ChildSlot
	[
		SNew(SEditableTextBox)
		.Padding(FMargin(0.0, 2.0, 0.0, 2.0))
		.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
		.MinDesiredWidth(10.f)
		.SelectAllTextWhenFocused(true)
		.Text(this, &SStringCurveKeyEditor::GetText)
		.OnTextCommitted(this, &SStringCurveKeyEditor::OnTextCommitted)
 	];
}

FText SStringCurveKeyEditor::GetText() const
{
	return FText::FromString(KeyEditor.GetCurrentValue());
}

void SStringCurveKeyEditor::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	FScopedTransaction Transaction(LOCTEXT("SetStringKey", "Set String Key Value"));
	KeyEditor.SetValueWithNotify(InText.ToString(), EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

#undef LOCTEXT_NAMESPACE
