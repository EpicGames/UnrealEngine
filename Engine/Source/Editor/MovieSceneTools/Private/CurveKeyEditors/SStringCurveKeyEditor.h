// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneStringChannel.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "CurveKeyEditors/SequencerKeyEditor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * A widget for editing a curve representing string keys.
 */
class SStringCurveKeyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStringCurveKeyEditor) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneStringChannel, FString>& InKeyEditor);

private:

	FText GetText() const;
	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	TSequencerKeyEditor<FMovieSceneStringChannel, FString> KeyEditor;
};
