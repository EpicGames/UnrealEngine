// Copyright Epic Games, Inc. All Rights Reserved.

#include "STextCurveKeyEditor.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "STextCurveKeyEditor"

void STextCurveKeyEditor::Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneTextChannel, FText>& InKeyEditor)
{
	KeyEditor = InKeyEditor;

	ChildSlot
	[
		SNew(SEditableTextBox)
		.MinDesiredWidth(10.f)
		.SelectAllTextWhenFocused(true)
		.Text(this, &STextCurveKeyEditor::GetText)
		.OnTextCommitted(this, &STextCurveKeyEditor::OnTextCommitted)
	];
}

FText STextCurveKeyEditor::GetText() const
{
	return KeyEditor.GetCurrentValue();
}

void STextCurveKeyEditor::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	FScopedTransaction Transaction(LOCTEXT("SetTextKey", "Set Text Key Value"));
	KeyEditor.SetValueWithNotify(InText, EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately);
}

#undef LOCTEXT_NAMESPACE
