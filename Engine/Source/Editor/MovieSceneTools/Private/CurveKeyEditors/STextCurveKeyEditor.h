// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerKeyEditor.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneTextChannel.h"
#include "Widgets/SCompoundWidget.h"

class ISequencer;

/**
 * A widget for editing a curve representing text keys.
 */
class STextCurveKeyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextCurveKeyEditor){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSequencerKeyEditor<FMovieSceneTextChannel, FText>& InKeyEditor);

private:
	/** Gets the evaluated Text value at the current sequencer time */
	FText GetText() const;

	/** Updates the CachedText result to the committed text and adds/updates a key to the text channel at the current sequencer time */
	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	TSequencerKeyEditor<FMovieSceneTextChannel, FText> KeyEditor;
};
