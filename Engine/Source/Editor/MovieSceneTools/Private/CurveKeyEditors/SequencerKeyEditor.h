// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "MovieSceneSection.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "MovieSceneCommonHelpers.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "SequencerChannelTraits.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "MovieSceneTimeHelpers.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

template<typename NumericType>
struct INumericTypeInterface;

template<typename ValueType>
struct ISequencerKeyEditor
{
	virtual ~ISequencerKeyEditor()= default;

	virtual TSharedPtr<INumericTypeInterface<ValueType>> GetNumericTypeInterface() const = 0;
	virtual TOptional<ValueType> GetExternalValue() const = 0;
	virtual ValueType GetCurrentValue() const = 0;
	virtual void SetValue(const ValueType& InValue) = 0;
	virtual void SetValueWithNotify(const ValueType& InValue, EMovieSceneDataChangeType NotifyType = EMovieSceneDataChangeType::TrackValueChanged) = 0;
	virtual const FGuid& GetObjectBindingID() const = 0;
	virtual ISequencer* GetSequencer() const = 0;
	virtual FTrackInstancePropertyBindings* GetPropertyBindings() const = 0;
	virtual FString GetMetaData(const FName& Key) const = 0;
	virtual bool GetEditingKeySelection() const = 0;
	virtual FFrameTime GetCurrentTime() const = 0;
	virtual void BeginEditing(FFrameTime) = 0;
	virtual void EndEditing() = 0;
};

template<typename ChannelType, typename ValueType>
struct TSequencerKeyEditor
{
	TSequencerKeyEditor()
	{}

	TSequencerKeyEditor(
		FGuid                                    InObjectBindingID,
		TMovieSceneChannelHandle<ChannelType>    InChannelHandle,
		TWeakObjectPtr<UMovieSceneSection>       InWeakSection,
		TWeakPtr<ISequencer>                     InWeakSequencer,
		TWeakPtr<FTrackInstancePropertyBindings> InWeakPropertyBindings,
		TFunction<TOptional<ValueType>(UObject&, FTrackInstancePropertyBindings*)> InOnGetExternalValue
	)
		: ObjectBindingID(InObjectBindingID)
		, ChannelHandle(InChannelHandle)
		, WeakSection(InWeakSection)
		, WeakSequencer(InWeakSequencer)
		, WeakPropertyBindings(InWeakPropertyBindings)
		, OnGetExternalValue(InOnGetExternalValue)
	{}

	static TOptional<ValueType> Get(const FGuid& ObjectBindingID, ISequencer* Sequencer, FTrackInstancePropertyBindings* PropertyBindings, const TFunction<TOptional<ValueType>(UObject&, FTrackInstancePropertyBindings*)>& OnGetExternalValue)
	{
		if (!Sequencer || !ObjectBindingID.IsValid() || !OnGetExternalValue)
		{
			return TOptional<ValueType>();
		}

		for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(ObjectBindingID, Sequencer->GetFocusedTemplateID()))
		{
			if (UObject* Object = WeakObject.Get())
			{
				TOptional<ValueType> ExternalValue = OnGetExternalValue(*Object, PropertyBindings);
				if (ExternalValue.IsSet())
				{
					return ExternalValue;
				}
			}
		}

		return TOptional<ValueType>();
	}

	void SetOwningObject(TWeakObjectPtr<UMovieSceneSignedObject> InWeakOwningObject)
	{
		WeakOwningObject = InWeakOwningObject;
	}

	void SetNumericTypeInterface(TSharedPtr<INumericTypeInterface<ValueType>> InNumericTypeInterface)
	{
		NumericTypeInterface = InNumericTypeInterface;
	}

	TSharedPtr<INumericTypeInterface<ValueType>> GetNumericTypeInterface() const
	{
		return NumericTypeInterface;
	}

	TOptional<ValueType> GetExternalValue() const
	{
		return Get(ObjectBindingID, WeakSequencer.Pin().Get(), WeakPropertyBindings.Pin().Get(), OnGetExternalValue);
	}

	ValueType GetCurrentValue() const
	{
		using namespace UE::MovieScene;

		ChannelType* Channel = ChannelHandle.Get();
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		UMovieSceneSection* OwningSection = WeakSection.Get();
		const FMovieSceneChannelMetaData* ChannelMetaData = ChannelHandle.GetMetaData();

		ValueType Result{};

		if (Channel && ChannelMetaData && Sequencer && OwningSection)
		{
			FFrameTime LocalTime = GetCurrentTime();
			const FFrameTime CurrentTime = UE::MovieScene::ClampToDiscreteRange(LocalTime, OwningSection->GetRange()) - ChannelMetaData->GetOffsetTime(OwningSection);

			//If we have no keys and no default, key with the external value if it exists
			if (!EvaluateChannel(OwningSection, Channel, CurrentTime, Result))
			{
				if (TOptional<ValueType> ExternalValue = GetExternalValue())
				{
					if (ExternalValue.IsSet())
					{
						Result = ExternalValue.GetValue();
					}
				}
			}

			if (ChannelMetaData->bInvertValue)
			{
				InvertValue(Result);
			}
		}

		return Result;
	}

	void InitializeChannelGroups() 
	{
		ChannelNameToInitialValue.Reset();

		using namespace UE::MovieScene;

		ChannelType* Channel = ChannelHandle.Get();
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		UMovieSceneSection* OwningSection = WeakSection.Get();
		const FMovieSceneChannelMetaData* ChannelMetaData = ChannelHandle.GetMetaData();

		if (Channel && ChannelMetaData && Sequencer && OwningSection)
		{
			FFrameTime LocalTime = GetCurrentTime();
			const FFrameTime CurrentTime = UE::MovieScene::ClampToDiscreteRange(LocalTime, OwningSection->GetRange()) - ChannelMetaData->GetOffsetTime(OwningSection);

			// Store the initial value of all the grouped channels
			const FMovieSceneChannelProxy& ChannelProxy = OwningSection->GetChannelProxy();
			TArray<FMovieSceneChannelHandle> GroupedChannels = GetGroupedChannels(ChannelProxy, ChannelMetaData->Group);
			for (FMovieSceneChannelHandle GroupChannelHandle : GroupedChannels)
			{
				const FMovieSceneChannelMetaData* GroupChannelMetaData = GroupChannelHandle.GetMetaData();
				if (!GroupChannelMetaData)
				{
					continue;
				}

				if (ChannelType* GroupChannel = reinterpret_cast<ChannelType*>(GroupChannelHandle.Get()))
				{
					ValueType Result{};

					if (EvaluateChannel(OwningSection, GroupChannel, CurrentTime, Result))
					{
						if (GroupChannelMetaData->bInvertValue)
						{
							InvertValue(Result);
						}

						ChannelNameToInitialValue.Add(GroupChannelMetaData->Name, Result);
					}
				}
			}
		}
	}

	bool GetEditingKeySelection() const
	{
		using namespace UE::MovieScene;
		using namespace Sequencer;
		using namespace UE::Sequencer;

		ChannelType* Channel = ChannelHandle.Get();

		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		const FKeySelection& KeySelection = Sequencer->GetViewModel()->GetSelection()->KeySelection;

		// Allow editing the key selection if the key editor's channel is one of the selected key's channels and there's more than 1 of those keys selected
		bool bAllowEditingKeySelection = false;
		int32 NumSelectedKeys = 0;
		for (FKeyHandle Key : KeySelection)
		{
			// Make sure we only manipulate the values of the channel with the same channel type we're editing
			TSharedPtr<FChannelModel> ChannelModel = KeySelection.GetModelForKey(Key);
			if (ChannelModel && ChannelModel->GetChannel() == Channel)
			{
				++NumSelectedKeys;
				if (NumSelectedKeys > 1)
				{
					return true;
				}
			}
		}

		return false;
	}

	TArray<FMovieSceneChannelHandle> GetGroupedChannels(const FMovieSceneChannelProxy& ChannelProxy, const FText& Group)
	{
		TArray<FMovieSceneChannelHandle> GroupedChannelHandles;

		for (const FMovieSceneChannelMetaData& EachChannelMetaData : ChannelProxy.GetMetaData<ChannelType>())
		{
			if (EachChannelMetaData.Group.IdenticalTo(Group))
			{
				FMovieSceneChannelHandle GroupChannelHandle = ChannelProxy.GetChannelByName(EachChannelMetaData.Name);
				GroupedChannelHandles.Add(GroupChannelHandle);
			}
		}

		return GroupedChannelHandles;
	}

	void SetValue(const ValueType& InValue)
	{
		using namespace UE::MovieScene;
		using namespace Sequencer;
		using namespace UE::Sequencer;

		UMovieSceneSection* OwningSection = WeakSection.Get();
		if (!OwningSection)
		{
			return;
		}

		ChannelType* Channel = ChannelHandle.Get();
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		const FMovieSceneChannelMetaData* ChannelMetaData = ChannelHandle.GetMetaData();

		if (OwningSection->IsReadOnly() || !Channel || !Sequencer || !ChannelMetaData)
		{
			return;
		}

		UMovieSceneSignedObject* Owner = WeakOwningObject.Get();
		if (!Owner)
		{
			Owner = OwningSection;
		}

		Owner->Modify();
		Owner->SetFlags(RF_Transactional);

		// When the owning object is different from the section (e.g., decoration
		// channels), also modify the section so its signature updates and
		// FChannelModel's cached keys detect the change.
		if (Owner != OwningSection)
		{
			OwningSection->Modify();
		}

		const bool bAutoSetTrackDefaults = Sequencer->GetAutoSetTrackDefaults();
		const bool bPreserveRatio = ChannelMetaData ? ChannelMetaData->bPreserveRatio : false;
		const bool bCanPreserveRatio = ChannelMetaData ? ChannelMetaData->bCanPreserveRatio : false;

		const FKeySelection& KeySelection = Sequencer->GetViewModel()->GetSelection()->KeySelection;

		ValueType NewValue = InValue;
		if (ChannelMetaData && ChannelMetaData->bInvertValue)
		{
			InvertValue(NewValue);
		}

		FFrameTime LocalTime = GetCurrentTime();
		const FFrameNumber CurrentTime = LocalTime.RoundToFrame() - ChannelMetaData->GetOffsetTime(OwningSection);

		const bool bEditingKeySelection = GetEditingKeySelection();
		if (bEditingKeySelection)
		{
			for (FKeyHandle Key : KeySelection)
			{
				// Make sure we only manipulate the values of the channel with the same channel type we're editing
				TSharedPtr<FChannelModel> ChannelModel = KeySelection.GetModelForKey(Key);
				if (ChannelModel && ChannelModel->GetKeyArea() && ChannelModel->GetKeyArea()->GetChannel().GetChannelTypeName() == ChannelHandle.GetChannelTypeName())
				{
					UMovieSceneSection* Section = ChannelModel->GetSection();
					if (Section && Section->TryModify())
					{
						AssignValue(reinterpret_cast<ChannelType*>(ChannelModel->GetChannel()), Key, NewValue);
					}
				}
			}
		}
		else if (bPreserveRatio && bCanPreserveRatio)
		{
			const FMovieSceneChannelProxy& ChannelProxy = OwningSection->GetChannelProxy();

			TArray<FMovieSceneChannelHandle> GroupedChannels = GetGroupedChannels(ChannelProxy, ChannelMetaData->Group);

			// First determine if there are any keys on any of the grouped channels. If any of the channels have keys, all channels will need to be keyed
			bool bHasAnyKeys = false;
			for (FMovieSceneChannelHandle GroupChannelHandle : GroupedChannels)
			{
				if (ChannelType* GroupChannel = reinterpret_cast<ChannelType*>(GroupChannelHandle.Get()))
				{
					if (GroupChannel->GetNumKeys() != 0)
					{
						bHasAnyKeys = true;
						break;
					}
				}
			}

			ValueType Ratio = NewValue;
			if (ChannelNameToInitialValue.Contains(ChannelMetaData->Name))
			{
				FMovieSceneChannelTraitsTransform<ValueType> Transform;
				Transform.Scale = ChannelNameToInitialValue[ChannelMetaData->Name];
				ReciprocalValue(Transform.Scale);
				TransformValue(Ratio, Transform);
			}

			for (FMovieSceneChannelHandle GroupChannelHandle : GroupedChannels)
			{
				if (ChannelType* GroupChannel = reinterpret_cast<ChannelType*>(GroupChannelHandle.Get()))
				{
					const FMovieSceneChannelMetaData* GroupChannelMetaData = GroupChannelHandle.GetMetaData();
					if (!GroupChannelMetaData)
					{
						continue;
					}

					ValueType NewPreserveRatioValue = NewValue;
					if (ChannelNameToInitialValue.Contains(GroupChannelMetaData->Name))
					{
						NewPreserveRatioValue = ChannelNameToInitialValue[GroupChannelMetaData->Name];
						FMovieSceneChannelTraitsTransform<ValueType> Transform;
						Transform.Scale = Ratio;
						TransformValue(NewPreserveRatioValue, Transform);
					}

					if (bHasAnyKeys)
					{
						TArray<FKeyHandle> GroupKeysAtCurrentTime;
						GroupChannel->GetKeys(TRange<FFrameNumber>(CurrentTime), nullptr, &GroupKeysAtCurrentTime);

						if (GroupKeysAtCurrentTime.Num() > 0)
						{
							AssignValue(GroupChannel, GroupKeysAtCurrentTime[0], NewPreserveRatioValue);
						}
						else
						{
							EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(GroupChannel, CurrentTime, Sequencer->GetKeyInterpolation());
							if (!ValueExistsAtTime(GroupChannel, CurrentTime, NewPreserveRatioValue))
							{
								AddKeyToChannel(GroupChannel, CurrentTime, NewPreserveRatioValue, Interpolation);
							}
							
							OwningSection->ExpandToFrame(LocalTime.RoundToFrame());
						}
					}

					// Always update the default value when auto-set default values is enabled so that the last changes
					// are always saved to the track.
					if (bAutoSetTrackDefaults)
					{
						SetChannelDefault(GroupChannel, NewPreserveRatioValue);
					}

					//need to tell channel change happened (float will call AutoSetTangents())
					GroupChannel->PostEditChange();
					TArray<FFrameNumber> Frames;
					Frames.Add(CurrentTime);
					Sequencer->OnChannelChangedWithTime().Broadcast(Frames, GroupChannelMetaData, OwningSection);
				}
			}
		}
		else
		{
			EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(Channel, CurrentTime, Sequencer->GetKeyInterpolation());

			TArray<FKeyHandle> KeysAtCurrentTime;
			Channel->GetKeys(TRange<FFrameNumber>(CurrentTime), nullptr, &KeysAtCurrentTime);

			if (KeysAtCurrentTime.Num() > 0)
			{
				AssignValue(Channel, KeysAtCurrentTime[0], NewValue);
			}
			else
			{
				bool bHasAnyKeys = Channel->GetNumKeys() != 0;

				if (bHasAnyKeys || bAutoSetTrackDefaults == false)
				{
					// When auto setting track defaults are disabled, add a key even when it's empty so that the changed
					// value is saved and is propagated to the property.
					if (!ValueExistsAtTime(Channel, CurrentTime, NewValue))
					{
						AddKeyToChannel(Channel, CurrentTime, NewValue, Interpolation);
					}
					
					bHasAnyKeys = Channel->GetNumKeys() != 0;
				}

				if (bHasAnyKeys)
				{
					OwningSection->ExpandToFrame(LocalTime.RoundToFrame());
				}
			}

			// Always update the default value when auto-set default values is enabled so that the last changes
			// are always saved to the track.
			if (bAutoSetTrackDefaults)
			{
				SetChannelDefault(Channel, NewValue);
			}
		 
			//need to tell channel change happened (float will call AutoSetTangents())
			Channel->PostEditChange();
			TArray<FFrameNumber> Frames;
			Frames.Add(CurrentTime);
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			Sequencer->OnChannelChangedWithTime().Broadcast(Frames, MetaData, OwningSection);
		}
	}

	void SetValueWithNotify(const ValueType& InValue, EMovieSceneDataChangeType NotifyType = EMovieSceneDataChangeType::TrackValueChanged)
	{
		SetValue(InValue);
		if (ISequencer* Sequencer = WeakSequencer.Pin().Get())
		{
			Sequencer->NotifyMovieSceneDataChanged(NotifyType);
		}
	}

	void SetApplyInUnwarpedLocalSpace(bool bInApplyInUnwarpedLocalSpace)
	{
		bApplyInUnwarpedLocalSpace = bInApplyInUnwarpedLocalSpace;
	}

	const FGuid& GetObjectBindingID() const
	{
		return ObjectBindingID;
	}

	ISequencer* GetSequencer() const
	{
		return WeakSequencer.Pin().Get();
	}

	FTrackInstancePropertyBindings* GetPropertyBindings() const
	{
		return WeakPropertyBindings.Pin().Get();
	}

	FString GetMetaData(const FName& Key) const
	{
		ISequencer* Sequencer = GetSequencer();
		FTrackInstancePropertyBindings* PropertyBindings = GetPropertyBindings();
		if (Sequencer && PropertyBindings)
		{
			for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(ObjectBindingID, Sequencer->GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					if (FProperty* Property = PropertyBindings->GetProperty(*Object))
					{
						return Property->GetMetaData(Key);
					}
				}
			}
		}

		if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
		{
			return MetaData->GetPropertyMetaData(Key);
		}

		return FString();
	}

	FFrameTime GetCurrentTime() const
	{
		if (EditingTime.IsSet())
		{
			return EditingTime.GetValue();
		}

		ISequencer* Sequencer = GetSequencer();
		if (Sequencer)
		{
			// @todo: Really bApplyInUnwarpedLocalSpace should be looking for an ITimeDomainExtension on a view model, but we don't
			//        have that information here because all these mechanisms pre-date the MVVM framework.
			return bApplyInUnwarpedLocalSpace ? Sequencer->GetUnwarpedLocalTime().Time : Sequencer->GetLocalTime().Time;
		}
		return 0;
	}

	void BeginEditing(FFrameTime InEditingTime)
	{
		EditingTime = InEditingTime;

		InitializeChannelGroups();
	}

	void EndEditing()
	{
		EditingTime.Reset();
	}

private:

	FGuid ObjectBindingID;
	TMovieSceneChannelHandle<ChannelType> ChannelHandle;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TWeakObjectPtr<UMovieSceneSignedObject> WeakOwningObject;
	TWeakPtr<ISequencer> WeakSequencer;
	TWeakPtr<FTrackInstancePropertyBindings> WeakPropertyBindings;
	TFunction<TOptional<ValueType>(UObject&, FTrackInstancePropertyBindings*)> OnGetExternalValue;
	TSharedPtr<INumericTypeInterface<ValueType>> NumericTypeInterface;
	bool bApplyInUnwarpedLocalSpace = false;

	// The time at which to edit the key value. This can be enforced when receiving focus on the spin box 
	// so that when focus is lost and the value is commited, the value is committed at the editing time 
	// rather than the Sequencer time which could have changed.
	TOptional<FFrameTime> EditingTime;
	TMap<FName, ValueType> ChannelNameToInitialValue;
};




template<typename ChannelType, typename ValueType>
struct TSequencerKeyEditorWrapper : ISequencerKeyEditor<ValueType>
{
	TSequencerKeyEditorWrapper(const TSequencerKeyEditor<ChannelType, ValueType>& InKeyEditor)
		: Impl(InKeyEditor)
	{}

	TSharedPtr<INumericTypeInterface<ValueType>> GetNumericTypeInterface() const override
	{
		return Impl.GetNumericTypeInterface();
	}
	TOptional<ValueType> GetExternalValue() const override
	{
		return Impl.GetExternalValue();
	}
	ValueType GetCurrentValue() const override
	{
		return Impl.GetCurrentValue();
	}
	void SetValue(const ValueType& InValue) override
	{
		return Impl.SetValue(InValue);
	}
	void SetValueWithNotify(const ValueType& InValue, EMovieSceneDataChangeType NotifyType) override
	{
		return Impl.SetValueWithNotify(InValue, NotifyType);
	}
	const FGuid& GetObjectBindingID() const override
	{
		return Impl.GetObjectBindingID();
	}
	ISequencer* GetSequencer() const override
	{
		return Impl.GetSequencer();
	}
	FTrackInstancePropertyBindings* GetPropertyBindings() const override
	{
		return Impl.GetPropertyBindings();
	}
	FString GetMetaData(const FName& Key) const override
	{
		return Impl.GetMetaData(Key);
	}
	bool GetEditingKeySelection() const override
	{
		return Impl.GetEditingKeySelection();
	}
	FFrameTime GetCurrentTime() const override
	{
		return Impl.GetCurrentTime();
	}
	void BeginEditing(FFrameTime InFrameTime) override
	{
		Impl.BeginEditing(InFrameTime);
	}
	void EndEditing() override
	{
		Impl.EndEditing();
	}

private:

	TSequencerKeyEditor<ChannelType, ValueType> Impl;
};
