// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/IMovieSceneDecorationMenuProvider.h"

FName IMovieSceneDecorationMenuProvider::GetModularFeatureName()
{
	static FName FeatureName = FName(TEXT("MovieSceneDecorationProvider"));
	return FeatureName;
}

IMovieSceneDecorationMenuProvider::~IMovieSceneDecorationMenuProvider()
{
}
