// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneDecorationContainerCustomization.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "PropertyHandle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IMovieSceneModule.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Templates/SubclassOf.h"

#define LOCTEXT_NAMESPACE "MovieSceneDecorationContainer"

TSharedRef<IPropertyTypeCustomization> FMovieSceneDecorationContainerCustomization::MakeInstance()
{
	return MakeShared<FMovieSceneDecorationContainerCustomization>();
}

void FMovieSceneDecorationContainerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// We don't show this property, just the children, and don't want to show the header
	PropertyHandle->MarkHiddenByCustomization();
}

void FMovieSceneDecorationContainerCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	// Add categories to parent layout to prevent an extra layer of headers
	IDetailLayoutBuilder& LayoutBuilder = ChildBuilder.GetParentCategory().GetParentLayout();

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() > 0)
	{
		if (UMovieSceneDecorationContainerObject* Container = Cast<UMovieSceneDecorationContainerObject>(OuterObjects[0]))
		{
			// Get all compatible (optional/removable) decorations for this container
			TSet<UClass*> CompatibleClasses;
			IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(Container, CompatibleClasses);

			for (UObject* DecorationObject : Container->GetDecorations())
			{
				if (DecorationObject)
				{
					UClass* DecorationClass = DecorationObject->GetClass();

					// Add category for the decoration object based on its class display name. Sort it to the bottom using uncommon category priority.
					IDetailCategoryBuilder& DecorationCategory = LayoutBuilder.EditCategory(*DecorationClass->GetName(),
						DecorationClass->GetDisplayNameText(), ECategoryPriority::Uncommon);

					// If this decoration is from the compatible list, add a remove button (b/c that means it's optional and the user has added it)
					if (CompatibleClasses.Contains(DecorationClass))
					{
						DecorationCategory.HeaderContent(
							SNew(SButton)
							.OnClicked_Lambda([this, Container, DecorationClass]()
							{
								OnRemoveDecoration(Container, DecorationClass);
								return FReply::Handled();
							})
							.ToolTipText(FText::Format(LOCTEXT("RemoveModifiersTooltip", "Remove {0} modifier from this container"), DecorationClass->GetDisplayNameText()))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("RemoveModifier", "Remove Modifier"))
							]
						);
					}

					// Add the object as external
					DecorationCategory.AddExternalObjects(TArray<UObject*>({DecorationObject}), EPropertyLocation::Default,
						FAddPropertyParams().HideRootObjectNode(true).AllowChildren(true));
				}
			}

			// Add UI for adding compatible decorations
			AddCompatibleDecorationsUI(Container, LayoutBuilder);
		}
	}
}

void FMovieSceneDecorationContainerCustomization::AddCompatibleDecorationsUI(UMovieSceneDecorationContainerObject* Container, IDetailLayoutBuilder& LayoutBuilder)
{
	// Get all compatible decorations from the MovieSceneModule registry and the container's own implementation
	TSet<UClass*> CompatibleClasses;
	IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(Container, CompatibleClasses);

	// Filter out already added decorations (only one of each type allowed)
	TArrayView<const TObjectPtr<UObject>> ExistingDecorations = Container->GetDecorations();
	for (const TObjectPtr<UObject>& Decoration : ExistingDecorations)
	{
		if (Decoration)
		{
			CompatibleClasses.Remove(Decoration->GetClass());
		}
	}

	// Only show the UI if there are compatible decorations to add
	if (CompatibleClasses.Num() == 0)
	{
		return;
	}

	// Create a category for adding decorations
	IDetailCategoryBuilder& AddDecorationsCategory = LayoutBuilder.EditCategory("Modifiers",
		LOCTEXT("AddModifiersCategory", "Modifiers"), ECategoryPriority::Uncommon);

	AddDecorationsCategory.AddCustomRow(LOCTEXT("AddModifierRow", "Add Modifier"))
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FMovieSceneDecorationContainerCustomization::OnGetAddDecorationMenuContent, Container, CompatibleClasses)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddModifier", "Add Modifier"))
				]
			]
		];
}

TSharedRef<SWidget> FMovieSceneDecorationContainerCustomization::OnGetAddDecorationMenuContent(UMovieSceneDecorationContainerObject* Container, TSet<UClass*> CompatibleClasses)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (UClass* DecorationClass : CompatibleClasses)
	{
		if (!DecorationClass)
		{
			continue;
		}

		FText DisplayName = DecorationClass->GetDisplayNameText();
		FText ToolTip = FText::Format(LOCTEXT("AddModifierTooltip", "Add {0} modifier to this container"), DisplayName);

		MenuBuilder.AddMenuEntry(
			DisplayName,
			ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FMovieSceneDecorationContainerCustomization::OnAddDecoration, Container, DecorationClass)
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

void FMovieSceneDecorationContainerCustomization::OnAddDecoration(UMovieSceneDecorationContainerObject* Container, UClass* DecorationClass)
{
	if (!Container || !DecorationClass)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddModifier", "Add Modifier"));
	Container->Modify();

	UObject* NewDecoration = NewObject<UObject>(Container, DecorationClass, NAME_None, RF_Transactional);
	Container->AddDecoration(NewDecoration);

	if (PropertyUtils.IsValid())
	{
		PropertyUtils.Pin()->RequestForceRefresh();
	}
}

void FMovieSceneDecorationContainerCustomization::OnRemoveDecoration(UMovieSceneDecorationContainerObject* Container, UClass* DecorationClass)
{
	if (!Container || !DecorationClass)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveModifierTransaction", "Remove Modifier"));
	Container->Modify();

	Container->RemoveDecoration(DecorationClass);

	if (PropertyUtils.IsValid())
	{
		PropertyUtils.Pin()->RequestForceRefresh();
	}
}

#undef LOCTEXT_NAMESPACE