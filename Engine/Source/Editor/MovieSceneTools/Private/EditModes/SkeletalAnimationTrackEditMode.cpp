// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalAnimationTrackEditMode.h"
#include "Toolkits/ToolkitManager.h"
#include "HitProxies.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "EditorViewportClient.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineUtils.h"
#include "Engine/SkeletalMesh.h"

#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"

#include "AnimationRuntime.h"
#include "Animation/AnimInstance.h"
#include "SkeletalDebugRendering.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributesRuntime.h"

#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneSkeletalAnimationRootHitProxy.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "SkeletalDebugRendering.h"
#include "AnimSequencerInstanceProxy.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"

FName FSkeletalAnimationTrackEditMode::ModeName("EditMode.SkeletalAnimationTrackEditMode");

#define LOCTEXT_NAMESPACE "SkeletalAnimationTrackEditMode"

//// Edit Mode Tool used by the Edit Mode

class FSkeletalAnimationTrackEditModeTool : public FModeTool
{
public:
	FSkeletalAnimationTrackEditModeTool(FSkeletalAnimationTrackEditMode* InMode) : EdMode(InMode) {}
	virtual ~FSkeletalAnimationTrackEditModeTool() {};

	virtual FString GetName() const override { return TEXT("Sequencer Edit"); }

	/**
	 * @return		true if the key was handled by this editor mode tool.
	 */
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;

private:
	FSkeletalAnimationTrackEditMode* EdMode;
};

bool FSkeletalAnimationTrackEditModeTool::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (Key == EKeys::LeftMouseButton)
	{
		if (Event == IE_Pressed)
		{
			int32 HitX = ViewportClient->Viewport->GetMouseX();
			int32 HitY = ViewportClient->Viewport->GetMouseY();
			HHitProxy* HitResult = ViewportClient->Viewport->GetHitProxy(HitX, HitY);

			if (HitResult)
			{
				if (HitResult->IsA(HMovieSceneSkeletalAnimationRootHitProxy::StaticGetType()))
				{
					HMovieSceneSkeletalAnimationRootHitProxy* Proxy = (HMovieSceneSkeletalAnimationRootHitProxy*)HitResult;
					EdMode->OnKeySelected(ViewportClient->Viewport, Proxy);
				}
			}
		}
	}

	return FModeTool::InputKey(ViewportClient, Viewport, Key, Event);
}


///// EDIT MODE
FSkeletalAnimationTrackEditMode::FSkeletalAnimationTrackEditMode()
	: bIsTransacting(false)
	, bManipulatorMadeChange(false)
	, RootTransform(FTransform::Identity)
{
	FSkeletalAnimationTrackEditModeTool* EdModeTool = new FSkeletalAnimationTrackEditModeTool(this);

	Tools.Add(EdModeTool);
	SetCurrentTool(EdModeTool);
}

FSkeletalAnimationTrackEditMode::~FSkeletalAnimationTrackEditMode()
{
}

bool FSkeletalAnimationTrackEditMode::UsesToolkits() const
{
	return false;
}

void FSkeletalAnimationTrackEditMode::Enter()
{
	// Call parent implementation
	FEdMode::Enter();


}

void FSkeletalAnimationTrackEditMode::Exit()
{
	if (bIsTransacting)
	{
		if (GEditor)
		{
			GEditor->EndTransaction();
		}
		bIsTransacting = false;
	}
	FEdMode::Exit();
}

//todo remove me in UE5
void FSkeletalAnimationTrackEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);
}
// check to see if virtual bone, only way is by bone name

static bool IsVirtualBone(USkeleton* Skeleton, FName BoneName)
{
	checkf(Skeleton != nullptr, TEXT("Invalid Skeleton ptr"));
	return Skeleton->GetVirtualBones().IndexOfByPredicate([&](const FVirtualBone& VirtualBone) { return VirtualBone.VirtualBoneName == BoneName; }) != INDEX_NONE;
}

static void DrawBonesFromCompactPose(const FCompactPose& Pose, USkeletalMeshComponent* MeshComponent, bool bIncludeRootBone, FPrimitiveDrawInterface* PDI,
	const FLinearColor& DrawColour, FCompactPoseBoneIndex RootBoneIndex, FVector& LocationOfRootBone)
{
	LocationOfRootBone = FVector(TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max());
	if (Pose.GetNumBones() > 0 && MeshComponent != nullptr)
	{
		FSkelDebugDrawConfig DrawConfig;
		DrawConfig.BoneDrawMode = EBoneDrawMode::All;
		DrawConfig.BoneDrawSize = 1.f;
		DrawConfig.bAddHitProxy = false;
		DrawConfig.bForceDraw = true;
		DrawConfig.DefaultBoneColor = DrawColour;
		DrawConfig.AffectedBoneColor = DrawColour;
		DrawConfig.SelectedBoneColor = DrawColour;
		DrawConfig.ParentOfSelectedBoneColor = DrawColour;

		TArray<uint16> RequiredBones;
		TArray<FTransform> WorldTransforms;
		WorldTransforms.AddUninitialized(Pose.GetBoneContainer().GetNumBones());

		TArray<FLinearColor> BoneColours;
		BoneColours.AddUninitialized(Pose.GetBoneContainer().GetNumBones());
		
		const FReferenceSkeleton& RefSkeleton = Pose.GetBoneContainer().GetReferenceSkeleton();

		// we could cache parent bones as we calculate, but right now I'm not worried about perf issue of this
		for (FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
		{
			if (IsVirtualBone(MeshComponent->GetSkeletalMeshAsset()->GetSkeleton(), RefSkeleton.GetBoneName(BoneIndex.GetInt())))
			{
				continue;
			}
			FMeshPoseBoneIndex MeshBoneIndex = Pose.GetBoneContainer().MakeMeshPoseIndex(BoneIndex);
			RequiredBones.Add(MeshBoneIndex.GetInt());

			int32 ParentIndex = Pose.GetBoneContainer().GetParentBoneIndex(MeshBoneIndex.GetInt());


			if (ParentIndex == INDEX_NONE)
			{
				FTransform RootBone = (bIncludeRootBone == true) ? Pose[BoneIndex] : FTransform::Identity;
				WorldTransforms[MeshBoneIndex.GetInt()] = RootBone * MeshComponent->GetComponentTransform();
			}
			else
			{
				WorldTransforms[MeshBoneIndex.GetInt()] = Pose[BoneIndex] * WorldTransforms[ParentIndex];
			}
			if (RootBoneIndex == BoneIndex)
			{
				LocationOfRootBone.X = WorldTransforms[MeshBoneIndex.GetInt()].GetLocation().X;
				LocationOfRootBone.Y = WorldTransforms[MeshBoneIndex.GetInt()].GetLocation().Y;
			}
			if (LocationOfRootBone.Z > WorldTransforms[MeshBoneIndex.GetInt()].GetLocation().Z)
			{
				LocationOfRootBone.Z = WorldTransforms[MeshBoneIndex.GetInt()].GetLocation().Z;
			}
			BoneColours[MeshBoneIndex.GetInt()] = DrawColour;
		}
		if (MeshComponent && MeshComponent->GetSkeletalMeshAsset())
		{
			SkeletalDebugRendering::DrawBones(
				PDI,
				MeshComponent->GetComponentLocation(),
				RequiredBones,
				MeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton(),
				WorldTransforms,
				/*SelectedBones*/TArray<int32>(),
				BoneColours,
				/*HitProxies*/TArray<TRefCountPtr<HHitProxy>>(),
				DrawConfig
			);
		}
	}
}
void FSkeletalAnimationTrackEditMode::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdMode::AddReferencedObjects(Collector);
}


static void RenderTrail(UObject* BoundObject, UMovieSceneSkeletalAnimationTrack* SkelAnimTrack, UE::MovieScene::FSystemInterrogator& Interrogator, USkeletalMeshComponent* SkelMeshComp,
	const TSharedPtr<ISequencer>& Sequencer, FPrimitiveDrawInterface* PDI)
{
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FLinearColor RootMotionColor(0.75, 0.75, 0.75, 1.0f);
	TOptional<FVector> OldKeyPos_G;

	int32 Index = 0;
	if (SkelAnimTrack->GetAllSections().Num() > 0)
	{
		TOptional<FTransform> InitialActorTransform;

		const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
		const UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
		if (EntityLinker)
		{
			const UMovieSceneSkeletalAnimationSystem* SkeletalAnimationSystem = EntityLinker->FindSystem<UMovieSceneSkeletalAnimationSystem>();
			if (SkeletalAnimationSystem)
			{
				// Attempt to get the pre-root motion actor transform so we can display the actor along the root motion path properly
				InitialActorTransform = SkeletalAnimationSystem->GetInitialActorTransform(SkelMeshComp);
				TOptional<FQuat> InverseMeshToActorRotation = SkeletalAnimationSystem->GetInverseMeshToActorRotation(SkelMeshComp);
				if (InitialActorTransform.IsSet() && InverseMeshToActorRotation.IsSet())
				{
					InitialActorTransform->SetRotation(InitialActorTransform->GetRotation() * InverseMeshToActorRotation.GetValue());
				}
			}
		}

		const FTransform SkelMeshTransform = SkelAnimTrack->RootMotionParams.RootMotionStartOffset * ((InitialActorTransform.IsSet()) ? InitialActorTransform.GetValue() : SkelMeshComp->GetSocketTransform(NAME_None));
		for (const FTransform& Transform : SkelAnimTrack->RootMotionParams.RootTransforms)
		{
			FVector NewKeyPos = Transform.GetLocation();
			FVector NewKeyPos_G = SkelMeshTransform.TransformPosition(NewKeyPos);
			if (OldKeyPos_G.IsSet())
			{
				PDI->DrawLine(OldKeyPos_G.GetValue(), NewKeyPos_G, RootMotionColor, SDPG_Foreground);
			}
			OldKeyPos_G = NewKeyPos_G;
		}
	}
}

void FSkeletalAnimationTrackEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}
	// Gather a map of object bindings to their implict selection state
	TMap<const FMovieSceneBinding*, bool> ObjectBindingNodesSelectionMap;

	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		TArrayView<TWeakObjectPtr<UObject>> BoundObjects = Sequencer->FindObjectsInCurrentSequence(Binding.GetObjectGuid());
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if(!Track->IsEvalDisabled())
			{ 
				if (UMovieSceneSkeletalAnimationTrack* SkelAnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
				{
					for (TWeakObjectPtr<> WeakBinding : BoundObjects)
					{
						UObject* BoundObject = WeakBinding.Get();
						if (BoundObject)
						{
							
							USkeletalMeshComponent* SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(BoundObject);
							if (SkelMeshComp && SkelMeshComp->GetAnimInstance())
							{

								//show root motions
								if (SkelAnimTrack->bShowRootMotionTrail)
								{
									if (SkelAnimTrack->RootMotionParams.bCacheRootTransforms == false)
									{
										SkelAnimTrack->RootMotionParams.bCacheRootTransforms = true;
										SkelAnimTrack->SetUpRootMotions(true);

									}
									RenderTrail(BoundObject, SkelAnimTrack, InterrogationLinker, SkelMeshComp, Sequencer, PDI);								
								}
								else
								{
									SkelAnimTrack->RootMotionParams.bCacheRootTransforms = false;
									SkelAnimTrack->RootMotionParams.RootTransforms.SetNum(0);
								}
								//show skeletons

								const FLinearColor Colors[5] = { FLinearColor(1.0f,0.0f,1.0f,1.0f), FLinearColor(0.0f,1.0f,0.0f,1.0f), FLinearColor(0.0f, 0.0f, 1.0f, 1.0f),
									FLinearColor(0.5f, 0.5f, 0.0f, 1.0f), FLinearColor(0.0f, 0.5f, 0.5f, 1.0f) };

								int32 SectionIndex = 0;
								FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();

								for (UMovieSceneSection* Section : Track->GetAllSections())
								{
									UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
									if (AnimSection && AnimSection->bShowSkeleton && AnimSection->IsActive())
									{
										UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSection->Params.Animation);
										if (AnimSequence)
										{
											int32 Index = AnimSection->SetBoneIndexForRootMotionCalculations(SkelAnimTrack->bBlendFirstChildOfRoot);
											FMemMark Mark(FMemStack::Get());	
											FCompactPose OutPose;
											OutPose.ResetToRefPose(SkelMeshComp->GetAnimInstance()->GetRequiredBones());

											FBlendedCurve OutCurve;
											OutCurve.InitFrom(SkelMeshComp->GetAnimInstance()->GetRequiredBones());
											UE::Anim::FStackAttributeContainer TempAttributes;
											FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);

											const double Seconds = AnimSection->MapTimeToAnimation(CurrentTime.Time, CurrentTime.Rate);
											FAnimExtractContext ExtractionContext(Seconds, false);

											AnimSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);

											FFrameTime TimeToSample = CurrentTime.Time;
											if (TimeToSample < AnimSection->GetRange().GetLowerBoundValue())
											{
												TimeToSample = AnimSection->GetRange().GetLowerBoundValue();
											}
											else if (TimeToSample > AnimSection->GetRange().GetUpperBoundValue())
											{
												TimeToSample = AnimSection->GetRange().GetUpperBoundValue();
											}
											FTransform RootMotionAtStart, ParentTransform;
											UMovieSceneSkeletalAnimationSection::FRootMotionTransformParam Param;
											Param.FrameRate = MovieScene->GetTickResolution();
											Param.CurrentTime = TimeToSample;
											AnimSection->GetRootMotionTransform(AnimationPoseData, Param);
											RootMotionAtStart = Param.OutTransform;
											RootMotionAtStart = RootMotionAtStart * AnimSection->PreviousTransform;
											FCompactPoseBoneIndex PoseIndex = AnimationPoseData.GetPose().GetBoneContainer().GetCompactPoseIndexFromSkeletonIndex(Index);
											OutPose[PoseIndex] = RootMotionAtStart;

											FLinearColor SectionColor = FLinearColor(AnimSection->GetColorTint());

											const float Alpha = SectionColor.A;
											SectionColor.A = 1.f;

											FLinearColor BoneColor = Colors[SectionIndex % 5] * (1.f - Alpha) + SectionColor * Alpha;

											FVector RootLocation;
											DrawBonesFromCompactPose(OutPose, SkelMeshComp, (AnimSection->Params.SwapRootBone == ESwapRootBone::SwapRootBone_None), PDI,
												BoneColor, PoseIndex, RootLocation);
											
											if (IsRootSelected(AnimSection))
											{
												RootTransform = RootMotionAtStart;
												RootTransform.SetLocation(RootLocation);
											}

											if (PDI != nullptr)
											{
												if (PDI->IsHitTesting())
												{
													PDI->SetHitProxy(new HMovieSceneSkeletalAnimationRootHitProxy(AnimSection, SkelMeshComp));
												}
												PDI->DrawPoint(RootLocation, Colors[SectionIndex % 5], 10.f, SDPG_Foreground);
												if (PDI->IsHitTesting())
												{
													PDI->SetHitProxy(nullptr);
												}

											}
											++SectionIndex;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

bool FSkeletalAnimationTrackEditMode::IsRootSelected(UMovieSceneSkeletalAnimationSection* AnimSection) const
{
	for (const FSelectedRootData& RootData : SelectedRootData)
	{
		if (RootData.SelectedSection.Get() == AnimSection)
		{
			return true;
		}
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	return FEdMode::InputKey(InViewportClient, InViewport, InKey, InEvent);
}

bool FSkeletalAnimationTrackEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (bIsTransacting)
	{
		if (bManipulatorMadeChange)
		{
			bManipulatorMadeChange = false;
			GEditor->EndTransaction();

		}
		bIsTransacting = false;
		return true;
	}

	bManipulatorMadeChange = false;

	return false;
}

bool FSkeletalAnimationTrackEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (!bIsTransacting)
	{
		bIsTransacting = (IsSomethingSelected());

		bManipulatorMadeChange = false;

		return bIsTransacting;
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::BeginTransform(const FGizmoState& InState)
{
	if (!bIsTransacting)
	{
		bIsTransacting = (IsSomethingSelected());

		bManipulatorMadeChange = false;

		return bIsTransacting;
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::EndTransform(const FGizmoState& InState)
{
	if (bIsTransacting)
	{
		if (bManipulatorMadeChange)
		{
			bManipulatorMadeChange = false;
			GEditor->EndTransaction();

		}
		bIsTransacting = false;
		return true;
	}

	bManipulatorMadeChange = false;

	return false;
}

bool FSkeletalAnimationTrackEditMode::UsesTransformWidget() const
{
	return IsSomethingSelected();
}

bool FSkeletalAnimationTrackEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return IsSomethingSelected();
}

FVector FSkeletalAnimationTrackEditMode::GetWidgetLocation() const
{
	FVector PivotLocation(0.0, 0.0, 0.0);
	if (IsSomethingSelected() && WeakSequencer.IsValid())
	{
		FQualifiedFrameTime CurrentTime = WeakSequencer.Pin()->GetLocalTime();
		bool  bSelectionValid = false;
		for (FSelectedRootData Data : SelectedRootData)
		{
			if (Data.SelectedMeshComp.IsValid() && Data.SelectedSection.IsValid() && Data.SelectedSection->GetRange().Contains(CurrentTime.Time.GetFrame()))
			{
				bSelectionValid = true;
			}
		}
		if (bSelectionValid)
		{
			PivotLocation = RootTransform.GetLocation();
			return PivotLocation;
		}
	}
	return FEdMode::GetWidgetLocation();
}

bool FSkeletalAnimationTrackEditMode::GetPivotForOrbit(FVector& OutPivot) const
{
	if (IsSomethingSelected())
	{
		OutPivot = GetWidgetLocation();
		return true;
	}
	return FEdMode::GetPivotForOrbit(OutPivot);
}

bool FSkeletalAnimationTrackEditMode::GetTransformAtFirstSectionStart(FTransform& OutTransform, FTransform& OutParentTransform) const
{ 
	if (IsSomethingSelected() && WeakSequencer.IsValid())
	{
		FQualifiedFrameTime CurrentTime = WeakSequencer.Pin()->GetLocalTime();
		int32 NumSelected = 0;
		for (FSelectedRootData Data : SelectedRootData)
		{
			if (Data.SelectedMeshComp.IsValid() && Data.SelectedMeshComp->GetAnimInstance() && Data.SelectedSection.IsValid() && Data.SelectedSection->GetRange().Contains(CurrentTime.Time.GetFrame()))
			{
				FTransform ComponentTransform = Data.SelectedMeshComp->GetComponentTransform();
				FTransform PivotTransform, PivotParentTransform;
				//we want the transform at the start!
				Data.CalcTransform(Data.SelectedSection->GetRange().GetLowerBoundValue(), PivotTransform, PivotParentTransform);
				OutTransform = PivotTransform * ComponentTransform;
				OutParentTransform = PivotParentTransform * ComponentTransform;
				return true;
			}
		}
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	//Get transform at the START of the section since that's the value we will manipulate
	FTransform Transform, ParentTransform;
	if (GetTransformAtFirstSectionStart(Transform, ParentTransform))
	{
		OutMatrix = Transform.ToMatrixNoScale().RemoveTranslation();
		return true;
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(OutMatrix, InData);
}


bool FSkeletalAnimationTrackEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	SelectedRootData.Empty();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		if (HitProxy && HitProxy->IsA(HMovieSceneSkeletalAnimationRootHitProxy::StaticGetType()))
		{
			HMovieSceneSkeletalAnimationRootHitProxy* Proxy = (HMovieSceneSkeletalAnimationRootHitProxy*)HitProxy;

			if (Sequencer.IsValid())
			{
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown() == false &&
					FSlateApplication::Get().GetModifierKeys().IsControlDown() == false)
				{
					SelectedRootData.Empty();
					if (GEditor)
					{
						GEditor->Exec(GetWorld(), TEXT("SELECT NONE"));
					}
				}
				UMovieSceneSkeletalAnimationSection* Section = Proxy->AnimSection.Get();
				USkeletalMeshComponent* Comp = Proxy->SkelMeshComp.Get();

				if (Section && Comp)
				{
					FSelectedRootData RootData(Section, Comp);

					if (FSlateApplication::Get().GetModifierKeys().IsControlDown() == true)
					{
						if (SelectedRootData.Contains(RootData))
						{
							SelectedRootData.Remove(RootData);
						}
						else
						{
							SelectedRootData.Add(RootData);
						}
					}
					else
					{
						if (SelectedRootData.Contains(RootData) == false)
						{
							SelectedRootData.Add(RootData);
						}
					}
				}
			}
		}
	}
	return false;
}

FSelectedRootData::FSelectedRootData(UMovieSceneSkeletalAnimationSection* InSection, USkeletalMeshComponent* InComp) : 
	SelectedSection(InSection), SelectedMeshComp(InComp)
{
	FFrameTime StartTime = SelectedSection->GetRange().GetLowerBoundValue();
}

void FSelectedRootData::CalcTransform(const FFrameTime& FrameTime, FTransform& OutTransform, FTransform& OutParentTransform)
{
	UMovieSceneSkeletalAnimationSection* Section = SelectedSection.Get();
	USkeletalMeshComponent* Comp = SelectedMeshComp.Get();
	OutTransform =  FTransform(FTransform::Identity);
	OutParentTransform = FTransform(FTransform::Identity);
	if (Section && Comp && Comp->GetAnimInstance())
	{
		if (UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>())
		{
			FMemMark Mark(FMemStack::Get());
			FCompactPose OutPose;
			OutPose.ResetToRefPose(Comp->GetAnimInstance()->GetRequiredBones());
			FBlendedCurve OutCurve;
			OutCurve.InitFrom(Comp->GetAnimInstance()->GetRequiredBones());
			UE::Anim::FStackAttributeContainer TempAttributes;
			FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);

			UMovieSceneSkeletalAnimationSection::FRootMotionTransformParam Param;
			Param.FrameRate = MovieScene->GetTickResolution();
			Param.CurrentTime = FrameTime;
			Section->GetRootMotionTransform(AnimationPoseData, Param);
			OutTransform = Param.OutTransform * Param.OutRootStartTransform * Section->PreviousTransform;
			OutParentTransform = Param.OutParentTransform * Param.OutRootStartTransform * Section->PreviousTransform;

		}
	}
}

bool FSkeletalAnimationTrackEditMode::IsSomethingSelected() const
{
	return	SelectedRootData.Num() > 0;
}

void FSkeletalAnimationTrackEditMode::SelectNone()
{
	SelectedRootData.Empty();
	FEdMode::SelectNone();
}

void FSkeletalAnimationTrackEditMode::OnKeySelected(FViewport* Viewport, HMovieSceneSkeletalAnimationRootHitProxy* Proxy)
{
	//something changed but this is no longer needed in 5.3
	return;
}

bool FSkeletalAnimationTrackEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	FVector Drag = InDrag;
	FRotator Rot = InRot;

	const bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);
	const bool bAltDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	const bool bMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);

	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();

	if (bIsTransacting && bMouseButtonDown && !bCtrlDown && !bShiftDown && !bAltDown && CurrentAxis != EAxisList::None)
	{
		const bool bDoRotation = !Rot.IsZero() && (WidgetMode == UE::Widget::WM_Rotate || WidgetMode == UE::Widget::WM_TranslateRotateZ);
		const bool bDoTranslation = !Drag.IsZero() && (WidgetMode == UE::Widget::WM_Translate || WidgetMode == UE::Widget::WM_TranslateRotateZ);

		FTransform CurrentTransform,ParentTransform;
		if ((bDoRotation || bDoTranslation) && GetTransformAtFirstSectionStart(CurrentTransform,ParentTransform))
		{
			for (const FSelectedRootData Data : SelectedRootData)
			{
				if (bDoRotation)
				{
					FQuat CurrentRotation = CurrentTransform.GetRotation();
					CurrentRotation = (InRot.Quaternion() * CurrentRotation);
					CurrentTransform.SetRotation(CurrentRotation);
				}
				if (bDoTranslation)
				{
					FVector CurrentLocation = CurrentTransform.GetLocation();
					CurrentLocation = CurrentLocation + InDrag;
					CurrentTransform.SetLocation(CurrentLocation);
				}

				if (bManipulatorMadeChange == false)
				{
					bManipulatorMadeChange = true;
					GEditor->BeginTransaction(LOCTEXT("MoveRootControlTransaction", "Move Root Control"));
				}
				if (Data.SelectedSection.IsValid() && Data.SelectedMeshComp.IsValid())
				{
					CurrentTransform = CurrentTransform.GetRelativeTransform(ParentTransform);
					
					if (bDoRotation)
					{
						UMovieSceneSkeletalAnimationTrack* Track = Data.SelectedSection.Get()->GetTypedOuter<UMovieSceneSkeletalAnimationTrack>();
						Track->Modify();
						Data.SelectedSection.Get()->Modify();
						Data.SelectedSection.Get()->StartRotationOffset = CurrentTransform.GetRotation().Rotator(); 
						Data.SelectedSection.Get()->GetRootMotionParams()->bRootMotionsDirty = true;
						TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
						if (Sequencer.IsValid())
						{
							Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
						}
					}
					if (bDoTranslation)
					{
						UMovieSceneSkeletalAnimationTrack* Track = Data.SelectedSection.Get()->GetTypedOuter<UMovieSceneSkeletalAnimationTrack>();
						Track->Modify();
						Data.SelectedSection.Get()->Modify();
						Data.SelectedSection.Get()->StartLocationOffset = CurrentTransform.GetLocation();
						Data.SelectedSection.Get()->GetRootMotionParams()->bRootMotionsDirty = true;
						TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
						if (Sequencer.IsValid())
						{
							Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
						}
					}
				}
			}
			return true;
		}
	}
	return false;
}

bool FSkeletalAnimationTrackEditMode::ShouldDrawWidget() const
{
	if (IsSomethingSelected())
	{
		return true;
	}

	return FEdMode::ShouldDrawWidget();
}

bool FSkeletalAnimationTrackEditMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return true;
}


#undef LOCTEXT_NAMESPACE


