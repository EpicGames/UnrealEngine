// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "EditorModeTools.h"
#include "EdMode.h"
#include "ISequencer.h"
#include "Components/SkeletalMeshComponent.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

struct HMovieSceneSkeletalAnimationRootHitProxy;

struct FSelectedRootData
{
	FSelectedRootData(UMovieSceneSkeletalAnimationSection* InSection, USkeletalMeshComponent* InComp);
	bool operator == (const FSelectedRootData& InData) const { return (InData.SelectedSection == SelectedSection && InData.SelectedMeshComp == SelectedMeshComp); }
	void CalcTransform(const FFrameTime& Frametime, FTransform& OutTransform, FTransform& OutParentTransform);

	TWeakObjectPtr<UMovieSceneSkeletalAnimationSection> SelectedSection;
	TWeakObjectPtr<USkeletalMeshComponent> SelectedMeshComp;
};

class FSkeletalAnimationTrackEditMode : public FEdMode
{
public:
	static FName ModeName;

	FSkeletalAnimationTrackEditMode();
	~FSkeletalAnimationTrackEditMode();

	// FEdMode interface
	virtual bool UsesToolkits() const override;
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override; 
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;
	virtual void SelectNone() override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const;
	virtual FVector GetWidgetLocation() const override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
	
	//GCObject 
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	void OnKeySelected(FViewport* Viewport, HMovieSceneSkeletalAnimationRootHitProxy* KeyProxy);

	/** Set The owning sequencer*/
	void SetSequencer(const TSharedPtr<ISequencer>& InSequencer) { WeakSequencer = InSequencer; }

protected:

	bool IsSomethingSelected() const;
	bool GetTransformAtFirstSectionStart(FTransform& OutWorld, FTransform& OutParent) const;
	bool IsRootSelected(UMovieSceneSkeletalAnimationSection* Section) const;
	
protected:
	/** Interrogator that is an gc object */
	UE::MovieScene::FSystemInterrogator InterrogationLinker;

	/** Sequencer that owns me*/
	TWeakPtr<ISequencer> WeakSequencer;

	/** Whether we are in the middle of a transaction */
	bool bIsTransacting;

	/** Whether a manipulator actually made a change when transacting */
	bool bManipulatorMadeChange;

	/** Array of Selected Root Setions*/
	TArray<FSelectedRootData> SelectedRootData;

	/** Cached location of root transform*/
	FTransform RootTransform;
	
};

