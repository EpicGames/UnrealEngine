// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubTrackEditorMode.h"

#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "UnrealClient.h"
#include "EditorViewportClient.h"
#include "IMovieScenePlaybackClient.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "SequencerChannelTraits.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "Tracks/MovieSceneSubTrack.h"

#define LOCTEXT_NAMESPACE "FSubTrackEditorMode"

FName FSubTrackEditorMode::ModeName("EditMode.SubTrackEditMode");

FSubTrackEditorMode::FSubTrackEditorMode()
{
	CachedLocation.Reset();
	PreviewCoordinateSpaceRotation.Reset();
	PreviewLocation.Reset();
}

FSubTrackEditorMode::~FSubTrackEditorMode()
{
}

void FSubTrackEditorMode::Initialize()
{
	CachedLocation.Reset();
	PreviewCoordinateSpaceRotation.Reset();
	PreviewLocation.Reset();
}

bool FSubTrackEditorMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if(!bIsTracking || AreAnyActorsSelected() || (InDrag.IsNearlyZero() && InRot.IsNearlyZero()))
	{
		return false;
	}
	const bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);
	const bool bAltDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	const bool bLeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool bMiddleMouseButtonDown = InViewport->KeyState(EKeys::MiddleMouseButton);
	const bool bRightMouseButtonDown = InViewport->KeyState(EKeys::RightMouseButton);
	const bool bAnyModifiers = bAltDown || bCtrlDown || bShiftDown;
	
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();

	const bool bIndirectManipulate = (bMiddleMouseButtonDown || bRightMouseButtonDown) && CurrentAxis != EAxisList::None;
	const bool bAxisManipulate = bCtrlDown && bLeftMouseButtonDown && CurrentAxis != EAxisList::None;
	const bool bDoManipulate = 
		(bLeftMouseButtonDown && !bAnyModifiers && CurrentAxis != EAxisList::None) ||
		bIndirectManipulate ||
		bAxisManipulate;

	if(bDoManipulate)
	{
		const FTransform TransformOriginFocusedSequence = GetTransformOriginForSequence(GetFocusedSequenceID());

		// Remove parent transform from inputs.
		const FTransform LocalRotation = TransformOriginFocusedSequence * FTransform(InRot) * TransformOriginFocusedSequence.Inverse();
		const FTransform LocalPosition = TransformOriginFocusedSequence * FTransform(InDrag) * TransformOriginFocusedSequence.Inverse();

		// Keep preview space up-to-date.
		PreviewCoordinateSpaceRotation.GetValue() *= FTransform(InRot).ToMatrixNoScale().RemoveTranslation();
		PreviewLocation.GetValue() += InDrag;
		
		OnOriginValueChanged.Broadcast(LocalPosition.GetLocation(), LocalRotation.Rotator());
		return true;
	}
	
	return false;
}

bool FSubTrackEditorMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (HandleBeginTransform())
	{
		return true;
	}
	
	return FEdMode::StartTracking(InViewportClient, InViewport);
}

bool FSubTrackEditorMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (HandleEndTransform())
	{
		return true;
	}
	
	return FEdMode::EndTracking(InViewportClient, InViewport);
}

bool FSubTrackEditorMode::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FSubTrackEditorMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform();
}

bool FSubTrackEditorMode::HandleBeginTransform()
{
	if (const UMovieSceneSubSection* SubSection = GetSectionToEdit())
	{
		bIsTracking = true;
		PreviewCoordinateSpaceRotation = GetFinalTransformOriginForSubSection(SubSection).ToMatrixNoScale().RemoveTranslation();
		if (!PreviewLocation.IsSet())
		{
			PreviewLocation = GetAverageLocationOfBindingsInSubSection(SubSection);
		}
		GEditor->BeginTransaction(LOCTEXT("EditSubseqeunceTransformOriginTransaction", "Edit Subsequence Transform Origin"));
		return true;
	}

	return false;
}

bool FSubTrackEditorMode::HandleEndTransform()
{
	if (bIsTracking)
	{
		bIsTracking = false;
		// Only reset the preview rotation. Resetting the preview location here could interfere with multiple rotation edits in a row.
		// The location would otherwise change after the user let go of the mouse, and would have to move it to the new location to continue rotating.
		PreviewCoordinateSpaceRotation.Reset();
		GEditor->EndTransaction();
		return true;
	}

	return false;
}

bool FSubTrackEditorMode::UsesTransformWidget() const
{
	const UMovieSceneSubSection* SubSection = GetSelectedSection();
	if(!AreAnyActorsSelected() && SubSection)
	{
		return DoesSubSectionHaveTransformOverrides(*SubSection);
	}
	return false;
}

bool FSubTrackEditorMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	const UMovieSceneSubSection* SubSection = GetSelectedSection();
	if(!AreAnyActorsSelected() && SubSection)
	{
		return DoesSubSectionHaveTransformOverrides(*SubSection);
	}
	return false;
}

TOptional<FMovieSceneSequenceID> FSubTrackEditorMode::GetSequenceIDForSubSection(const UMovieSceneSubSection* InSubSection) const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	if(!Sequencer)
	{
		return TOptional<FMovieSceneSequenceID>();
	}
	
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
	
	UMovieSceneCompiledDataManager* CompiledDataManager = EvaluationTemplate.GetCompiledDataManager();
	UMovieSceneSequence* RootSequence = EvaluationTemplate.GetSequence(Sequencer->GetRootTemplateID());
	const FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence);

	const FMovieSceneSequenceHierarchy& Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);

	UE::MovieScene::FSubSequencePath Path;
	const TOptional<FMovieSceneSequenceID> ParentSequenceID = GetFocusedSequenceID();

	if(!ParentSequenceID)
	{
		return TOptional<FMovieSceneSequenceID>();
	}
	
	Path.Reset(ParentSequenceID.GetValue(), &Hierarchy);

	return Path.ResolveChildSequenceID(InSubSection->GetSequenceID());
}

TOptional<FMovieSceneSequenceID> FSubTrackEditorMode::GetFocusedSequenceID() const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	if(!Sequencer)
	{
		return TOptional<FMovieSceneSequenceID>();
	}

	return Sequencer->GetFocusedTemplateID();
}

FTransform FSubTrackEditorMode::GetFinalTransformOriginForSubSection(const UMovieSceneSubSection* InSubSection) const
{
	const TOptional<FMovieSceneSequenceID> ChildSequenceID = GetSequenceIDForSubSection(InSubSection);

	return GetTransformOriginForSequence(ChildSequenceID);
}

FTransform FSubTrackEditorMode::GetTransformOriginForSequence(const TOptional<FMovieSceneSequenceID> InSequenceID) const
{
	FTransform TransformOrigin = FTransform::Identity;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	if(!Sequencer)
	{
		return TransformOrigin;
	}

	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();

	const IMovieScenePlaybackClient*  Client       = Sequencer->GetPlaybackClient();
	const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
	const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

	const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
	if (bHasInterface)
	{
		// Retrieve the current origin
		TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);
	}
	
	const UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if(!EntityLinker || !InSequenceID)
	{
		return TransformOrigin;
	}

	const UMovieSceneTransformOriginSystem* TransformOriginSystem = EntityLinker->FindSystem<UMovieSceneTransformOriginSystem>();

	if(!TransformOriginSystem)
	{
		return TransformOrigin;
	}
	
	const TSparseArray<FTransform>& TransformOrigins = TransformOriginSystem->GetTransformOriginsByInstanceID();
	const TMap<FMovieSceneSequenceID, UE::MovieScene::FInstanceHandle> SequenceIDToInstanceHandle = TransformOriginSystem->GetSequenceIDToInstanceHandle();

	if(SequenceIDToInstanceHandle.Contains(InSequenceID.GetValue()))
	{
		const UE::MovieScene::FInstanceHandle CurrentHandle = SequenceIDToInstanceHandle[InSequenceID.GetValue()];
		if(TransformOrigins.IsValidIndex(CurrentHandle.InstanceID))
		{
			TransformOrigin = TransformOrigins[CurrentHandle.InstanceID];
		}
	}

	return TransformOrigin;
}

bool FSubTrackEditorMode::AreAnyActorsSelected() const
{
	if (Owner)
	{
		if (USelection* SelectedActors = Owner->GetSelectedActors())
		{
			return SelectedActors->Num() > 0;
		}
	}

	return false;
}

FVector FSubTrackEditorMode::GetWidgetLocation() const
{
	if(const UMovieSceneSubSection* SubSection = GetSectionToEdit())
	{
		const FVector NewLocation = PreviewLocation.IsSet() ? PreviewLocation.GetValue() : GetAverageLocationOfBindingsInSubSection(SubSection);
		
		if(!CachedLocation.IsSet() || !NewLocation.Equals(CachedLocation.GetValue()))
		{
			CachedLocation = NewLocation;
			// Invalidate hit proxies, otherwise the hit proxy for the widget can be out of sync, and still at the old widget location
			GEditor->RedrawLevelEditingViewports(true);
		}
		return CachedLocation.GetValue();
	}
	
	return FEdMode::GetWidgetLocation();
}

FVector FSubTrackEditorMode::GetAverageLocationOfBindingsInSubSection(const UMovieSceneSubSection* SubSection) const
{
	FVector TotalPosition = FVector::ZeroVector;
	int32 ActorCount = 0;

	const UMovieSceneSequence* CurrentSequence = SubSection->GetSequence();
	if (!CurrentSequence)
	{
		return TotalPosition;
	}

	const UMovieScene* CurrentMovieScene = CurrentSequence->GetMovieScene();
	if (!CurrentMovieScene)
	{
		return TotalPosition;
	}

	TArray<FMovieSceneBinding> Bindings = CurrentMovieScene->GetBindings();

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	if (!Sequencer)
	{
		return FVector::ZeroVector;
	}

	const FMovieSceneSequenceID FocusedSequenceID = GetFocusedSequenceID().GetValue();
	
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
	UMovieSceneCompiledDataManager* CompiledDataManager = EvaluationTemplate.GetCompiledDataManager();
	UMovieSceneSequence* RootSequence = Sequencer->GetRootMovieSceneSequence();
	

	const FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence);
	const FMovieSceneSequenceHierarchy Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);

	RecursiveAccumulateBindingPositions(SubSection, TotalPosition, ActorCount, &Hierarchy, FocusedSequenceID, FocusedSequenceID, Sequencer);

	if (ActorCount > 0)
	{
		return TotalPosition / ActorCount;
	}
	
	return FVector::ZeroVector;
}

void FSubTrackEditorMode::RecursiveAccumulateBindingPositions(const UMovieSceneSubSection* SubSection, FVector& AccumulatedLocation,
	int32& ActorCount, const FMovieSceneSequenceHierarchy* Hierarchy, const FMovieSceneSequenceID FocusedSequenceID, const FMovieSceneSequenceID ParentSequenceID, const TSharedPtr<ISequencer> Sequencer) const
{
	const UMovieSceneSequence* CurrentSequence = SubSection->GetSequence();
	if (!CurrentSequence)
	{
		return;
	}

	const UMovieScene* CurrentMovieScene = CurrentSequence->GetMovieScene();
	if (!CurrentMovieScene)
	{
		return;
	}
	
	TArray<FMovieSceneBinding> Bindings = CurrentMovieScene->GetBindings();

	UE::MovieScene::FSubSequencePath Path;
	
	Path.Reset(ParentSequenceID, Hierarchy);

	const FMovieSceneSequenceID ResolvedSequenceID = Path.ResolveChildSequenceID(SubSection->GetSequenceID());
	
	for (FMovieSceneBinding& Binding : Bindings)
	{
		FMovieSceneObjectBindingID RelativeBinding = UE::MovieScene::FRelativeObjectBindingID(FocusedSequenceID, ResolvedSequenceID, Binding.GetObjectGuid(), Hierarchy);
	
		for (TWeakObjectPtr<> Object : RelativeBinding.ResolveBoundObjects(FocusedSequenceID, *Sequencer.Get()))
		{
			if (TStrongObjectPtr<UObject> StrongObject = Object.Pin())
			{
				if (const AActor* BoundActor = Cast<AActor>(StrongObject.Get()))
				{
					AccumulatedLocation += BoundActor->GetActorLocation();
					ActorCount++;
				}
			}
		}
	}

	const TArray<UMovieSceneTrack*> Tracks = CurrentMovieScene->GetTracks();

	for (UMovieSceneTrack* Track : Tracks)
	{
		if (Track)
		{
			if (const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
			{
				for (UMovieSceneSection* Section : SubTrack->GetAllSections())
				{
					const UMovieSceneSubSection* ChildSubSection = Cast<UMovieSceneSubSection>(Section);

					RecursiveAccumulateBindingPositions(ChildSubSection, AccumulatedLocation, ActorCount, Hierarchy, FocusedSequenceID, ResolvedSequenceID, Sequencer);
				}
			}
		}
	}
}

bool FSubTrackEditorMode::ShouldDrawWidget() const
{
	if (GetSectionToEdit())
	{
		return true;
	}
	// If the widget is not being drawn, its hit proxies need to be invalidated the next time it is drawn.
	// Resetting the cached location will trigger the invalidation in GetWidgetLocation
	CachedLocation.Reset();
	return false;
}

bool FSubTrackEditorMode::GetPivotForOrbit(FVector& OutPivot) const
{
	return FEdMode::GetPivotForOrbit(OutPivot);
}

bool FSubTrackEditorMode::GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	if (GetModeManager()->GetCoordSystem() != COORD_Local)
	{
		return false;
	}

	if (const UMovieSceneSubSection* SubSection = GetSectionToEdit())
	{
		// While manipulating the gizmo, the preview coordinate space is kept up-to-date directly, since the transform origin data is set by a callback,
		// and can be out of date, which would cause the gizmo to jitter.
		if (PreviewCoordinateSpaceRotation.IsSet())
		{
			OutMatrix = PreviewCoordinateSpaceRotation.GetValue().RemoveTranslation();
			return true;
		}

		OutMatrix = GetFinalTransformOriginForSubSection(SubSection).ToMatrixNoScale().RemoveTranslation();
		return true;
	}
	
	return FEdMode::GetCustomDrawingCoordinateSystem(OutMatrix, InData);
}

bool FSubTrackEditorMode::GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	return FEdMode::GetCustomInputCoordinateSystem(OutMatrix, InData);
}

bool FSubTrackEditorMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	if(IncompatibleEditorModes.Contains(OtherModeID))
	{
		return false;
	}
	return true;
}

void FSubTrackEditorMode::ClearCachedCoordinates()
{
	PreviewLocation.Reset();
	CachedLocation.Reset();
	PreviewCoordinateSpaceRotation.Reset();
}

bool FSubTrackEditorMode::DoesSubSectionHaveTransformOverrides(const UMovieSceneSubSection& SubSection)
{
	if (SubSection.IsActive())
	{
		const EMovieSceneTransformChannel SectionTransformChannels = SubSection.GetMask().GetChannels();

		return EnumHasAnyFlags(SectionTransformChannels, EMovieSceneTransformChannel::Translation) || EnumHasAnyFlags(SectionTransformChannels, EMovieSceneTransformChannel::Rotation);
	}
	
	return false;
}

UMovieSceneSubSection* FSubTrackEditorMode::GetSelectedSection() const
{
	const TSharedPtr<ISequencer> PinnedSequencer = WeakSequencer.Pin();
	UMovieSceneSubSection* SelectedSection = nullptr;
	if(!PinnedSequencer.IsValid())
	{
		return SelectedSection;
	}

	TArray<UMovieSceneSection*> SelectedSections;
	PinnedSequencer->GetSelectedSections(SelectedSections);
	
	for(UMovieSceneSection* Section : SelectedSections)
	{
		if(UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			// Mirror behavior when multiple actors are selected in the level editor, and pick the last selected item that can still be edited.
			if(SubSection->IsTransformOriginEditable())
			{
				SelectedSection = SubSection;
			}
		}
	}

	if(SelectedSection)
	{
		return SelectedSection;
	}

	TArray<UMovieSceneTrack*> SelectedTracks;
	PinnedSequencer->GetSelectedTracks(SelectedTracks);
	for(UMovieSceneTrack* Track : SelectedTracks)
	{
		// Similarly to section selection, pick the last selected track.
		if(const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			if(SubTrack->GetSectionToKey())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(SubTrack->GetSectionToKey());
				if(SubSection && SubSection->IsTransformOriginEditable())
				{
					SelectedSection = SubSection;
				}
			}
			else if(SubTrack->GetAllSections().Num())
			{
				// Since the first section is the section that will be keyed by default, select the first section from the track.
				for(UMovieSceneSection* Section : SubTrack->FindAllSections(PinnedSequencer.Get()->GetLocalTime().Time.FrameNumber))
				{
					if(UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
					{
						if(SubSection->IsTransformOriginEditable())
						{
							SelectedSection = SubSection;
							break;
						}
					}
				}
			}
		}
			
	}
	return SelectedSection;
}

UMovieSceneSubSection* FSubTrackEditorMode::GetSectionToEdit() const
{
	UMovieSceneSubSection* SubSectionToReturn = GetSelectedSection();
	if (!AreAnyActorsSelected() && SubSectionToReturn && DoesSubSectionHaveTransformOverrides(*SubSectionToReturn))
	{
		return SubSectionToReturn;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE