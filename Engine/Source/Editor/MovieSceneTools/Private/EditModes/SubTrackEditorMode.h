// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "ISequencer.h"

class FSubTrackEditorMode : public FEdMode
{
public:
	static FName ModeName;
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOriginValueChanged, FVector, FRotator)
	
	FSubTrackEditorMode();
	virtual ~FSubTrackEditorMode() override;

	// FEdMode interface
	virtual void Initialize() override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	void SetSequencer(const TSharedPtr<ISequencer>& InSequencer) { WeakSequencer = InSequencer; }
	FOnOriginValueChanged& GetOnOriginValueChanged() { return OnOriginValueChanged; };

	void ClearCachedCoordinates();

	// Start IGizmoEdModeInterface overrides
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	// End IGizmoEdModeInterface overrides

private:
	// Returns true if there are any sub tracks with active transform origin overrides.
	static bool DoesSubSectionHaveTransformOverrides(const UMovieSceneSubSection& SubSection);

	// Gets the sequence ID from the context of the subsection in the current hierarchy.
	TOptional<FMovieSceneSequenceID> GetSequenceIDForSubSection(const UMovieSceneSubSection* InSubSection) const;
	// Gets the sequence ID of the currently focused sequence.
	TOptional<FMovieSceneSequenceID> GetFocusedSequenceID() const;
	// Gets the transform origin of the provided section, after all parent transforms have been applied.
	FTransform GetFinalTransformOriginForSubSection(const UMovieSceneSubSection* SubSection) const;
	// Gets the transform origin corresponding to the sequence in the current hierarchy matching the provided sequence ID.
	FTransform GetTransformOriginForSequence(TOptional<FMovieSceneSequenceID> InSequenceID) const;
	// returns true if any actors are selected in the level editor. This is used to prevent this editor mode from being active when actors are selected.
	bool AreAnyActorsSelected() const;
	/** Gets the average location of the objects in a subsection. */
	FVector GetAverageLocationOfBindingsInSubSection(const UMovieSceneSubSection* SubSection) const;
	/** Accumulate the positions and counts of all bound objects in and descended from the given SubSection */
	void RecursiveAccumulateBindingPositions(const UMovieSceneSubSection* SubSection, FVector& AccumulatedLocation,
	int32& ActorCount, const FMovieSceneSequenceHierarchy* Hierarchy, const FMovieSceneSequenceID FocusedSequenceID, const FMovieSceneSequenceID ParentSequenceID, const TSharedPtr<ISequencer> Sequencer) const;

	/** Returns the currently selected section. */
	UMovieSceneSubSection* GetSelectedSection() const;
	/** Returns the selected section, if its origin overrides can be edited, and there are no selected actors. */
	UMovieSceneSubSection* GetSectionToEdit() const;

	/** Sequencer that owns this editor mode */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Delegate called when the origin is modified from the editor gizmo */
	FOnOriginValueChanged OnOriginValueChanged;

	/** Used to tell if the gizmo has moved, and if the editor hit proxies need to be invalidated as a result */
	mutable TOptional<FVector> CachedLocation;

	/** Used to ensure that we mirror the behavior in StartTracking if the selection were to change mid-drag. */
	bool bIsTracking = false;

	/** Caches the transform space to use for editing with the gizmo.
	 *  Updates to the channel data aren't reflected in time for the UI, so keeping the preview space up-to-date prevents the gizmo from flickering. */
	TOptional<FMatrix> PreviewCoordinateSpaceRotation;
	/** Caches the location to use for editing with the gizmo.
	 * When rotating, the average location of the actors in a subsequence can change, so instead of querying it every frame,
	 * the value is cached when an edit begins, and is only updated if the gizmo is dragged. */
	TOptional<FVector> PreviewLocation;

	const TArray<FName> IncompatibleEditorModes = TArray<FName>({ "EditMode.ControlRig", "EM_Landscape" });

	/** Manages the start and end of a transform action in the viewport */
	bool HandleBeginTransform();
	bool HandleEndTransform();
};
