// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLExport.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "MovieSceneTimeHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/MetaData.h"
#include "Logging/TokenizedMessage.h"
#include "Sound/SoundWave.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "FCPXMLExporter"

DEFINE_LOG_CATEGORY_STATIC(LogFCPXMLExporter, Log, All);

FFCPXMLExportVisitor::FFCPXMLExportVisitor(FString InSaveFilename, TSharedRef<FMovieSceneExportData> InExportData, TSharedRef<FMovieSceneTranslatorContext> InExportContext) :
	  FFCPXMLNodeVisitor()
	, ExportData(InExportData)
	, ExportContext(InExportContext)
	, SaveFilePath(InSaveFilename)
{
	SaveFilePath = FPaths::GetPath(InSaveFilename);
	if (FPaths::IsRelative(SaveFilePath))
	{
		SaveFilePath = FPaths::ConvertRelativePathToFull(SaveFilePath);
	}

	SequenceId = 0;
	MasterClipId = 0;
	ClipItemId = 0;
	FileId = 0;
}

FFCPXMLExportVisitor::~FFCPXMLExportVisitor() {}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode)
{
	return InBasicNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode)
{
	float EditTime = 0.f;

	// Construct the FCP 7 XML structure from Sequencer class objects
	if (InXmemlNode->GetChildCount() == 0)
	{
		if (!ConstructProjectNode(InXmemlNode)) 
		{
			return false;
		}
	}

	/** @todo - MERGE METADATA

	   Merging the newly exported XML structure with pre-existing
	   XML metadata may be implemented here. The traversal would proceed
	   through the new XML structure, referring back to the metadata 
	   XML structure to incorporate any missing or desired attributes or
	   elements. Alternatively, the traversal might be invoked directly on
	   the metadata referring back to and modifying the new XML structure. 

	*/

	return InXmemlNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode)
{
	return InSequenceNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode)
{
	return InVideoNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode)
{
	return InAudioNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode)
{
	return InTrackNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode)
{
	return InClipNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode)
{
	return InClipItemNode->VisitChildren(*this);
}

bool FFCPXMLExportVisitor::VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode)
{
	return InFileNode->VisitChildren(*this);
}


/** Creates project node. */
bool FFCPXMLExportVisitor::ConstructProjectNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	TSharedRef<FFCPXMLNode> ProjectNode = InParentNode->CreateChildNode(TEXT("project"));

	TSharedRef<FFCPXMLNode> NameNode = ProjectNode->CreateChildNode(TEXT("name"));
	NameNode->SetContent(ExportData->MovieSceneData->Name + TEXT("Project"));

	TSharedRef<FFCPXMLNode> ChildrenNode = ProjectNode->CreateChildNode(TEXT("children"));

	if (!ConstructMasterVideoClipNodes(ChildrenNode))
	{
		return false;
	}

	if (!ConstructMasterAudioClipNodes(ChildrenNode))
	{
		return false;
	}

	if (!ConstructSequenceNode(ChildrenNode))
	{
		return false;
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructMasterVideoClipNodes(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid() || !ExportData->MovieSceneData.IsValid() || !ExportData->MovieSceneData->CinematicData.IsValid())
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportCinematicData> CinematicData = ExportData->MovieSceneData->CinematicData;
	if (!CinematicData.IsValid())
	{
		return false;
	}

	for (TSharedPtr<FMovieSceneExportCinematicSectionData> CinematicSection : CinematicData->CinematicSections)
	{
		if (!CinematicSection.IsValid())
		{
			continue;
		}

		// skip sections outside of playback range
		if (CinematicSection->bWithinPlaybackRange == false)
		{
			CinematicSection->bEnabled = false;
			continue;
		}

		if (!ConstructMasterClipNode(InParentNode, CinematicSection, CinematicData))
		{
			return false;
		}
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructMasterAudioClipNodes(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid() || !ExportData->MovieSceneData.IsValid())
	{
		return false;
	}

	for (TSharedPtr<FMovieSceneExportAudioData> AudioData : ExportData->MovieSceneData->AudioData)
	{
		if (!AudioData.IsValid())
		{
			return false;
		}

		for (TSharedPtr<FMovieSceneExportAudioTrackData> AudioTrack : AudioData->AudioTracks)
		{
			if (!AudioTrack.IsValid())
			{
				return false;
			}

			for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : AudioTrack->AudioSections)
			{
				bool bMasterClipExists = false;
				FString MasterClipName;
				HasMasterClipIdName(AudioSection, MasterClipName, bMasterClipExists);

				if (!bMasterClipExists)
				{
					ConstructMasterClipNode(InParentNode, AudioSection, AudioData);
				}
			}
		}
	}
	return true;
}

bool FFCPXMLExportVisitor::ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData)
{
	if (!InCinematicSectionData.IsValid())
	{
		return false;
	}

	int32 Duration{ 0 };
	int32 StartFrame{ 0 };
	int32 EndFrame{ 0 };
	int32 InFrame{ 0 };
	int32 OutFrame{ 0 };
	FString SectionName = InCinematicSectionData->DisplayName;
	GetCinematicSectionFrames(InCinematicSectionData, Duration, StartFrame, EndFrame, InFrame, OutFrame);

	/** Construct a master clip id name based on the cinematic section and id */
	FString MasterClipName{ TEXT("") };
	GetMasterClipIdName(InCinematicSectionData, MasterClipName);

	TSharedRef<FFCPXMLNode> ClipNode = InParentNode->CreateChildNode(TEXT("clip"));
	ClipNode->AddAttribute(TEXT("id"), MasterClipName);

	// @todo add to file's masterclip and refidmap HERE

	ClipNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipName);
	ClipNode->CreateChildNode(TEXT("ismasterclip"))->SetContent(true);
	ClipNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);

	if (!ConstructRateNode(ClipNode))
	{
		return false;
	}

	ClipNode->CreateChildNode(TEXT("in"))->SetContent(InFrame);
	ClipNode->CreateChildNode(TEXT("out"))->SetContent(OutFrame);
	ClipNode->CreateChildNode(TEXT("name"))->SetContent(SectionName);
	TSharedRef<FFCPXMLNode> MediaNode = ClipNode->CreateChildNode(TEXT("media"));
	TSharedRef<FFCPXMLNode> VideoNode = MediaNode->CreateChildNode(TEXT("video"));
	TSharedRef<FFCPXMLNode> TrackNode = VideoNode->CreateChildNode(TEXT("track"));

	if (!ConstructVideoClipItemNode(TrackNode, InCinematicSectionData, InCinematicData, true))
	{
		return false;
	}

	if (!ConstructLoggingInfoNode(ClipNode, InCinematicSectionData->MovieSceneSection))
	{
		return false;
	}

	if (!ConstructColorInfoNode(ClipNode))
	{
		return false;
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, const TSharedPtr<FMovieSceneExportAudioData> InAudioData)
{
	if (!InAudioSectionData.IsValid())
	{
		return false;
	}

	bool bIsStereo = (InAudioSectionData->NumChannels == 2);

	int32 Duration{ 0 };
	int32 StartFrame{ 0 };
	int32 EndFrame{ 0 };
	int32 InFrame{ 0 };
	int32 OutFrame{ 0 };
	FString SectionName = InAudioSectionData->DisplayName;
	GetAudioSectionFrames(InAudioSectionData, Duration, StartFrame, EndFrame, InFrame, OutFrame);

	/** Construct a master clip id name based on the audio section and id */
	FString MasterClipName{ TEXT("") };
	GetMasterClipIdName(InAudioSectionData, MasterClipName);

	TSharedRef<FFCPXMLNode> ClipNode = InParentNode->CreateChildNode(TEXT("clip"));
	ClipNode->AddAttribute(TEXT("id"), MasterClipName);
	ClipNode->AddAttribute(TEXT("explodedTracks"), TEXT("true"));

	// @todo add to file's masterclip and refidmap HERE

	ClipNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipName);
	ClipNode->CreateChildNode(TEXT("ismasterclip"))->SetContent(true);
	ClipNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);

	if (!ConstructRateNode(ClipNode))
	{
		return false;
	}

	ClipNode->CreateChildNode(TEXT("in"))->SetContent(InFrame);
	ClipNode->CreateChildNode(TEXT("out"))->SetContent(OutFrame);
	ClipNode->CreateChildNode(TEXT("name"))->SetContent(SectionName);
	TSharedRef<FFCPXMLNode> MediaNode = ClipNode->CreateChildNode(TEXT("media"));
	TSharedRef<FFCPXMLNode> AudioNode = MediaNode->CreateChildNode(TEXT("audio"));
	TSharedRef<FFCPXMLNode> TrackNode = AudioNode->CreateChildNode(TEXT("track"));

	FString ClipItemIdName1{ TEXT("") };
	FString ClipItemIdName2{ TEXT("") };
	GetNextClipItemIdName(ClipItemIdName1);
	if (bIsStereo)
	{
		GetNextClipItemIdName(ClipItemIdName2);
	}
	if (!ConstructAudioClipItemNode(TrackNode, InAudioSectionData, InAudioData, 1, true, ClipItemIdName1, ClipItemIdName2, 1, 1, 1, 2))
	{
		return false;
	}

	// handle stereo master clip
	if (bIsStereo)
	{
		TrackNode = AudioNode->CreateChildNode(TEXT("track"));

		if (!ConstructAudioClipItemNode(TrackNode, InAudioSectionData, InAudioData, 2, true, ClipItemIdName1, ClipItemIdName2, 1, 1, 1, 2))
		{
			return false;
		}
	}

	if (!ConstructLoggingInfoNode(ClipNode, InAudioSectionData))
	{
		return false;
	}


	return true;
}


/** Creates logginginfo node. */
bool FFCPXMLExportVisitor::ConstructLoggingInfoNode(TSharedRef<FFCPXMLNode> InParentNode, const UMovieSceneSection* InMovieSceneSection)
{
	if (!IsValid(InMovieSceneSection))
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> LoggingInfoNode = InParentNode->CreateChildNode(TEXT("logginginfo"));
	ConstructLoggingInfoElements(LoggingInfoNode, InMovieSceneSection);

	TSharedPtr<FFCPXMLNode> LogNoteNode = LoggingInfoNode->GetChildNode(TEXT("lognote"), ENodeInherit::NoInherit, ENodeReference::NoReferences);
	if (!LogNoteNode.IsValid())
	{
		LogNoteNode = LoggingInfoNode->CreateChildNode(TEXT("lognote"));
	}

	FString Metadata{ TEXT("") };
	const UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(InMovieSceneSection);
	if (ShotSection == nullptr)
	{
		return false;
	}

	if (!CreateCinematicSectionMetadata(ShotSection, Metadata))
	{
		return false;
	}
	LogNoteNode->SetContent(Metadata);

	return true;
}

/** Creates logginginfo node. */
bool FFCPXMLExportVisitor::ConstructLoggingInfoNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InSectionData)
{
	if (!ExportData->MovieSceneData.IsValid() || !InSectionData.IsValid() || InSectionData->MovieSceneSection == nullptr)
	{
		return false;
	}
	const UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(InSectionData->MovieSceneSection);
	if (AudioSection == nullptr)
	{
		return false;
	}
	USoundBase *Sound = AudioSection->GetSound();
	if (Sound == nullptr)
	{
		// skip logging 
		return true;
	}
	USoundWave* SoundWave = Cast<USoundWave>(Sound);
	if (SoundWave == nullptr)
	{
		// skip logging 
		return true;
	}

	TSharedRef<FFCPXMLNode> LoggingInfoNode = InParentNode->CreateChildNode(TEXT("logginginfo"));
	ConstructLoggingInfoElements(LoggingInfoNode, InSectionData->MovieSceneSection);

	TSharedPtr<FFCPXMLNode> LogNoteNode = LoggingInfoNode->GetChildNode(TEXT("lognote"), ENodeInherit::NoInherit, ENodeReference::NoReferences);
	if (!LogNoteNode.IsValid())
	{
		LogNoteNode = LoggingInfoNode->CreateChildNode(TEXT("lognote"));
	}

	TArray <TSharedPtr<FMovieSceneExportAudioSectionData> > AudioSectionsData;
	ExportData->FindAudioSections(SoundWave->GetPathName(), AudioSectionsData);

	TArray< const UMovieSceneAudioSection*> AudioSections;
	for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSectionData : AudioSectionsData)
	{
		if (AudioSectionData.IsValid() && AudioSectionData->MovieSceneSection != nullptr)
		{
			const UMovieSceneAudioSection* Section = Cast<UMovieSceneAudioSection>(AudioSectionData->MovieSceneSection);
			if (Section != nullptr && Section->GetSound() != nullptr)
			{
				if (Section->GetSound()->GetPathName() == SoundWave->GetPathName())
				{
					AudioSections.Add(Section);
				}
			}
		}
	}

	FString Metadata{ TEXT("") };
	if (!CreateSoundWaveMetadata(SoundWave, AudioSections, Metadata))
	{
		return false;
	}
	LogNoteNode->SetContent(Metadata);

	return true;
}

/** Creates logginginfo elements. */
void FFCPXMLExportVisitor::ConstructLoggingInfoElements(TSharedRef<FFCPXMLNode> InLoggingInfoNode, const UObject* InObject)
{
	TSharedRef<FFCPXMLNode> DescriptionNode = InLoggingInfoNode->CreateChildNode(TEXT("description"));
	TSharedRef<FFCPXMLNode> SceneNode = InLoggingInfoNode->CreateChildNode(TEXT("scene"));
	TSharedRef<FFCPXMLNode> ShotTakeNode = InLoggingInfoNode->CreateChildNode(TEXT("shottake"));
	TSharedRef<FFCPXMLNode> GoodNode = InLoggingInfoNode->CreateChildNode(TEXT("good"));
	TSharedRef<FFCPXMLNode> OriginalVideoNode = InLoggingInfoNode->CreateChildNode(TEXT("originalvideofilename"));
	TSharedRef<FFCPXMLNode> OriginalAudioNode = InLoggingInfoNode->CreateChildNode(TEXT("originalaudiofilename"));

	if (InObject != nullptr)
	{
		SetLoggingInfoElementValue(DescriptionNode, InObject, TEXT("description"));
		SetLoggingInfoElementValue(SceneNode, InObject, TEXT("scene"));
		SetLoggingInfoElementValue(ShotTakeNode, InObject, TEXT("shottake"));
		SetLoggingInfoElementValue(GoodNode, InObject, TEXT("good"));
		SetLoggingInfoElementValue(OriginalVideoNode, InObject, TEXT("originalvideofilename"));
		SetLoggingInfoElementValue(OriginalAudioNode, InObject, TEXT("originalaudiofilename"));
	}
}

/** Set logginginfo element value. */
void FFCPXMLExportVisitor::SetLoggingInfoElementValue(TSharedPtr<FFCPXMLNode> InNode, const UObject* InObject, const FString& InElement)
{
	if (InNode.IsValid() && InObject != nullptr)
	{
		UPackage* Package = InObject->GetOutermost();
		check(Package);

		FMetaData& MetaData = Package->GetMetaData();

		if (MetaData.HasValue(InObject, *InElement))
		{
			FString Value = MetaData.GetValue(InObject, *InElement);
			InNode->SetContent(Value);
		}
	}	
}

/** Creates colorinfo node. */
bool FFCPXMLExportVisitor::ConstructColorInfoNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	TSharedPtr<FFCPXMLNode> ColorInfoNode = InParentNode->CreateChildNode(TEXT("colorinfo"));
	ColorInfoNode->CreateChildNode(TEXT("lut"));
	ColorInfoNode->CreateChildNode(TEXT("lut1"));
	ColorInfoNode->CreateChildNode(TEXT("asc_sop"));
	ColorInfoNode->CreateChildNode(TEXT("asc_sat"));
	ColorInfoNode->CreateChildNode(TEXT("lut2"));

	return true;
}

bool FFCPXMLExportVisitor::ConstructSequenceNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid() || !ExportData->MovieSceneData.IsValid()) 
	{ 
		return false; 
	}

	TSharedRef<FFCPXMLNode> SequenceNode = InParentNode->CreateChildNode("sequence");

	// attributes
	SequenceNode->AddAttribute(TEXT("id"), FString::Printf(TEXT("sequence-%d"), ++SequenceId));

	// required elements
	TSharedRef<FFCPXMLNode> DurationNode = SequenceNode->CreateChildNode(TEXT("duration"));
	DurationNode->SetContent(ExportData->MovieSceneData->Duration);

	if (!ConstructRateNode(SequenceNode)) 
	{ 
		return false; 
	}

	TSharedRef<FFCPXMLNode> NameNode = SequenceNode->CreateChildNode(TEXT("name"));
	NameNode->SetContent(ExportData->MovieSceneData->Name);

	TSharedRef<FFCPXMLNode> MediaNode = SequenceNode->CreateChildNode(TEXT("media"));

	if (!ConstructVideoNode(MediaNode)) 
	{ 
		return false; 
	}

	if (!ConstructAudioNode(MediaNode))
	{
		return false;
	}

	if (!ConstructTimecodeNode(SequenceNode))
	{
		return false;
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructVideoNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid() || !ExportData->MovieSceneData.IsValid() || !ExportData->MovieSceneData->CinematicData.IsValid())
	{ 
		return false; 
	}

	TSharedPtr<FMovieSceneExportCinematicData> CinematicData = ExportData->MovieSceneData->CinematicData;
	if (!CinematicData.IsValid())
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> VideoNode = InParentNode->CreateChildNode(TEXT("video"));

	TSharedRef<FFCPXMLNode> FormatNode = VideoNode->CreateChildNode(TEXT("format"));

	if (!ConstructVideoSampleCharacteristicsNode(FormatNode, ExportData->GetResX(), ExportData->GetResY()))
	{
		return false;
	}
 
	// Add in reverse order
	for (int32 RowIndex = CinematicData->CinematicTracks.Num()-1; RowIndex >= 0; --RowIndex)
	{
		if (!ConstructVideoTrackNode(VideoNode, CinematicData->CinematicTracks[RowIndex], CinematicData))
		{ 
			return false;
		}
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructAudioNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid())
	{ 
		return false; 
	}

	TSharedRef<FFCPXMLNode> AudioNode = InParentNode->CreateChildNode(TEXT("audio"));

	int32 NumChannels = 1;
	for (TSharedPtr<FMovieSceneExportAudioData> AudioData : ExportData->MovieSceneData->AudioData)
	{
		if (HasStereoAudioSections(AudioData->AudioSections))
		{
			NumChannels = 2;
			break;
		}
	}
	AudioNode->CreateChildNode(TEXT("numOutputChannels"))->SetContent(NumChannels);

	TSharedRef<FFCPXMLNode> FormatNode = AudioNode->CreateChildNode(TEXT("format"));

	if (!ConstructAudioSampleCharacteristicsNode(FormatNode, ExportData->GetDefaultAudioDepth(), ExportData->GetDefaultAudioSampleRate()))
	{
		return false;
	}

	int32 Downmix = 0;
	TSharedRef<FFCPXMLNode> OutputsNode = AudioNode->CreateChildNode(TEXT("outputs"));

	TSharedRef<FFCPXMLNode> GroupNode = OutputsNode->CreateChildNode(TEXT("group"));
	GroupNode->CreateChildNode(TEXT("index"))->SetContent(1);
	GroupNode->CreateChildNode(TEXT("numchannels"))->SetContent(1);
	GroupNode->CreateChildNode(TEXT("downmix"))->SetContent(Downmix);
	TSharedRef<FFCPXMLNode> ChannelNode = GroupNode->CreateChildNode(TEXT("channel"));
	ChannelNode->CreateChildNode(TEXT("index"))->SetContent(1);

	if (NumChannels == 2)
	{
		GroupNode = OutputsNode->CreateChildNode(TEXT("group"));
		GroupNode->CreateChildNode(TEXT("index"))->SetContent(2);
		GroupNode->CreateChildNode(TEXT("numchannels"))->SetContent(1);
		GroupNode->CreateChildNode(TEXT("downmix"))->SetContent(Downmix);
		ChannelNode = GroupNode->CreateChildNode(TEXT("channel"));
		ChannelNode->CreateChildNode(TEXT("index"))->SetContent(2);
	}

	uint32 TrackIndex = 1;

	// Add in reverse order
	for (int32 RowIndex = ExportData->MovieSceneData->AudioData.Num() - 1; RowIndex >= 0; --RowIndex)
	{
		if (!ExportData->MovieSceneData->AudioData[RowIndex].IsValid())
		{
			return false;
		}

		for (TSharedPtr<FMovieSceneExportAudioTrackData> AudioTrack : ExportData->MovieSceneData->AudioData[RowIndex]->AudioTracks)
		{
			uint32 OutNumTracks{ 0 };
			if (!ConstructAudioTrackNode(AudioNode, AudioTrack, ExportData->MovieSceneData->AudioData[RowIndex], TrackIndex, OutNumTracks))
			{
				return false;
			}
			TrackIndex += OutNumTracks;
		}
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructVideoTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicTrackData> InCinematicTrackData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData)
{
	if (!ExportData->IsExportDataValid() || !InCinematicTrackData.IsValid())
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> TrackNode = InParentNode->CreateChildNode(TEXT("track"));

	for (TSharedPtr<FMovieSceneExportCinematicSectionData> CinematicSection : InCinematicTrackData->CinematicSections)
	{
		// skip disabled sections
		if (!CinematicSection.IsValid() || !CinematicSection->bEnabled)
		{
			continue;
		}
		
		if (!ConstructVideoClipItemNode(TrackNode, CinematicSection, InCinematicData, false))
		{
			return false;
		}
	}

	TSharedRef<FFCPXMLNode> EnabledNode = TrackNode->CreateChildNode(TEXT("enabled"));
	EnabledNode->SetContent(true);

	TSharedRef<FFCPXMLNode> LockedNode = TrackNode->CreateChildNode(TEXT("locked"));
	LockedNode->SetContent(false);

	return true;
}

bool FFCPXMLExportVisitor::HasStereoAudioSections(const TArray<TSharedPtr<FMovieSceneExportAudioSectionData>>& InAudioSections) const
{
	for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : InAudioSections)
	{
		if (AudioSection.IsValid() && AudioSection->NumChannels == 2)
		{
			return true;
		}
	}
	return false;
}

bool FFCPXMLExportVisitor::ConstructAudioTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioTrackData> InAudioTrackData, const TSharedPtr<FMovieSceneExportAudioData> InAudioData, uint32 InTrackIndex, uint32 OutNumTracks)
{
	if (!ExportData->IsExportDataValid() || !InAudioTrackData.IsValid())
	{
		return false;
	}

	bool bTrackHasStereoClips = HasStereoAudioSections(InAudioTrackData->AudioSections);
	int32 TrackIndex1{ 0 };
	int32 TrackIndex2{ 0 };

	if (bTrackHasStereoClips) 
	{
		OutNumTracks = 2;
		TrackIndex1 = InTrackIndex;
		TrackIndex2 = InTrackIndex + 1;
	}
	else
	{
		OutNumTracks = 1;
		TrackIndex1 = InTrackIndex;
		TrackIndex2 = InTrackIndex;
	}

	// Generate all clipitem names for this track so that linked clipitems can be associated
	FString ClipItemIdName{ TEXT("") };
	TArray<FString> ClipItem1;
	TArray<FString> ClipItem2;
	TArray<int32> ClipIndex1;
	TArray<int32> ClipIndex2;

	int32 Index = 0;
	for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : InAudioTrackData->AudioSections)
	{
		if (AudioSection->NumChannels < 1)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning,
				FText::Format(LOCTEXT("FCPXMLAudioChannelsInvalidWarning", "FCP XML export only supports mono or stereo audio. Skipping audio section '{0}' which an invalid number of channels: '{1}'."),
					FText::FromString(AudioSection->DisplayName),
					FText::FromString(FString::FromInt(AudioSection->NumChannels))));

			continue;
		}
		else if (AudioSection->NumChannels > 2)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning,
				FText::Format(LOCTEXT("FCPXMLAudioChannelsUnsupportedWarning", "FCP XML export only supports mono or stereo audio. Skipping audio section '{0}' which has '{1}' channels."),
					FText::FromString(AudioSection->DisplayName),
					FText::FromString(FString::FromInt(AudioSection->NumChannels))));

			continue;
		}

		GetNextClipItemIdName(ClipItemIdName);
		ClipItem1.Add(ClipItemIdName);
		ClipIndex1.Add(++Index);
	}

	Index = 0;
	for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : InAudioTrackData->AudioSections)
	{
		if (AudioSection->NumChannels == 1)
		{
			ClipItem2.Add(FString(TEXT("")));
			ClipIndex2.Add(-1);
		}
		else if (AudioSection->NumChannels == 2)
		{
			GetNextClipItemIdName(ClipItemIdName);
			ClipItem2.Add(ClipItemIdName);
			ClipIndex2.Add(++Index);
		}
	}

	// construct track 1
	TSharedRef<FFCPXMLNode> TrackNode = InParentNode->CreateChildNode(TEXT("track"));
	TrackNode->AddAttribute(TEXT("currentExplodedTrackIndex"), TEXT("0"));
	TrackNode->AddAttribute(TEXT("totalExplodedTrackCount"), bTrackHasStereoClips ? TEXT("2") : TEXT("1"));
	TrackNode->AddAttribute(TEXT("premiereTrackType"), bTrackHasStereoClips ? TEXT("Stereo") : TEXT("Mono"));

	Index = 0;
	for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : InAudioTrackData->AudioSections)
	{
		if (AudioSection->NumChannels < 1 || AudioSection->NumChannels > 2)
		{
			continue;
		}

		if (!ConstructAudioClipItemNode(TrackNode, AudioSection, InAudioData, 1, false, ClipItem1[Index], ClipItem2[Index], ClipIndex1[Index], ClipIndex2[Index], TrackIndex1, TrackIndex2))
		{
			return false;
		}
		Index++;
	}


	TSharedRef<FFCPXMLNode> EnabledNode = TrackNode->CreateChildNode(TEXT("enabled"));
	EnabledNode->SetContent(true);

	TSharedRef<FFCPXMLNode> LockedNode = TrackNode->CreateChildNode(TEXT("locked"));
	LockedNode->SetContent(false);

	// construct track 2, if stereo clipitems exist
	if (bTrackHasStereoClips)
	{
		TrackNode = InParentNode->CreateChildNode(TEXT("track"));
		TrackNode->AddAttribute(TEXT("currentExplodedTrackIndex"), TEXT("1"));
		TrackNode->AddAttribute(TEXT("totalExplodedTrackCount"), TEXT("2"));
		TrackNode->AddAttribute(TEXT("premiereTrackType"), TEXT("Stereo"));

		Index = 0;
		for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSection : InAudioTrackData->AudioSections)
		{
			if (AudioSection->NumChannels < 1 || AudioSection->NumChannels > 2)
			{
				continue;
			}

			if (AudioSection->NumChannels == 2)
			{
				if (!ConstructAudioClipItemNode(TrackNode, AudioSection, InAudioData, 2, false, ClipItem1[Index], ClipItem2[Index], ClipIndex1[Index], ClipIndex2[Index], TrackIndex1, TrackIndex2))
				{
					return false;
				}
			}
			Index++;

		}

		EnabledNode = TrackNode->CreateChildNode(TEXT("enabled"));
		EnabledNode->SetContent(true);

		LockedNode = TrackNode->CreateChildNode(TEXT("locked"));
		LockedNode->SetContent(false);
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructVideoClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, bool bInMasterClip)
{
	if (!ExportData->IsExportDataValid() || !InCinematicSectionData.IsValid())
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> ClipItemNode = InParentNode->CreateChildNode(TEXT("clipitem"));

	int32 Duration{ 0 };
	int32 In{ 0 };
	int32 Out{ 0 };
	int32 Start{ 0 };
	int32 End{ 0 };
	GetCinematicSectionFrames(InCinematicSectionData, Duration, Start, End, In, Out);

	FString MasterClipIdName = TEXT("");
	GetMasterClipIdName(InCinematicSectionData, MasterClipIdName);

	FString ClipItemIdName{ TEXT("") };
	GetNextClipItemIdName(ClipItemIdName);

	// attributes
	ClipItemNode->AddAttribute(TEXT("id"), ClipItemIdName);

	// elements
	ClipItemNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipIdName);
	ClipItemNode->CreateChildNode(TEXT("ismasterclip"))->SetContent(bInMasterClip);
	ClipItemNode->CreateChildNode(TEXT("name"))->SetContent(InCinematicSectionData->DisplayName);
	ClipItemNode->CreateChildNode(TEXT("enabled"))->SetContent(true);
	ClipItemNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);

	if (!ConstructRateNode(ClipItemNode))
	{
		return false;
	}

	if (!bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("start"))->SetContent(Start);
		ClipItemNode->CreateChildNode(TEXT("end"))->SetContent(End);
	}

	ClipItemNode->CreateChildNode(TEXT("in"))->SetContent(In);
	ClipItemNode->CreateChildNode(TEXT("out"))->SetContent(Out);

	if (bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("anamorphic"))->SetContent(false);
		ClipItemNode->CreateChildNode(TEXT("pixelaspectratio"))->SetContent(FString(TEXT("square")));
		ClipItemNode->CreateChildNode(TEXT("fielddominance"))->SetContent(FString(TEXT("lower")));
	}

	if (!ConstructVideoFileNode(ClipItemNode, InCinematicSectionData, Duration, bInMasterClip))
	{
		return false;
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructAudioClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, 
	const TSharedPtr<FMovieSceneExportAudioData> InAudioData, int32 InChannel, bool bInMasterClip, 
	const FString& InClipItemIdName1, const FString& InClipItemIdName2, int32 InClipIndex1, int32 InClipIndex2, int32 InTrackIndex1, int32 InTrackIndex2)
{
	if (!ExportData->IsExportDataValid() || !InAudioSectionData.IsValid())
	{
		return false;
	}

	bool bIsStereo = (InAudioSectionData->NumChannels == 2);

	int32 Duration{ 0 };
	int32 In{ 0 };
	int32 Out{ 0 };
	int32 Start{ 0 };
	int32 End{ 0 };
	GetAudioSectionFrames(InAudioSectionData, Duration, Start, End, In, Out);

	FString MasterClipIdName;
	GetMasterClipIdName(InAudioSectionData, MasterClipIdName);

	FString ClipItemIdName = (InChannel == 1 ? InClipItemIdName1 : InClipItemIdName2);

	TSharedRef<FFCPXMLNode> ClipItemNode = InParentNode->CreateChildNode(TEXT("clipitem"));
	ClipItemNode->AddAttribute(TEXT("id"), ClipItemIdName);
	if (!bInMasterClip)
	{
		ClipItemNode->AddAttribute(TEXT("premiereChannelType"), InAudioSectionData->NumChannels == 2 ? TEXT("stereo") : TEXT("mono"));
	}

	// elements
	ClipItemNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipIdName);
	ClipItemNode->CreateChildNode(TEXT("name"))->SetContent(InAudioSectionData->DisplayName);

	if (!bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("enabled"))->SetContent(true);
		ClipItemNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);
	}

	if (!ConstructRateNode(ClipItemNode))
	{
		return false;
	}

	if (!bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("start"))->SetContent(Start);
		ClipItemNode->CreateChildNode(TEXT("end"))->SetContent(End);
		ClipItemNode->CreateChildNode(TEXT("in"))->SetContent(In);
		ClipItemNode->CreateChildNode(TEXT("out"))->SetContent(Out);
	}

	if (!ConstructAudioFileNode(ClipItemNode, InAudioSectionData, InChannel))
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> SourceTrackNode = ClipItemNode->CreateChildNode(TEXT("sourcetrack"));
	SourceTrackNode->CreateChildNode(TEXT("mediatype"))->SetContent(FString(TEXT("audio")));
	SourceTrackNode->CreateChildNode(TEXT("trackindex"))->SetContent(InChannel);

	// stereo track clipitems must be linked using the linkclipref element
	if (bIsStereo)
	{
		TSharedRef<FFCPXMLNode> LinkNode = ClipItemNode->CreateChildNode(TEXT("link"));
		LinkNode->CreateChildNode(TEXT("linkclipref"))->SetContent(InClipItemIdName1);
		LinkNode->CreateChildNode(TEXT("mediatype"))->SetContent(FString(TEXT("audio")));
		LinkNode->CreateChildNode(TEXT("trackindex"))->SetContent(InTrackIndex1);
		LinkNode->CreateChildNode(TEXT("clipindex"))->SetContent(InClipIndex1);
		LinkNode->CreateChildNode(TEXT("groupindex"))->SetContent(1);

		LinkNode = ClipItemNode->CreateChildNode(TEXT("link"));
		LinkNode->CreateChildNode(TEXT("linkclipref"))->SetContent(InClipItemIdName2);
		LinkNode->CreateChildNode(TEXT("mediatype"))->SetContent(FString(TEXT("audio")));
		LinkNode->CreateChildNode(TEXT("trackindex"))->SetContent(InTrackIndex2);
		LinkNode->CreateChildNode(TEXT("clipindex"))->SetContent(InClipIndex1);
		LinkNode->CreateChildNode(TEXT("groupindex"))->SetContent(1);
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructVideoFileNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, int32 Duration, bool bInMasterClip)
{
	if (!ExportData->IsExportDataValid() || !InCinematicSectionData.IsValid())
	{
		return false;
	}

	FString FileIdName{ TEXT("") };
	bool bFileExists = false;
	GetFileIdName(InCinematicSectionData, FileIdName, bFileExists);

	// attributes
	TSharedRef<FFCPXMLNode> FileNode = InParentNode->CreateChildNode(TEXT("file"));
	FileNode->AddAttribute(TEXT("id"), FileIdName);

	if (!bFileExists)
	{
		FString FilePath = InCinematicSectionData->SourceFilePath.IsEmpty() ? SaveFilePath : InCinematicSectionData->SourceFilePath;
		FString FilePathName = SaveFilePath + TEXT("/") + InCinematicSectionData->SourceFilename;
		FString FilePathUrl = FString(TEXT("file://localhost/")) + FilePathName.Replace(TEXT(" "), TEXT("%20")).Replace(TEXT(":"), TEXT("%3a"));

		// required elements
		TSharedRef<FFCPXMLNode> NameNode = FileNode->CreateChildNode(TEXT("name"));
		NameNode->SetContent(InCinematicSectionData->SourceFilename);

		TSharedRef<FFCPXMLNode> PathUrlNode = FileNode->CreateChildNode(TEXT("pathurl"));
		PathUrlNode->SetContent(FilePathUrl);

		if (!ConstructRateNode(FileNode))
		{
			return false;
		}

		TSharedRef<FFCPXMLNode> DurationNode = FileNode->CreateChildNode(TEXT("duration"));
		DurationNode->SetContent(static_cast<int32>(Duration));

		if (!ConstructTimecodeNode(FileNode))
		{
			return false;
		}

		TSharedRef<FFCPXMLNode> MediaNode = FileNode->CreateChildNode(TEXT("media"));
		TSharedRef<FFCPXMLNode> VideoNode = MediaNode->CreateChildNode(TEXT("video"));

		if (!ConstructVideoSampleCharacteristicsNode(VideoNode, ExportData->GetResX(), ExportData->GetResY()))
		{
			return false;
		}
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructAudioFileNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, int32 InChannel)
{
	FString FileIdName{ TEXT("") };
	bool bFileExists = false;
	GetFileIdName(InAudioSectionData, FileIdName, bFileExists);

	int32 Duration{ 0 };
	int32 In{ 0 };
	int32 Out{ 0 };
	int32 Start{ 0 };
	int32 End{ 0 };
	GetAudioSectionFrames(InAudioSectionData, Duration, Start, End, In, Out);

	// attributes
	TSharedRef<FFCPXMLNode> FileNode = InParentNode->CreateChildNode(TEXT("file"));
	FileNode->AddAttribute(TEXT("id"), FileIdName);

	// only add details if file id did not already exist
	if (!bFileExists)
	{
		// FPaths
		FString FilePathName = InAudioSectionData->SourceFilePath + TEXT("/") + InAudioSectionData->SourceFilename;
		FString FilePathUrl = FString(TEXT("file://localhost/")) + FilePathName.Replace(TEXT(" "), TEXT("%20")).Replace(TEXT(":"), TEXT("%3a"));

		// required elements
		TSharedRef<FFCPXMLNode> NameNode = FileNode->CreateChildNode(TEXT("name"));
		NameNode->SetContent(InAudioSectionData->SourceFilename);

		TSharedRef<FFCPXMLNode> PathUrlNode = FileNode->CreateChildNode(TEXT("pathurl"));
		PathUrlNode->SetContent(FilePathUrl);

		if (!ConstructRateNode(FileNode))
		{
			return false;
		}

		if (!ConstructTimecodeNode(FileNode))
		{
			return false;
		}

		FileNode->CreateChildNode(TEXT("duration"))->SetContent(static_cast<int32>(Duration));

		TSharedRef<FFCPXMLNode> MediaNode = FileNode->CreateChildNode(TEXT("media"));
		TSharedRef<FFCPXMLNode> AudioNode = MediaNode->CreateChildNode(TEXT("audio"));

		if (!ConstructAudioSampleCharacteristicsNode(AudioNode, InAudioSectionData->Depth, InAudioSectionData->SampleRate))
		{
			return false;
		}
		AudioNode->CreateChildNode(TEXT("channelcount"))->SetContent(1);
		
		if (InAudioSectionData->NumChannels == 2)
		{
			AudioNode->CreateChildNode(TEXT("layout"))->SetContent(TEXT("stereo"));
			TSharedRef<FFCPXMLNode> AudioChannelNode = AudioNode->CreateChildNode(TEXT("audiochannel"));
			AudioChannelNode->CreateChildNode(TEXT("sourcechannel"))->SetContent(1);
			AudioChannelNode->CreateChildNode(TEXT("channellabel"))->SetContent(FString(TEXT("left")));

			// second audio channel
			AudioNode = MediaNode->CreateChildNode(TEXT("audio"));
			if (!ConstructAudioSampleCharacteristicsNode(AudioNode, InAudioSectionData->Depth, InAudioSectionData->SampleRate))
			{
				return false;
			}
			AudioNode->CreateChildNode(TEXT("channelcount"))->SetContent(1);
			AudioNode->CreateChildNode(TEXT("layout"))->SetContent(TEXT("stereo"));

			AudioChannelNode = AudioNode->CreateChildNode(TEXT("audiochannel"));
			AudioChannelNode->CreateChildNode(TEXT("sourcechannel"))->SetContent(2);
			AudioChannelNode->CreateChildNode(TEXT("channellabel"))->SetContent(FString(TEXT("right")));
		}
	}

	return true;
}

bool FFCPXMLExportVisitor::ConstructVideoSampleCharacteristicsNode(TSharedRef<FFCPXMLNode> InParentNode, int InWidth, int InHeight)
{
	TSharedRef<FFCPXMLNode> SampleCharacteristicsNode = InParentNode->CreateChildNode(TEXT("samplecharacteristics"));

	if (!ConstructRateNode(SampleCharacteristicsNode))
	{
		return false;
	}

	SampleCharacteristicsNode->CreateChildNode(TEXT("width"))->SetContent(InWidth);
	SampleCharacteristicsNode->CreateChildNode(TEXT("height"))->SetContent(InHeight);
	SampleCharacteristicsNode->CreateChildNode(TEXT("anamorphic"))->SetContent(false);
	SampleCharacteristicsNode->CreateChildNode(TEXT("pixelaspectratio"))->SetContent(FString(TEXT("square")));
	SampleCharacteristicsNode->CreateChildNode(TEXT("fielddominance"))->SetContent(FString(TEXT("lower")));

	return true;
}

bool FFCPXMLExportVisitor::ConstructAudioSampleCharacteristicsNode(TSharedRef<FFCPXMLNode> InParentNode, int InDepth, int InSampleRate)
{
	TSharedRef<FFCPXMLNode> SampleCharacteristicsNode = InParentNode->CreateChildNode(TEXT("samplecharacteristics"));
	SampleCharacteristicsNode->CreateChildNode(TEXT("depth"))->SetContent(InDepth);
	SampleCharacteristicsNode->CreateChildNode(TEXT("samplerate"))->SetContent(InSampleRate);

	return true;
}

bool FFCPXMLExportVisitor::ConstructRateNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	TSharedRef<FFCPXMLNode> RateNode = InParentNode->CreateChildNode(TEXT("rate"));

	TSharedRef<FFCPXMLNode> TimebaseNode = RateNode->CreateChildNode(TEXT("timebase"));
	TimebaseNode->SetContent(static_cast<int32>(ExportData->GetNearestWholeFrameRate()));

	TSharedRef<FFCPXMLNode> NTSCNode = RateNode->CreateChildNode(TEXT("ntsc"));
	NTSCNode->SetContent(ExportData->GetFrameRateIsNTSC());

	return true;
}

bool FFCPXMLExportVisitor::ConstructTimecodeNode(TSharedRef<FFCPXMLNode> InParentNode)
{
	TSharedRef<FFCPXMLNode> TimecodeNode = InParentNode->CreateChildNode(TEXT("timecode"));

	if (!ConstructRateNode(TimecodeNode))
	{
		return false;
	}

	TimecodeNode->CreateChildNode(TEXT("string"))->SetContent(FString(TEXT("00:00:00:00")));
	TimecodeNode->CreateChildNode(TEXT("frame"))->SetContent(0);

	return true;
}


/** Get duration, in and out frames for a given video shot section */
bool FFCPXMLExportVisitor::GetCinematicSectionFrames(const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, int32& OutDuration, int32& OutStartFrame, int32&OutEndFrame, int32& OutInFrame, int32& OutOutFrame)
{
	if (!InCinematicSectionData.IsValid() || !ExportData->MovieSceneData.IsValid())
	{
		return false;
	}

	int32 HandleFrames = ExportData->GetHandleFrames();
	OutStartFrame = InCinematicSectionData->StartFrame.Value;

	//
	// The cinematic section's end frame is exclusive, FCPXML is also exclusive. 
	// For example, a 50 frame duration movie that starts at frame 0 should have an end frame value of 50. 
	// See corresponding note in FCPXMLMetadataExport.cpp ConstructVideoClipItemNode and ConstructMasterClipNode which 
	// constructs the end frame from the actual outputted frames.
	//
	OutEndFrame = InCinematicSectionData->EndFrame.Value;
	OutDuration = OutEndFrame - OutStartFrame;
	OutInFrame = HandleFrames;
	OutOutFrame = HandleFrames + OutDuration;

	return true;
}

/** Get duration, in and out frames for a given audio shot section */
bool FFCPXMLExportVisitor::GetAudioSectionFrames(const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, int32& OutDuration, int32& OutStartFrame, int32&OutEndFrame, int32& OutInFrame, int32& OutOutFrame)
{
	if (!InAudioSectionData.IsValid() || !ExportData->MovieSceneData.IsValid())
	{
		return false;
	}

	OutStartFrame = InAudioSectionData->StartFrame.Value;
	OutEndFrame = InAudioSectionData->EndFrame.Value;
	OutDuration = OutEndFrame - OutStartFrame;
	OutInFrame = 0;
	OutOutFrame = OutDuration;

	return true;
}

bool FFCPXMLExportVisitor::HasMasterClipIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName, bool& bOutMasterClipExists)
{
	if (!InSection.IsValid())
	{
		return false;
	}

	FString Key;
	if (!ComposeFileKey(InSection, Key))
	{
		return false;
	}

	if (MasterClipIdMap.Num() > 0)
	{
		uint32 *FoundId = MasterClipIdMap.Find(Key);
		if (FoundId != nullptr)
		{
			OutName = FString::Printf(TEXT("masterclip-%d"), *FoundId);
			bOutMasterClipExists = true;
			return true;
		}
	}

	bOutMasterClipExists = false;
	return true;
}

bool FFCPXMLExportVisitor::GetMasterClipIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName)
{
	if (!InSection.IsValid())
	{
		return false;
	}

	bool bMasterClipExists = false;
	if (!HasMasterClipIdName(InSection, OutName, bMasterClipExists))
	{
		return false;
	}

	FString Key;
	if (!ComposeFileKey(InSection, Key))
	{
		return false;
	}

	if (!bMasterClipExists)
	{
		++MasterClipId;
		MasterClipIdMap.Add(Key, MasterClipId);
		OutName = FString::Printf(TEXT("masterclip-%d"), MasterClipId);
	}
	return true;
}

bool FFCPXMLExportVisitor::GetFileIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutFileIdName, bool& OutFileExists)
{
	if (!InSection.IsValid())
	{
		return false;
	}

	FString Key;
	if (!ComposeFileKey(InSection, Key))
	{
		return false;
	}

	if (FileIdMap.Num() > 0)
	{
		uint32 *FoundFileId = FileIdMap.Find(Key);
		if (FoundFileId != nullptr)
		{
			OutFileIdName = FString::Printf(TEXT("file-%d"), *FoundFileId);
			OutFileExists = true;
			return true;
		}
	}

	++FileId;
	FileIdMap.Add(Key, FileId);
	OutFileIdName = FString::Printf(TEXT("file-%d"), FileId);
	OutFileExists = false;
	return true;
}

void FFCPXMLExportVisitor::GetNextClipItemIdName(FString& OutName)
{
	++ClipItemId;
	OutName = FString::Printf(TEXT("clipitem-%d"), ClipItemId);
}

/** Compose a unique key string for audio sections based on channel */
bool FFCPXMLExportVisitor::ComposeFileKey(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName)
{
	if (!InSection.IsValid())
	{
		return false;
	}

	OutName = InSection->MovieSceneSection->GetPathName();
	return true;
}

bool FFCPXMLExportVisitor::CreateCinematicSectionMetadata(const UMovieSceneCinematicShotSection* InSection, FString& OutMetadata) const
{
	if (InSection == nullptr)
	{
		return false;
	}

	OutMetadata = TEXT("[UEShotSection=") + InSection->GetPathName() + TEXT("]");

	// Store the start offset and the handle frames for round-tripping to compute the new start offset
	int32 HandleFrames = ExportData->GetHandleFrames();
	FFrameRate TickResolution = InSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	int32 StartFrameOffset = ConvertFrameTime(InSection->Parameters.StartFrameOffset, TickResolution, ExportData->GetFrameRate()).CeilToFrame().Value;

	OutMetadata += TEXT("[UEShotStartOffset=") + FString::FromInt(StartFrameOffset) + TEXT("]");
	OutMetadata += TEXT("[UEShotHandleFrames=") + FString::FromInt(HandleFrames) + TEXT("]");
	return true;
}

/** Get metadata section name from sequencer shot name - format is "[UESoundWave=soundwaveobjectname][UESoundSectionTopLevel=toplevelobjectname][UESoundSection=sectionobjectname]", whitespace ok. */
bool FFCPXMLExportVisitor::CreateSoundWaveMetadata(const USoundWave* InSoundWave, const TArray<const UMovieSceneAudioSection*> InAudioSections, FString& OutMetadata) const
{
	if (InSoundWave == nullptr)
	{
		return false;
	}

	TArray<FString> SectionsAdded;
	bool bTopLevelAdded = false;
	OutMetadata = TEXT("[UESoundWave=") + InSoundWave->GetPathName() + TEXT("]");
	for (const UMovieSceneAudioSection* AudioSection : InAudioSections)
	{
		if (!bTopLevelAdded)
		{
			OutMetadata += TEXT("[UEAudioSectionTopLevel=") + FFCPXMLExportVisitor::GetAudioSectionTopLevelName(AudioSection) + TEXT("]");
			bTopLevelAdded = true;
		}

		// skip duplicate section names
		FString SectionName = FFCPXMLExportVisitor::GetAudioSectionName(AudioSection);
		if (SectionsAdded.Num() == 0 || !SectionsAdded.Contains(SectionName))
		{
			OutMetadata += TEXT("[UEAudioSection=") + SectionName + TEXT("]");
			SectionsAdded.Add(SectionName);
		}
	}
	return true;
}

FString FFCPXMLExportVisitor::GetAudioSectionTopLevelName(const UMovieSceneAudioSection* InAudioSection)
{
	return InAudioSection->GetOutermost()->GetName();
}

FString FFCPXMLExportVisitor::GetAudioSectionName(const UMovieSceneSection* InAudioSection)
{
	return InAudioSection->GetFullGroupName(false);
}


#undef LOCTEXT_NAMESPACE
