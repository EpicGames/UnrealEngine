// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FCPXML/FCPXMLNode.h"
#include "MovieScene.h"
#include "MovieSceneTranslator.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"

/** The FFCPXMLExportVisitor class exports from Sequencer data into the FCP 7 XML structure.

    This class will eventually be used to merge the data with an existing XML structure 
	representing previously imported material. This is intended to preserve metadata 
	roundtrip between Sequencer and the FCP XML format.
	
	Currently the VisitNode functions are mostly empty. This is where the
	merge data functionality will be implemented. */

class FFCPXMLExportVisitor : public FFCPXMLNodeVisitor
{
public:
	/** Constructor */
	FFCPXMLExportVisitor(FString InSaveFilename, TSharedRef<FMovieSceneExportData> InExportData, TSharedRef<FMovieSceneTranslatorContext> InExportContext);
	/** Destructor */
	virtual ~FFCPXMLExportVisitor();

public:

	/** Called when visiting a FFCPXMLBasicNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode) override final;
	/** Called when visiting a FFCPXMLXmemlNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode) override final;
	/** Called when visiting a FFCPXMLSequenceNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode) override final;
	/** Called when visiting a FFCPXMLVideoNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode) override final;
	/** Called when visiting a FFCPXMLAudioNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode) override final;
	/** Called when visiting a FFCPXMLTrackNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode) override final;
	/** Called when visiting a FFCPXMLClipNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode) override final;
	/** Called when visiting a FFCPXMLClipItemNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode) override final;
	/** Called when visiting a FFCPXMLFileNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode) override final;

public:
	/** Creates project node. */
	bool ConstructProjectNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates master video clip nodes. */
	virtual bool ConstructMasterVideoClipNodes(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates master audio clip nodes. */
	bool ConstructMasterAudioClipNodes(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates master video clip node. */
	bool ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData);
	/** Creates master audio clip node. */
	bool ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, const TSharedPtr<FMovieSceneExportAudioData> InCinematicData);
	/** Creates colorinfo node. */
	bool ConstructColorInfoNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates logginginfo node. */
	bool ConstructLoggingInfoNode(TSharedRef<FFCPXMLNode> InParentNode, const UMovieSceneSection* InMovieSceneSection);
	/** Creates logginginfo node. */
	bool ConstructLoggingInfoNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InSectionData);
	/** Creates sequence node. */
	bool ConstructSequenceNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates video node. */
	bool ConstructVideoNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates audio node. */
	bool ConstructAudioNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates video track node. */
	virtual bool ConstructVideoTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicTrackData> InCinematicTrackData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData);
	/** Creates audio track node. */
	bool ConstructAudioTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioTrackData> InAudioTrackData, const TSharedPtr<FMovieSceneExportAudioData> InAudioData, uint32 InTrackIndex, uint32 OutNumTracks);
	/** Creates video clip item node. */
	bool ConstructVideoClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, bool bInMasterClip);
	/** Creates audio clip item node. */
	bool ConstructAudioClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, 
		const TSharedPtr<FMovieSceneExportAudioData> InAudioData, int32 InChannel, bool bInMasterClip, 
		const FString& InClipItemIdName1, const FString& InClipItemIdName2, int32 InClipIndex1, int32 InClipIndex2, int32 InTrackIndex1, int32 InTrackIndex2);
	/** Creates video file node. */
	bool ConstructVideoFileNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, int32 InDuration, bool bInMasterClip);
	/** Creates audio file node. */
	bool ConstructAudioFileNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, int32 InChannel);
	/** Creates video sample characteristics node */
	bool ConstructVideoSampleCharacteristicsNode(TSharedRef<FFCPXMLNode> InParentNode, int InWidth, int InHeight);
	/** Creates audio video sample characteristics node */
	bool ConstructAudioSampleCharacteristicsNode(TSharedRef<FFCPXMLNode> InParentNode, int InDepth, int InSampleRate);
	/** Creates rate node. */
	bool ConstructRateNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates timecode node. */
	bool ConstructTimecodeNode(TSharedRef<FFCPXMLNode> InParentNode);
	/** Creates logginginfo elements. */
	void ConstructLoggingInfoElements(TSharedRef<FFCPXMLNode> InLoggingInfoNode, const UObject* InObject = nullptr);
	/** Set logginginfo element value. */
	void SetLoggingInfoElementValue(TSharedPtr<FFCPXMLNode> InNode, const UObject* InObject, const FString& InElement);
	/** Get cinematic duration, start, end, in, out frames for a given cinematic shot section */
	bool GetCinematicSectionFrames(const TSharedPtr<FMovieSceneExportCinematicSectionData> InCinematicSectionData, int32& OutDuration, int32& OutStartFrame, int32& OutEndFrame, int32& OutInFrame, int32& OutOutFrame);
	/** Get audio duration, start, end, in, out frames for a given cinematic shot section */
	bool GetAudioSectionFrames(const TSharedPtr<FMovieSceneExportAudioSectionData> InAudioSectionData, int32& OutDuration, int32& OutStartFrame, int32& OutEndFrame, int32& OutInFrame, int32& OutOutFrame);

protected:

	/** Has master clip id for given section */
	bool HasMasterClipIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName, bool& bOutMasterClipExists);
	/** Get master clip id name for section */
	bool GetMasterClipIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName);
	/** Get file id name for section, adds to file map if a new id name is created */
	bool GetFileIdName(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName, bool& bFileExists);
	/** Get next clip item name */
	void GetNextClipItemIdName(FString& OutName);

	/** Compose key for section file */
	bool ComposeFileKey(const TSharedPtr<FMovieSceneExportSectionData> InSection, FString& OutName);

	/** Returns true if audio track data contains sections with 2 channels */
	bool HasStereoAudioSections(const  TArray<TSharedPtr<FMovieSceneExportAudioSectionData>>& InAudioTrackData) const;

	/** Get metadata section name from sequencer shot name - format is "[UESection=sectionobjectname]", whitespace ok. */
	bool CreateCinematicSectionMetadata(const UMovieSceneCinematicShotSection* InSection, FString& OutMetadata) const;
	/** Get metadata section name from sequencer shot name - format is "[UESoundWave=soundwaveobjectname]", whitespace ok. */
	bool CreateSoundWaveMetadata(const USoundWave* InSoundWave, const TArray<const UMovieSceneAudioSection*> InAudioSections, FString& OutMetadata) const;
	/** Get id for audio top level section */
	static FString GetAudioSectionTopLevelName(const UMovieSceneAudioSection* InAudioSection);
	/** Get audio section group name */
	static FString GetAudioSectionName(const UMovieSceneSection* InAudioSection);

protected:

	TSharedRef<FMovieSceneExportData> ExportData;
	TSharedRef<FMovieSceneTranslatorContext> ExportContext;
	FString SaveFilePath;
	uint32 SequenceId;
	uint32 MasterClipId;
	uint32 ClipItemId;
	uint32 FileId;

	/** Map section's unique key string to the id used for masterclip element names */
	TMap<FString, uint32> MasterClipIdMap;
	/** Map section's source file name to its file element name */
	TMap<FString, uint32> FileIdMap;
};
