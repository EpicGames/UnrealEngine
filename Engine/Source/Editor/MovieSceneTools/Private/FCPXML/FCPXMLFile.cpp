// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLFile.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "XmlNode.h"

#define LOCTEXT_NAMESPACE "FCPXMLImporter"

FFCPXMLFile::FFCPXMLFile()
{
	RootNode = nullptr;
}

void FFCPXMLFile::ConstructFile(const FString& InProjectName)
{
	RootNode = MakeShared<FFCPXMLXmemlNode>(nullptr, AsShared());
	RootNode->AddAttribute(TEXT("version"), TEXT("4"));
}

bool FFCPXMLFile::LoadFile(const FString& InFile, EConstructMethod::Type InConstructMethod)
{
	Clear();

	bool bResult = true;

	FXmlFile *XmlFile = new FXmlFile(InFile, InConstructMethod);
	if (XmlFile->IsValid())
	{
		FXmlNode* XmlRootNode = XmlFile->GetRootNode();
		if (XmlRootNode == nullptr)
		{
			bResult = false;
		}
		else
		{
			RootNode = FFCPXMLNode::CreateFFCPXMLNode(XmlRootNode->GetTag(), nullptr, AsShared());
			RootNode->CopyFrom(XmlRootNode);

			// construct reference id map
			FFCPXMLFileSetupVisitor FileSetupVisitor;
			bResult = Accept(FileSetupVisitor);
		}
	}
	else
	{
		ErrorMessage = FText::FromString(XmlFile->GetLastError());
		bResult = false;
	}

	delete XmlFile;

	return bResult;
}

FText FFCPXMLFile::GetLastError() const
{
	return ErrorMessage;
}

void FFCPXMLFile::Clear()
{
	// create placeholder node until we load the next file
	RootNode = nullptr;
	ErrorMessage = FText();
	ReferenceMap.Empty();
}

bool FFCPXMLFile::IsValidFile() const
{
	return (RootNode.IsValid());
}

bool FFCPXMLFile::Accept(FFCPXMLNodeVisitor& InNodeVisitor) 
{
	if (!RootNode.IsValid()) 
	{ 
		return false; 
	}
	return RootNode->Accept(InNodeVisitor);
}

const TSharedPtr<FFCPXMLNode> FFCPXMLFile::GetRootNode() const
{
	return RootNode;
}

TSharedPtr<FFCPXMLNode> FFCPXMLFile::GetRootNode()
{
	return RootNode;
}

bool FFCPXMLFile::Save(const FString& InPath)
{
	if (!RootNode.IsValid()) 
	{ 
		return false; 
	}
	bool bResult = false;

	FString Xml(TEXT(""));
	FString Indent(TEXT(""));
	RootNode->GetXmlBuffer(Indent, Xml);

	FXmlFile *XmlFile = new FXmlFile(Xml, EConstructMethod::ConstructFromBuffer);
	if (XmlFile->IsValid())
	{
		bResult = XmlFile->Save(InPath);
		if (!bResult)
		{
			ErrorMessage = FText::FromString(XmlFile->GetLastError());
		}
	}
	else
	{
		ErrorMessage = LOCTEXT("FileSaveFail", "Failed to import the file.");
		bResult = false;
	}

	delete XmlFile;

	return bResult;
}

bool FFCPXMLFile::AddReference(const FString& InElement, const FString &InId, TSharedPtr<FFCPXMLNode> InNode)
{
	FString Key = ComposeKey(InElement, InId);
	if (ReferenceMap.Contains(Key))
	{
		return false;
	}
	ReferenceMap.Add(Key, InNode);
	return true;
}

TSharedPtr<FFCPXMLNode> FFCPXMLFile::GetReference(const FString &InElement, const FString &InId)
{
	TSharedPtr<FFCPXMLNode>* NodePtr = ReferenceMap.Find(ComposeKey(InElement, InId));
	if (NodePtr == nullptr)
	{
		return nullptr;
	}
	return *NodePtr;
}

FString FFCPXMLFile::ComposeKey(const FString& A, const FString& B)
{
	return FString(A + TEXT("|") + B);
}

/** Node visitor class used to setup the file when it is loaded */
FFCPXMLFileSetupVisitor::FFCPXMLFileSetupVisitor() {}

FFCPXMLFileSetupVisitor::~FFCPXMLFileSetupVisitor() {}

/** Adds a reference id to node common data, if id does not already exist */
void FFCPXMLFileSetupVisitor::AddReferenceId(FString InType, TSharedRef<FFCPXMLNode> InNode)
{
	FString IdValue;
	
	if (InNode->GetAttributeValue(TEXT("id"), IdValue))
	{
		TSharedPtr<FFCPXMLNode> RefNode = InNode->GetReference(InType, IdValue);
		if (!RefNode.IsValid())
		{
			InNode->AddReference(InType, IdValue, InNode);
		}
	}
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode)
{
	// @todo "timecode" and "effect" also support reference ids but are not handled yet
	return InBasicNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode)
{
	return InXmemlNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode)
{
	AddReferenceId(TEXT("sequence"), InSequenceNode);
	return InSequenceNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode)
{
	return InVideoNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode)
{
	return InAudioNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode)
{
	return InTrackNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode)
{
	AddReferenceId(TEXT("clip"), InClipNode);
	return InClipNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode)
{
	AddReferenceId(TEXT("clipitem"), InClipItemNode);
	return InClipItemNode->VisitChildren(*this);
}

bool FFCPXMLFileSetupVisitor::VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode)
{
	AddReferenceId(TEXT("file"), InFileNode);
	return InFileNode->VisitChildren(*this);
}

#undef LOCTEXT_NAMESPACE
