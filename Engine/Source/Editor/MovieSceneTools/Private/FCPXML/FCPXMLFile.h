// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "FCPXML/FCPXMLNode.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "XmlFile.h"

/**
	The FFCPXMLFile class represents the overall FCP 7 XML file and is the containing class for
	the node tree structure.

	The FCP 7 XML Translator uses the FXmlParser classes to parse and write out XML. 
	It constructs its own representation using the classes FFCPXMLFile and FFCPXMLNode
	so the import and export can manipulate the XML data as needed. 
*/
class FFCPXMLFile : public TSharedFromThis<FFCPXMLFile>
{
public:

	/** Default constructor */
	FFCPXMLFile();

	/** Destructor */
	~FFCPXMLFile() { Clear(); };

public:

	/** Initializes the file root in preparation to construct XML file */
	void ConstructFile(const FString& InProjectName);

	/**
	* Loads the file with the passed path. Path is either treated as a filename to open, or as a text
	* buffer to load.
	* @param	Path				The path/text to use
	* @param	ConstructMethod		Whether to load a file of use the string as a buffer of xml data
	*/
	bool LoadFile(const FString& InPath, EConstructMethod::Type InConstructMethod = EConstructMethod::ConstructFromFile);

	/** Gets the last error message from the class */
	FText GetLastError() const;

	/** Clears the file of all internals. Note: Makes any existing pointers to file and nodes INVALID */
	void Clear();

	/** Checks to see if a file is loaded */
	bool IsValidFile() const;

	/** Accepts a node visitor */
	bool Accept(FFCPXMLNodeVisitor& InNodeVisitor);

	/**
	* Returns the root node of the loaded file. nullptr if no file loaded.
	* It is assumed that there will always be one and only one root node.
	* @return						Pointer to root node
	*/
	const TSharedPtr<FFCPXMLNode> GetRootNode() const;

	/**
	* Returns the root node of the loaded file. nullptr if no file loaded.
	* It is assumed that there will always be one and only one root node.
	* @return						Pointer to root node
	*/
	TSharedPtr<FFCPXMLNode> GetRootNode();

	/**
	* Write to disk, UTF-8 format
	* @param	Path				File path to save to
	* @return						Whether writing the XML to a file succeeded
	*/
	bool Save(const FString& InPath);

	/** Add reference id to map */
	bool AddReference(const FString& InElement, const FString &InId, TSharedPtr<FFCPXMLNode>);

	/** Get node associated with reference id */
	TSharedPtr<FFCPXMLNode> GetReference(const FString &InElement, const FString &InId);

private:

	/** Compose single map key using two string inputs */
	FString ComposeKey(const FString& A, const FString& B);

private:
	/** The passed-in path of the loaded file (might be absolute or relative) */
	FString LoadedFile;
	/** An error message generated on errors to return to the client */
	FText ErrorMessage;
	/** A pointer to the root node */
	TSharedPtr<FFCPXMLNode> RootNode;
	/** Map of reference ids in currently loaded file */
	TMap<FString, TSharedPtr<FFCPXMLNode>> ReferenceMap;
};

/** Node visitor class used to set up reference id map for loaded file */
class FFCPXMLFileSetupVisitor : public FFCPXMLNodeVisitor
{
public:
	/** Constructor */
	FFCPXMLFileSetupVisitor();
	/** Destructor */
	~FFCPXMLFileSetupVisitor();

public:
	/** Called when visiting a FFCPXMLBasicNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode) override final;
	/** Called when visiting a FFCPXMLMemlNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode) override final;
	/** Called when visiting a FFCPXMLSequenceNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode) override final;
	/** Called when visiting a FFCPXMLVideoNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode) override final;
	/** Called when visiting a FFCPXMLAudioNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode) override final;
	/** Called when visiting a FFCPXMLTrackNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode) override final;
	/** Called when visiting a FFCPXMLClipNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode) override final;
	/** Called when visiting a FFCPXMLClipItemNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode) override final;
	/** Called when visiting a FFCPXMLFileNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode) override final;

private:
	/** Adds a reference id to node common data, if id does not already exist */
	void AddReferenceId(FString InType, TSharedRef<FFCPXMLNode> InNode);
};
