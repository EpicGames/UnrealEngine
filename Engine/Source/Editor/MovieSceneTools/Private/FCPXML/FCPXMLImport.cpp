// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLImport.h"
#include "MovieScene.h"
#include "MovieSceneTranslator.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/MetaData.h"
#include "Sound/SoundWave.h"
#include "UObject/Package.h"

/**
 METADATA NOTES:

 Cinematic sections have a 1:1 relationships with their source .avi files. Audio sections, by contrast, have a many:1 relationship with
 their source .wav files. Sections are represented by clipitem nodes in FCP XML while their source files are represented by clip nodes. 

 Although the LoggingInfo node which stores metadata can be attached to any FCP XML node, Premiere only recognizes them for clip and 
 file nodes. There is no way to associate metadata at the section (clipitem node) or track (track node) level that Premiere will roundtrip. 
 Confusingly, Premiere exports the clip LoggingInfo on the clipitem. But it is always the same on all clipitems associated with a 
 given clip.

 For each audio asset, the clip metadata includes all the section path names associated with that audio asset. These are then used
 on import by associated a clip item with the next path name from metadata and then marking that path name as used.

 TRACK NOTES:

 Because it is not possible to roundtrip track node metadata through Premiere, the importer creates a one-to-one ordered correspondence 
 between Sequencer tracks and incoming tracks from FCP XML. 

 In FCP XML, each track with stereo clipitems is encoded as 2 separate but linked tracks. But in Premiere, this only appears as a single
 track. The importer skips reading the second track, representing the second channel, since the relevant duration, start, end 
 times will be the same as on the clipitems in the first channel track.

 CURRENT LIMITATIONS:

 - Rendered movie files must be named '{shot}.avi
 - Cinematic and audio sections can be updated/moved/add by import but never removed.
 - Sound cues are not supported
 - Nested sequences are not supported.

 */

#define LOCTEXT_NAMESPACE "FCPXMLImporter"

#define INHERIT true
#define NO_INHERIT false

FFCPXMLImportVisitor::FFCPXMLImportVisitor(TSharedRef<FMovieSceneImportData> InImportData, TSharedRef<FMovieSceneTranslatorContext> InImportContext) :
	FFCPXMLNodeVisitor()
, ImportData(InImportData)
, ImportContext(InImportContext)
, bInSequenceNode(false)
, bInVideoNode(false)
, bInAudioNode(false)
, bInVideoTrackNode(false)
, bInAudioTrackNode(false)
, CurrVideoTrackRowIndex(0)
, CurrAudioTrackListIndex(0)
, CurrAudioData(nullptr)
, CurrAudioTrackRowIndex(0)
, bCurrImportAudioTrackIsStereoChannel(false)
, MaxVideoTrackRowIndex(0)
, MaxAudioTrackRowIndex(0)
{
	ConstructAudioTrackList();
	if (AudioTrackList.Num() > 0)
	{
		if (AudioTrackList[0].IsValid())
		{
			CurrAudioData = AudioTrackList[0]->AudioTrackData;
			CurrAudioTrackRowIndex = AudioTrackList[0]->RowIndex;
		}
	}
}

FFCPXMLImportVisitor::~FFCPXMLImportVisitor() {}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode)
{
	return InBasicNode->VisitChildren(*this);
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode)
{
	return InXmemlNode->VisitChildren(*this);
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode)
{
	bool bPrev = bInSequenceNode;
	bInSequenceNode = true;

	// Sequences can be referenced so flag to visit reference node children
	bool bSuccess = InSequenceNode->VisitChildren(*this, true);

	bInSequenceNode = false;

	return bSuccess;
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode)
{
	bool bPrev = bInVideoNode;
	bInVideoNode = true;
	int32 PrevVideoTrackRowIndex = CurrVideoTrackRowIndex;
	CurrVideoTrackRowIndex = 0;

	bool bSuccess = InVideoNode->VisitChildren(*this);

	bInVideoNode = bPrev;
	CurrVideoTrackRowIndex = PrevVideoTrackRowIndex;

	return bSuccess;
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode)
{
	bool bPrev = bInAudioNode;
	bInAudioNode = true;

	bool bSuccess = InAudioNode->VisitChildren(*this);

	bInAudioNode = bPrev;

	return bSuccess;
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode)
{
	bool bPrevInTrackNode = false;
	int32 PrevTrackIndex = 0;

	if (bInSequenceNode && bInVideoNode)
	{
		bPrevInTrackNode = bInVideoTrackNode;
		bInVideoTrackNode = true;
	}
	else if (bInSequenceNode && bInAudioNode)
	{
		bPrevInTrackNode = bInAudioTrackNode;
		bInAudioTrackNode = true;
		bCurrImportAudioTrackIsStereoChannel = false;
	}

	bool bSuccess = InTrackNode->VisitChildren(*this);

	if (bInSequenceNode && bInVideoNode)
	{
		bInVideoTrackNode = bPrevInTrackNode;
		CurrVideoTrackRowIndex++;
		if (MaxVideoTrackRowIndex < CurrVideoTrackRowIndex)
		{
			MaxVideoTrackRowIndex = CurrVideoTrackRowIndex;
		}
	}
	else if (bInSequenceNode && bInAudioNode)
	{
		bInAudioTrackNode = bPrevInTrackNode;
		if (!bCurrImportAudioTrackIsStereoChannel)
		{
			CurrAudioTrackListIndex++;
			if (CurrAudioTrackListIndex < AudioTrackList.Num())
			{
				if (!AudioTrackList[CurrAudioTrackListIndex].IsValid())
				{
					return false;
				}
				CurrAudioData = AudioTrackList[CurrAudioTrackListIndex]->AudioTrackData;
				CurrAudioTrackRowIndex = AudioTrackList[CurrAudioTrackListIndex]->RowIndex;
			}
			else
			{
				CurrAudioTrackRowIndex++;
			}
		}
		if (MaxAudioTrackRowIndex < CurrAudioTrackRowIndex)
		{
			MaxAudioTrackRowIndex = CurrAudioTrackRowIndex;
		}
	}

	return bSuccess;
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode)
{
	// masterclip
	bool bIsMasterClip = false;
	FString MasterClipName(TEXT(""));
	FString LogNoteMetadata(TEXT(""));

	InClipNode->GetChildValue<bool>("ismasterclip", bIsMasterClip, ENodeInherit::NoInherit);

	bool bHasMasterClipName = InClipNode->GetChildValue<FString>("masterclipid", MasterClipName, ENodeInherit::NoInherit);

	TSharedPtr<FFCPXMLNode> LoggingInfoNode = InClipNode->GetChildNode("logginginfo", ENodeInherit::NoInherit, ENodeReference::NoReferences);
	if (LoggingInfoNode.IsValid())
	{
		bool bHasLoggingShotTrack = LoggingInfoNode->GetChildValue<FString>("lognote", LogNoteMetadata, ENodeInherit::NoInherit, ENodeReference::NoReferences);

		if (!LogNoteMetadata.IsEmpty() && bHasMasterClipName && !MasterClipName.IsEmpty())
		{
			FString SectionName{ TEXT("") };
			TSharedPtr<FFCPXMLImportAudioMetadata> AudioMetadata;

			if (GetCinematicSectionPathNameFromMetadata(LogNoteMetadata, SectionName))
			{
				AddMasterClipCinematicSectionPathName(MasterClipName, SectionName);
			}
			else if (GetAudioFromMetadata(LogNoteMetadata, AudioMetadata))
			{
				AddMasterClipAudioMetadata(MasterClipName, AudioMetadata);
			}
			AddMasterClipLoggingNode(MasterClipName, LoggingInfoNode);
		}
	}

	// Clips can be referenced so flag to visit reference node children
	return InClipNode->VisitChildren(*this, true);
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode)
{
	if (bInSequenceNode)
	{
		if (bInVideoTrackNode && !VisitVideoClipItemNode(InClipItemNode))
		{
			return false;
		}
		else if (bInAudioTrackNode && !VisitAudioClipItemNode(InClipItemNode))
		{
			return false;
		}
	}

	// Clip items can be referenced so flag to visit reference node children
	return InClipItemNode->VisitChildren(*this, true);
}

bool FFCPXMLImportVisitor::VisitVideoClipItemNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode)
{
	if (!bInSequenceNode || !bInVideoTrackNode)
	{
		return false;
	}

	FString ClipItemName{ TEXT("") };
	FString ClipItemId{ TEXT("") };
	FString MasterClipId{ TEXT("") };
	FString LogNote{ TEXT("") };
	FString Filename{ TEXT("") };
	TSharedPtr<FFCPXMLNode> LoggingInfoNode = nullptr;
	FFrameRate FrameRate;
	FFrameNumber StartOffset;
	FFrameNumber Start;
	FFrameNumber End;

	if (!GetClipItemNodeData(InClipItemNode, ClipItemName, ClipItemId, MasterClipId, LoggingInfoNode, LogNote, Filename, FrameRate, StartOffset,Start, End))
	{
		return false;
	}

	// check for metadata
	if (!LoggingInfoNode.IsValid() && !MasterClipId.IsEmpty())
	{
		GetMasterClipLoggingNode(MasterClipId, LoggingInfoNode);

		// If the logging info node is validated, get the lognote data since it's a child of the new logging info node
		if (LoggingInfoNode.IsValid())
		{
			if (!LoggingInfoNode->GetChildValue<FString>("lognote", LogNote, ENodeInherit::NoInherit, ENodeReference::NoReferences))
			{
				LogNote = TEXT("");
			}
		}
	}

	FString SectionPathName = GetCinematicSectionPathName(LogNote, MasterClipId);

	int32 HandleFrames = 0;
	int32 OriginalStartOffset = 0;
	TOptional<FFrameNumber> NewStartOffset;
	if (GetCinematicSectionHandleFramesFromMetadata(LogNote, HandleFrames) && GetCinematicSectionStartOffsetFromMetadata(LogNote, OriginalStartOffset))
	{
		NewStartOffset = OriginalStartOffset - (HandleFrames - StartOffset);
	}
	else
	{
		NewStartOffset = StartOffset;
	}

	// Find actual section
	TSharedPtr<FMovieSceneImportCinematicSectionData> SectionData = nullptr;
	if (!SectionPathName.IsEmpty())
	{
		SectionData = ImportData->FindCinematicSection(SectionPathName);
	}

	if (SectionData.IsValid())
	{
		// Update existing cinematic section
		if (!ImportData->SetCinematicSection(SectionData, CurrVideoTrackRowIndex, FrameRate, Start, End, NewStartOffset))
		{
			return false;
		}
	}
	else
	{
		// Add new cinematic section
		SectionData = ImportData->CreateCinematicSection(ClipItemName, CurrVideoTrackRowIndex, FrameRate, Start, End, NewStartOffset.IsSet() ? NewStartOffset.GetValue() : 0);
		if (!SectionData.IsValid())
		{
			return false;
		}
	}

	// Import metadata to cinematic section
	if (LoggingInfoNode.IsValid())
	{
		ImportSectionMetaData(LoggingInfoNode, SectionData->CinematicSection);
	}
	
	return true;
}


bool FFCPXMLImportVisitor::VisitAudioClipItemNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode)
{
	// This should only have been called if clipitem is in an audio track node within a sequence node
	if (!bInSequenceNode || !bInAudioTrackNode)
	{
		return false;
	}

	FString ClipItemName{ TEXT("") };
	FString ClipItemId{ TEXT("") };
	FString MasterClipId{ TEXT("") };
	FString LogNote{ TEXT("") };
	FString Filename{ TEXT("") };
	TSharedPtr<FFCPXMLNode> LoggingInfoNode = nullptr;
	FFrameRate FrameRate;
	FFrameNumber StartOffset;
	FFrameNumber Start;
	FFrameNumber End;

	// Get relevant data from this clip item node
	if (!GetClipItemNodeData(InClipItemNode, ClipItemName, ClipItemId, MasterClipId, LoggingInfoNode, LogNote, Filename, FrameRate, StartOffset, Start, End))
	{
		return false;
	}

	// Check if we are in track containing second channel of stereo clips.
	if (!bCurrImportAudioTrackIsStereoChannel && GetAudioClipItemNodeChannel(InClipItemNode, ClipItemId) == 2)
	{
		bCurrImportAudioTrackIsStereoChannel = true;
	}

	// Skip track holding second channel of stereo clips.
	if (bCurrImportAudioTrackIsStereoChannel)
	{
		return true;
	}

	// Get master clip logging info, if no logging info here
	if (!LoggingInfoNode.IsValid() && !MasterClipId.IsEmpty())
	{
		GetMasterClipLoggingNode(MasterClipId, LoggingInfoNode);

		// If the logging info node is validated, get the lognote data since it's a child of the new logging info node
		if (LoggingInfoNode.IsValid())
		{
			if (!LoggingInfoNode->GetChildValue<FString>("lognote", LogNote, ENodeInherit::NoInherit, ENodeReference::NoReferences))
			{
				LogNote = TEXT("");
			}
		}
	}

	// Get audio metadata
	TSharedPtr<FFCPXMLImportAudioMetadata> AudioMetadata = GetAudioMetadataObject(LogNote, MasterClipId);

	// Get next audio section based on the audio metadata
	TSharedPtr<FMovieSceneImportAudioSectionData> AudioSectionData = nullptr;
	TSharedPtr<FMovieSceneImportAudioData> AudioData = nullptr;
	if (AudioMetadata.IsValid())
	{
		if (!GetNextAudioSection(AudioMetadata, AudioData, AudioSectionData))
		{
			return false;
		}
	}

	if (AudioSectionData.IsValid())
	{
		if (CurrAudioData.IsValid() && CurrAudioData->MovieSceneTrack != nullptr &&
			AudioData.IsValid() && AudioData->MovieSceneTrack != nullptr &&
			CurrAudioData->MovieSceneTrack->GetFullName() != AudioData->MovieSceneTrack->GetFullName())
		{
			// Move audio section
			if (!ImportData->MoveAudioSection(AudioSectionData, AudioData, CurrAudioData, CurrAudioTrackRowIndex))
			{
				return false;
			}
		}
		// Update existing audio section
		if (!ImportData->SetAudioSection(AudioSectionData, CurrAudioTrackRowIndex, FrameRate, Start, End, StartOffset))
		{
			return false;
		}
	}
	else
	{
		bool bUseSoundPathName = (AudioMetadata.IsValid() && !AudioMetadata->SoundPathName.IsEmpty());
		FString SoundWaveName =  bUseSoundPathName ? AudioMetadata->SoundPathName : Filename;
		if (!SoundWaveName.IsEmpty())
		{
			// Add new audio section
			AudioSectionData = ImportData->CreateAudioSection(SoundWaveName, bUseSoundPathName, CurrAudioData, CurrAudioTrackRowIndex, FrameRate, Start, End, StartOffset);
			if (!AudioSectionData.IsValid())
			{
				return false;
			}
		}
	}

	// Import metadata to cinematic section
	if (LoggingInfoNode.IsValid() && AudioSectionData.IsValid())
	{
		ImportSectionMetaData(LoggingInfoNode, AudioSectionData->AudioSection);
	}

	return true;
}

bool FFCPXMLImportVisitor::VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode)
{
	// Files can be referenced so flag to visit reference node children
	return InFileNode->VisitChildren(*this, true);
}


bool FFCPXMLImportVisitor::GetClipItemNodeData(TSharedRef<FFCPXMLClipItemNode> InClipItemNode, FString& OutClipItemName, FString& OutClipItemId, FString& OutMasterClipId,
	TSharedPtr<FFCPXMLNode>& OutLoggingInfoNode, FString& OutLogNote, FString& OutFilename, FFrameRate& OutFrameRate, FFrameNumber& OutStartOffset,
	FFrameNumber& OutStart, FFrameNumber &OutEnd)
{
	// Shot name
	if (!InClipItemNode->GetChildValue("name", OutClipItemName, ENodeInherit::NoInherit) || OutClipItemName.IsEmpty())
	{
		return false;
	}

	// clip item id
	if (!InClipItemNode->GetAttributeValue("id", OutClipItemId) || OutClipItemId.IsEmpty())
	{
		return false;
	}

	// masterclip id
	if (!InClipItemNode->GetChildValue<FString>("masterclipid", OutMasterClipId, ENodeInherit::NoInherit))
	{
		OutMasterClipId = TEXT("");
	}

	// logging info node
	OutLoggingInfoNode = InClipItemNode->GetChildNode("logginginfo", ENodeInherit::NoInherit, ENodeReference::NoReferences);

	// log note
	if (OutLoggingInfoNode.IsValid())
	{
		if (!OutLoggingInfoNode->GetChildValue<FString>("lognote", OutLogNote, ENodeInherit::NoInherit, ENodeReference::NoReferences))
		{
			OutLogNote = TEXT("");
		}
	}

	// Frame rate 
	int32 FrameRateTimebase = 0;
	if (!InClipItemNode->GetChildSubValue<int32>("rate", "timebase", FrameRateTimebase, ENodeInherit::CheckInherit))
	{
		return false;
	}

	bool bFrameRateIsNTSC = false;
	if (!InClipItemNode->GetChildSubValue<bool>("rate", "ntsc", bFrameRateIsNTSC, ENodeInherit::CheckInherit))
	{
		bFrameRateIsNTSC = false;
	}

	// Start frame 
	int32 StartVal = 0;
	if (!InClipItemNode->GetChildValue("start", StartVal, ENodeInherit::NoInherit))
	{
		return false;
	}
	FFrameNumber StartFrame = StartVal;

	// End frame 
	int32 EndVal = 0;
	if (!InClipItemNode->GetChildValue("end", EndVal, ENodeInherit::NoInherit))
	{
		return false;
	}

	// In frame 
	int32 InVal = 0;
	bool bInDefault = false;
	if (!InClipItemNode->GetChildValue("in", InVal, ENodeInherit::NoInherit))
	{
		bInDefault = true;
	}

	// Out frame 
	int32 OutVal = 0;
	bool bOutDefault = false;
	if (!InClipItemNode->GetChildValue("out", OutVal, ENodeInherit::NoInherit))
	{
		bOutDefault = true;
	}

	uint32 Numerator = static_cast<uint32>(FrameRateTimebase);
	OutFrameRate = FFrameRate(Numerator, 1);
	OutStartOffset = bInDefault ? 0 : InVal;
	OutStart = StartVal;
	OutEnd = EndVal;

	TSharedPtr<FFCPXMLNode> LinkNode = InClipItemNode->GetChildNode("file", ENodeInherit::NoInherit, ENodeReference::CheckReferences);
	if (LinkNode.IsValid())
	{
		TSharedPtr<FFCPXMLNode> NameNode = LinkNode->GetChildNode("name", ENodeInherit::NoInherit, ENodeReference::CheckReferences);
		if (NameNode.IsValid())
		{
			NameNode->GetContent(OutFilename);
		}
	}

	return true;
}

int32 FFCPXMLImportVisitor::GetAudioClipItemNodeChannel(TSharedRef<FFCPXMLClipItemNode> InClipItemNode, const FString& InClipItemId)
{
	TSharedPtr<FFCPXMLNode> LinkNode = InClipItemNode->GetChildNode("link", ENodeInherit::NoInherit, ENodeReference::NoReferences);
	if (LinkNode.IsValid())
	{
		TSharedPtr<FFCPXMLNode>LinkClipRefNode = LinkNode->GetChildNode("linkclipref", ENodeInherit::NoInherit, ENodeReference::NoReferences);
		if (LinkClipRefNode.IsValid())
		{
			FString CompareClipItemId;
			LinkClipRefNode->GetContent(CompareClipItemId);
			return (InClipItemId == CompareClipItemId) ? 1 : 2;
		}
	}
	return 1;
}

bool FFCPXMLImportVisitor::ConstructAudioTrackList()
{
	if (!ImportData->MovieSceneData.IsValid())
	{
		return false;
	}

	for (TSharedPtr<FMovieSceneImportAudioData> AudioData : ImportData->MovieSceneData->AudioData)
	{
		for (TSharedPtr<FMovieSceneImportAudioTrackData> AudioTrackData : AudioData->AudioTracks)
		{
			if (AudioTrackData.IsValid())
			{
				TSharedPtr<FFCPXMLImportAudioTrackListItem> ListItem = MakeShared<FFCPXMLImportAudioTrackListItem>(AudioData, AudioTrackData->RowIndex);
				AudioTrackList.Add(ListItem);
			}
		}
	}
	return true;
}

/** Add entry to master clip section name map */
bool FFCPXMLImportVisitor::AddMasterClipCinematicSectionPathName(const FString& InMasterClipIdName, const FString& InSectionPathName)
{
	if (MasterClipCinematicSectionMap.Contains(InMasterClipIdName))
	{
		return false;
	}

	MasterClipCinematicSectionMap.Add(InMasterClipIdName, InSectionPathName);
	return true;
}

/** Query master clip section name map */
bool FFCPXMLImportVisitor::GetMasterClipCinematicSectionPathName(const FString& InMasterClipIdName, FString& OutSectionPathName) const
{
	const FString* Name = MasterClipCinematicSectionMap.Find(InMasterClipIdName);
	if (Name == nullptr)
	{
		return false;
	}

	OutSectionPathName = *Name;
	return true;
}

/** Get cinematic section based on metadata and master clip id */
FString FFCPXMLImportVisitor::GetCinematicSectionPathName(const FString& InMetadata, const FString& InMasterClipId)
{
	if (!InMetadata.IsEmpty())
	{
		FString SectionPathName(TEXT(""));
		if (GetCinematicSectionPathNameFromMetadata(InMetadata, SectionPathName))
		{
			if (!InMasterClipId.IsEmpty())
			{
				AddMasterClipCinematicSectionPathName(InMasterClipId, SectionPathName);
			}
			return SectionPathName;
		}
	}

	if (!InMasterClipId.IsEmpty())
	{
		FString SectionPathName(TEXT(""));
		if (GetMasterClipCinematicSectionPathName(InMasterClipId, SectionPathName))
		{
			return SectionPathName;
		}
	}
	return TEXT("");
}

/** Add entry to master clip section name map */
bool FFCPXMLImportVisitor::AddMasterClipAudioMetadata(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLImportAudioMetadata> InAudioMetadata)
{
	if (MasterClipAudioSectionMap.Contains(InMasterClipIdName))
	{
		return false;
	}

	MasterClipAudioSectionMap.Add(InMasterClipIdName, InAudioMetadata);

	return true;
}

/** Query master clip section name map */
bool FFCPXMLImportVisitor::GetMasterClipAudioMetadata(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLImportAudioMetadata>& OutAudioMetadata) const
{
	const TSharedPtr<FFCPXMLImportAudioMetadata>* AudioMetadata = MasterClipAudioSectionMap.Find(InMasterClipIdName);
	if (AudioMetadata == nullptr)
	{
		return false;
	}

	OutAudioMetadata = *AudioMetadata;
	return true;
}


TSharedPtr<FFCPXMLImportAudioMetadata> FFCPXMLImportVisitor::GetAudioMetadataObject(const FString& InLogNote, const FString& InMasterClipId)
{
	// Premiere exports the masterclip's logging info onto each clipitem node.
	// So check the master clip FIRST and use this audio metadata if it exists.

	TSharedPtr<FFCPXMLImportAudioMetadata> AudioMetadata = nullptr;

	if (!InMasterClipId.IsEmpty())
	{
		GetMasterClipAudioMetadata(InMasterClipId, AudioMetadata);
		if (AudioMetadata.IsValid())
		{
			return AudioMetadata;
		}
	}

	if (!InLogNote.IsEmpty())
	{
		GetAudioFromMetadata(InLogNote, AudioMetadata);

		if (!InMasterClipId.IsEmpty() && AudioMetadata.IsValid())
		{
			AddMasterClipAudioMetadata(InMasterClipId, AudioMetadata);
		}
	}

	return AudioMetadata;
}

/** Get audio section path name based on node metadata and masterclip id */
bool FFCPXMLImportVisitor::GetNextAudioSection(TSharedPtr<FFCPXMLImportAudioMetadata> InAudioMetadata, TSharedPtr<FMovieSceneImportAudioData>& OutAudioData, TSharedPtr<FMovieSceneImportAudioSectionData>& OutAudioSectionData)
{
	if (!InAudioMetadata.IsValid())
	{
		return false;
	}

	// if audio metadata exists, look for matching section
	FString AudioSectionPathName{ TEXT("") };
	for (TSharedPtr<FFCPXMLImportAudioSectionMetadata> Section : InAudioMetadata->AudioSections)
	{
		if (Section.IsValid() && !Section->bAudioSectionUpdated)
		{
			AudioSectionPathName = Section->AudioSectionPathName;
			Section->bAudioSectionUpdated = true;
			break;
		}
	}

	// Find actual audio section
	if (!AudioSectionPathName.IsEmpty())
	{
		OutAudioSectionData = ImportData->FindAudioSection(AudioSectionPathName, OutAudioData);
	}

	return true;
}

bool FFCPXMLImportVisitor::AddMasterClipLoggingNode(const FString& InMasterClipName, TSharedPtr<FFCPXMLNode> InLoggingInfoNode)
{
	if (MasterClipLoggingNodeMap.Contains(InMasterClipName))
	{
		return false;
	}

	MasterClipLoggingNodeMap.Add(InMasterClipName, InLoggingInfoNode);
	return true;
}

bool FFCPXMLImportVisitor::GetMasterClipLoggingNode(const FString& InMasterClipName, TSharedPtr<FFCPXMLNode>& OutLoggingInfoNode) const
{
	const TSharedPtr<FFCPXMLNode>* LoggingNode = MasterClipLoggingNodeMap.Find(InMasterClipName);
	if (LoggingNode == nullptr)
	{
		return false;
	}

	OutLoggingInfoNode = *LoggingNode;
	return true;
}

bool FFCPXMLImportVisitor::SetMetaDataValue(TSharedPtr<FFCPXMLNode> InNode, const FString& InElement, FMetaData& InMetaData, const UMovieSceneSection* InSection) const
{
	if (!InNode.IsValid() || InSection == nullptr)
	{
		return false;
	}

	FString Value;
	InNode->GetChildValue(InElement, Value, ENodeInherit::NoInherit, ENodeReference::NoReferences);
	if (!Value.IsEmpty())
	{
		InMetaData.SetValue(InSection, *InElement, *Value);
	}
	return true;
}

/** Store clip metadata */
bool FFCPXMLImportVisitor::ImportSectionMetaData(const TSharedPtr<FFCPXMLNode>& InLoggingInfoNode, const UMovieSceneSection* InSection) const
{
	if (InLoggingInfoNode.IsValid())
	{
		UPackage* Package = InSection->GetOutermost();
		check(Package);

		FMetaData& MetaData = Package->GetMetaData();

		SetMetaDataValue(InLoggingInfoNode, TEXT("description"), MetaData, InSection);
		SetMetaDataValue(InLoggingInfoNode, TEXT("scene"), MetaData, InSection);
		SetMetaDataValue(InLoggingInfoNode, TEXT("shottake"), MetaData, InSection);
		SetMetaDataValue(InLoggingInfoNode, TEXT("good"), MetaData, InSection);
		SetMetaDataValue(InLoggingInfoNode, TEXT("originalvideofilename"), MetaData, InSection);
		SetMetaDataValue(InLoggingInfoNode, TEXT("originalaudiofilename"), MetaData, InSection);
	}

	return true;
}

/** parse metadata of the format "[key=value]", whitespace ok.  */
bool FFCPXMLImportVisitor::ParseMetadata(const FString& InMetadata, const FString& InKey, FString& OutValue, FString& OutMetadata) const
{
	FString A{ TEXT("") };
	FString B{ TEXT("") };
	FString C{ TEXT("") };

	if (InMetadata.Split(InKey, &A, &B))
	{
		if (B.Split(TEXT("="), &A, &C))
		{
			if (C.Split(TEXT("]"), &A, &OutMetadata))
			{
				OutValue = A.TrimStartAndEnd();
				return true;
			}
		}
	}
	return false;
}

/** parse metadata of the format "[key=value]", whitespace ok.  */
bool FFCPXMLImportVisitor::ParseMetadata(const FString& InMetadata, const FString& InKey, FString& OutValue) const
{
	FString MetadataRemaining{ TEXT("") };
	return ParseMetadata(InMetadata, InKey, OutValue, MetadataRemaining);
}

/** Get sequencer section name from section metadata. Format is "[UEShotSection=sectionobjectname]", whitespace ok. */
bool FFCPXMLImportVisitor::GetCinematicSectionPathNameFromMetadata(const FString& InMetadata, FString& OutSectionObjectName) const
{
	return ParseMetadata(InMetadata, TEXT("UEShotSection"), OutSectionObjectName);
}

/** Get sequencer shot handle frames from section metadata. Format is "[UEShotHandleFrames=handleframes]", whitespace ok. */
bool FFCPXMLImportVisitor::GetCinematicSectionHandleFramesFromMetadata(const FString& InMetadata, int32& OutHandleFrames) const
{
	FString HandleFrameData;
	bool bSuccess = ParseMetadata(InMetadata, TEXT("UEShotHandleFrames"), HandleFrameData);
	if (bSuccess)
	{
		OutHandleFrames = FCString::Atoi(*HandleFrameData);
	}
	return bSuccess;
}

/** Get sequencer shot start offset frame from section metadata. Format is "[UEShotStartOffset=startoffset]", whitespace ok. */
bool FFCPXMLImportVisitor::GetCinematicSectionStartOffsetFromMetadata(const FString& InMetadata, int32& OutStartOffset) const
{
	FString StartOffsetData;
	bool bSuccess = ParseMetadata(InMetadata, TEXT("UEShotStartOffset"), StartOffsetData);
	if (bSuccess)
	{
		OutStartOffset = FCString::Atoi(*StartOffsetData);
	}
	return bSuccess;
}

/** Get sequencer track name from track metadata. Format is "[UETrack=trackobjectname][UERow=rowindex]", whitespace ok. */
bool FFCPXMLImportVisitor::GetAudioFromMetadata(const FString& InMetadata, TSharedPtr<FFCPXMLImportAudioMetadata>& OutAudioMetadata) const
{
	FString Metadata1{ TEXT("") };
	FString SoundWavePathName;
	bool bSuccess = ParseMetadata(InMetadata, TEXT("UESoundWave"), SoundWavePathName, Metadata1);
	OutAudioMetadata = MakeShared<FFCPXMLImportAudioMetadata>(SoundWavePathName);

	if (bSuccess)
	{
		FString Metadata2{ TEXT("") };
		FString AudioSectionTopLevel{ TEXT("") };
		bSuccess = ParseMetadata(Metadata1, TEXT("UEAudioSectionTopLevel"), AudioSectionTopLevel, Metadata2);

		if (bSuccess)
		{
			FString AudioSection{ TEXT("") };
			while (ParseMetadata(Metadata2, TEXT("UEAudioSection"), AudioSection, Metadata1))
			{
				FString AudioSectionPathName = AudioSectionTopLevel + TEXT(".") + AudioSection;
				TSharedPtr<FFCPXMLImportAudioSectionMetadata> AudioSectionMetadata = MakeShared<FFCPXMLImportAudioSectionMetadata>(AudioSectionPathName);
				OutAudioMetadata->AudioSections.Add(AudioSectionMetadata);
				Metadata2 = Metadata1;
			}
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
