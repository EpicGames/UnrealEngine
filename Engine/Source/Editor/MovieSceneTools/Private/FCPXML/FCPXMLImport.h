// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FCPXML/FCPXMLNode.h"
#include "MovieSceneTranslator.h"
#include "MovieScene.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"

/** FCP XML node visitor classes. */

struct FFCPXMLImportAudioTrackListItem
{
	FFCPXMLImportAudioTrackListItem(TSharedPtr<FMovieSceneImportAudioData> InAudioTrackData, int InRowIndex) : AudioTrackData(InAudioTrackData), RowIndex(InRowIndex) {}

	TSharedPtr<FMovieSceneImportAudioData> AudioTrackData;
	int32 RowIndex;
};

struct FFCPXMLImportAudioSectionMetadata
{
	FFCPXMLImportAudioSectionMetadata(const FString& InAudioSectionPathName) : AudioSectionPathName(InAudioSectionPathName), bAudioSectionUpdated(false) {}

	FString AudioSectionPathName;
	bool bAudioSectionUpdated;
};

struct FFCPXMLImportAudioMetadata
{
	FFCPXMLImportAudioMetadata(const FString& InSoundPathName) : SoundPathName(InSoundPathName) {}
	FFCPXMLImportAudioMetadata() : SoundPathName(TEXT("")) {}

	FString SoundPathName;
	TArray< TSharedPtr<FFCPXMLImportAudioSectionMetadata> > AudioSections;
};

/** The FFCPXMLImportVisitor class imports from the FCP 7 XML structure into Sequencer classes. */

class FFCPXMLImportVisitor : public FFCPXMLNodeVisitor
{
public:
	/** Constructor */
	FFCPXMLImportVisitor(TSharedRef<FMovieSceneImportData> InImportData, TSharedRef<FMovieSceneTranslatorContext> InImportContext);
	/** Destructor */
	~FFCPXMLImportVisitor();

public:
	/** Called when visiting a FFCPXMLBasicNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLBasicNode> InBasicNode) override final;
	/** Called when visiting a FFCPXMLMemlNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLXmemlNode> InXmemlNode) override final;
	/** Called when visiting a FFCPXMLSequenceNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLSequenceNode> InSequenceNode) override final;
	/** Called when visiting a FFCPXMLVideoNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLVideoNode> InVideoNode) override final;
	/** Called when visiting a FFCPXMLAudioNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLAudioNode> InAudioNode) override final;
	/** Called when visiting a FFCPXMLTrackNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLTrackNode> InTrackNode) override final;
	/** Called when visiting a FFCPXMLClipNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipNode> InClipNode) override final;
	/** Called when visiting a FFCPXMLClipItemNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode) override final;
	/** Called when visiting a FFCPXMLFileNode during visitor traversal. */
	virtual bool VisitNode(TSharedRef<FFCPXMLFileNode> InFileNode) override final;

private:
	/** Called when visiting a video FFCPXMLClipItemNode during visitor traversal. */
	bool VisitVideoClipItemNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode);
	/** Called when visiting an audio FFCPXMLClipItemNode during visitor traversal. */
	bool VisitAudioClipItemNode(TSharedRef<FFCPXMLClipItemNode> InClipItemNode);
	/** Get clipitem node data used by both video and audio clipitems */
	bool GetClipItemNodeData(TSharedRef<FFCPXMLClipItemNode> InClipItemNode, FString& OutClipItemName, FString& OutClipItemId, FString& OutMasterClipId, TSharedPtr<FFCPXMLNode>& OutLoggingInfoNode, 
							 FString& OutLogNote, FString& OutFilename, FFrameRate& OutFrameRate, FFrameNumber& StartOffset, FFrameNumber& Start, FFrameNumber &End);
	/** Query which channel audio clipitem node represents */
	int32 GetAudioClipItemNodeChannel(TSharedRef<FFCPXMLClipItemNode> InClipItemNode, const FString& InClipItemId);

	/** Construct list of audio tracks and row indexes */
	bool ConstructAudioTrackList();

	/** Add entry to master cinematic clip section name map */
	bool AddMasterClipCinematicSectionPathName(const FString& InMasterClipIdName, const FString& InSectionPathName);
	/** Query master cinematic clip section name map */
	bool GetMasterClipCinematicSectionPathName(const FString& InMasterClipIdName, FString& OutSectionPathName) const;
	/** Get cinematic section path name based on node metadata and masterclip id */
	FString GetCinematicSectionPathName(const FString& InMetadata, const FString& InMasterClipId);

	/** Add entry to master clip audio section map */
	bool AddMasterClipAudioMetadata(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLImportAudioMetadata> InAudioMetadata);
	/** Query master clip audio section map */
	bool GetMasterClipAudioMetadata(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLImportAudioMetadata>& OutAudioMetadata) const;
	/** Get audio metadata object based on log note and master clip id */
	TSharedPtr<FFCPXMLImportAudioMetadata> GetAudioMetadataObject(const FString& InLogNote, const FString& InMasterClipId);
	/** Get next audio section based on audio metadata */
	bool GetNextAudioSection(TSharedPtr<FFCPXMLImportAudioMetadata> InAudioMetadata, TSharedPtr<FMovieSceneImportAudioData>& OutAudioData, TSharedPtr<FMovieSceneImportAudioSectionData>& OutAudioSectionData);

	/** Add entry to metadata map */
	bool AddMasterClipLoggingNode(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLNode> InLoggingInfoNode);
	/** Query master clip logging node */
	bool GetMasterClipLoggingNode(const FString& InMasterClipIdName, TSharedPtr<FFCPXMLNode>& InLoggingInfoNode) const;

	/** Helper method */
	bool SetMetaDataValue(TSharedPtr<FFCPXMLNode> InNode, const FString& InElement, FMetaData& InMetaData, const UMovieSceneSection* InSection) const;
	/** Import logging info metadata to sequencer section */
	bool ImportSectionMetaData(const TSharedPtr<FFCPXMLNode>& InLoggingInfoNode, const UMovieSceneSection* InSection) const;

	/** parse metadata of the format "[tag=value]", whitespace ok. */
	bool ParseMetadata(const FString& InMetadata, const FString& InKey, FString& OutValue, FString& OutMetadata) const;
	/** parse metadata of the format "[tag=value]", whitespace ok. */
	bool ParseMetadata(const FString& InMetadata, const FString& InKey, FString& OutValue) const;
	/** Get sequencer section id from section metadata. Format is "[UEShotSection=sectionobjectname]", whitespace ok. */
	bool GetCinematicSectionPathNameFromMetadata(const FString& InMetadata, FString& OutSectionObjectName) const;
	/** Get sequencer shot handle frames from section metadata. Format is "[UEShotHandleFrames=handleframes]", whitespace ok. */
	bool GetCinematicSectionHandleFramesFromMetadata(const FString& InMetadata, int32& OutHandleFrames) const;
	/** Get sequencer shot start offset frame from section metadata. Format is "[UEShotStartOffset=startoffset]", whitespace ok. */
	bool GetCinematicSectionStartOffsetFromMetadata(const FString& InMetadata, int32& OutStartOffset) const;
	/** Get sequencer sound wave id and audio section ids from metadata. Format is "[UESoundWave=trackobjectname][UEAudioSectionTopLevel=toplevelobjectname][UEAudioSection=audiosectionobjectname]", whitespace ok. */
	bool GetAudioFromMetadata(const FString& InMetadata, TSharedPtr<FFCPXMLImportAudioMetadata>& OutAudioMetadata) const;
	
public:

	int32 GetMaxVideoTrackRowIndex() const { return MaxVideoTrackRowIndex; }
	int32 GetMaxAudioTrackRowIndex() const { return MaxAudioTrackRowIndex; }

private:
	TSharedRef<FMovieSceneImportData> ImportData;
	TSharedRef<FMovieSceneTranslatorContext> ImportContext;

	/** Map of masterclip names to sequencer section id names. */
	TMap<FString, FString> MasterClipCinematicSectionMap;

	/** Map of masterclip names to sequencer audio metadata */
	TMap<FString, TSharedPtr<FFCPXMLImportAudioMetadata>> MasterClipAudioSectionMap;

	/** Map of masterclip names to logging info nodes */
	TMap<FString, TSharedPtr<FFCPXMLNode>> MasterClipLoggingNodeMap;

	/** List of existing audio tracks including row index */
	TArray< TSharedPtr<FFCPXMLImportAudioTrackListItem> > AudioTrackList;

	/** Traversal state variables */
	bool bInSequenceNode;
	bool bInVideoNode;
	bool bInAudioNode;
	bool bInVideoTrackNode;
	bool bInAudioTrackNode;

	int32 CurrVideoTrackRowIndex;
	int32 CurrAudioTrackListIndex;
	TSharedPtr<FMovieSceneImportAudioData> CurrAudioData;
	int32 CurrAudioTrackRowIndex;
	bool bCurrImportAudioTrackIsStereoChannel;

	int32 MaxVideoTrackRowIndex;
	int32 MaxAudioTrackRowIndex;
};

