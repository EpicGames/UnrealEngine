// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLMetadataExport.h"
#include "MovieScene.h"

DEFINE_LOG_CATEGORY_STATIC(LogFCPXMLMetadataExporter, Log, All);

FFCPXMLMetadataExportVisitor::FFCPXMLMetadataExportVisitor(FString InSaveFilename, TSharedRef<FMovieSceneExportData> InExportData, TSharedRef<FMovieSceneTranslatorContext> InExportContext, const FMovieSceneExportMetadata* InMovieSceneExportMetadata) :
	FFCPXMLExportVisitor(InSaveFilename, InExportData, InExportContext),
	MovieSceneExportMetadata(InMovieSceneExportMetadata)
{
}

FFCPXMLMetadataExportVisitor::~FFCPXMLMetadataExportVisitor() {}

// Return which of the exported clip formats is preferred for use in the project file
FString GetPreferredFormat(const TArray<FString>& Extensions)
{
	// Format extensions in order of preference
	const TArray<FString> PreferredFormats = {
		TEXT("MXF"),
		TEXT("MOV"),
		TEXT("AVI"),
		TEXT("EXR"),
		TEXT("PNG"),
		TEXT("JPEG"),
		TEXT("JPG")
	};

	for (const FString& Format : PreferredFormats)
	{
		if (Extensions.Contains(Format))
		{
			return Format;
		}
	}

	return Extensions[0]; // If we didn't find a preferred format, use the first one
}

bool FFCPXMLMetadataExportVisitor::ConstructMasterVideoClipNodes(TSharedRef<FFCPXMLNode> InParentNode)
{
	if (!ExportData->IsExportDataValid() || !ExportData->MovieSceneData.IsValid() || !ExportData->MovieSceneData->CinematicData.IsValid())
	{
		return false;
	}

	if (!MovieSceneExportMetadata)
	{
		return false;
	}


	for (const FMovieSceneExportMetadataShot& Shot : MovieSceneExportMetadata->Shots)
	{
		for (const TPair < FString, TMap<FString, FMovieSceneExportMetadataClip> >& Clip : Shot.Clips)
		{
			const TMap<FString, FMovieSceneExportMetadataClip>& ExtensionList = Clip.Value;
			if (ExtensionList.Num() > 0)
			{
				TArray<FString> Extensions;
				ExtensionList.GetKeys(Extensions);
				FString PreferredFormat = GetPreferredFormat(Extensions);
				check(ExtensionList.Contains(PreferredFormat));
				const FMovieSceneExportMetadataClip& ClipMetadata = ExtensionList[PreferredFormat];
				if (ClipMetadata.IsValid())
				{
					if (!ConstructMasterClipNode(InParentNode, Clip.Key, Shot, ClipMetadata, Shot.HandleFrames))
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}


bool FFCPXMLMetadataExportVisitor::ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InClipName, const FMovieSceneExportMetadataShot& InShotMetadata, const FMovieSceneExportMetadataClip& InClipMetadata, const int32 HandleFrames)
{
	int32 Duration = InClipMetadata.GetDuration();
	int32 StartFrame = InClipMetadata.StartFrame;

	//
	// EndFrame + 1: The clip meta data's end frame is the frame that is actually output. It is not exclusive. FCPXML is exclusive. 
	// For example, a 50 frame duration movie that starts at frame 0 should have an end frame value of 50. 
	// See corresponding note in FCPXMLExport.cpp GetCinematicSectionFrames which constructs the end frame from 
	// the section's exclusive end frame.
	// 
	int32 EndFrame = InClipMetadata.EndFrame + 1;
	int32 InFrame = HandleFrames;
	int32 OutFrame = InFrame + Duration;
	FString SectionName = InClipName;
	
	/** Construct a master clip id name based on the cinematic section and id */
	FString MasterClipName{ TEXT("") };
	GetMasterClipIdName(InClipName, MasterClipName);

	TSharedRef<FFCPXMLNode> ClipNode = InParentNode->CreateChildNode(TEXT("clip"));
	ClipNode->AddAttribute(TEXT("id"), MasterClipName);

	// @todo add to file's masterclip and refidmap HERE

	ClipNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipName);
	ClipNode->CreateChildNode(TEXT("ismasterclip"))->SetContent(true);
	ClipNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);

	if (!ConstructRateNode(ClipNode))
	{
		return false;
	}

	ClipNode->CreateChildNode(TEXT("in"))->SetContent(InFrame);
	ClipNode->CreateChildNode(TEXT("out"))->SetContent(OutFrame);
	ClipNode->CreateChildNode(TEXT("name"))->SetContent(SectionName);
	TSharedRef<FFCPXMLNode> MediaNode = ClipNode->CreateChildNode(TEXT("media"));
	TSharedRef<FFCPXMLNode> VideoNode = MediaNode->CreateChildNode(TEXT("video"));
	TSharedRef<FFCPXMLNode> TrackNode = VideoNode->CreateChildNode(TEXT("track"));

	if (!ConstructVideoClipItemNode(TrackNode, SectionName, InClipMetadata, HandleFrames, true))
	{
		return false;
	}

	UMovieSceneCinematicShotSection* ShotSection = InShotMetadata.MovieSceneShotSection.Get();
	if (ShotSection)
	{
		if (!ConstructLoggingInfoNode(ClipNode, ShotSection))
		{
			return false;
		}
	}

	if (!ConstructColorInfoNode(ClipNode))
	{
		return false;
	}

	return true;
}

bool FFCPXMLMetadataExportVisitor::ConstructVideoClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InClipName, const FMovieSceneExportMetadataClip& InClipMetadata, const int32 HandleFrames, bool bInMasterClip)
{

	TSharedRef<FFCPXMLNode> ClipItemNode = InParentNode->CreateChildNode(TEXT("clipitem"));

	int32 Duration = InClipMetadata.GetDuration();
	int32 StartFrame = InClipMetadata.StartFrame;

	//
	// EndFrame + 1: The clip meta data's end frame is the frame that is actually output. It is not exclusive. FCPXML is exclusive. 
	// For example, a 50 frame duration movie that starts at frame 0 should have an end frame value of 50. 
	// See corresponding note in FCPXMLExport.cpp GetCinematicSectionFrames which constructs the end frame from 
	// the section's exclusive end frame.
	// 
	int32 EndFrame = InClipMetadata.EndFrame + 1;
	int32 InFrame = HandleFrames;
	int32 OutFrame = InFrame + Duration;

	FString MasterClipIdName = TEXT("");
	GetMasterClipIdName(InClipName, MasterClipIdName);

	FString ClipItemIdName{ TEXT("") };
	GetNextClipItemIdName(ClipItemIdName);

	// attributes
	ClipItemNode->AddAttribute(TEXT("id"), ClipItemIdName);

	// elements
	ClipItemNode->CreateChildNode(TEXT("masterclipid"))->SetContent(MasterClipIdName);
	ClipItemNode->CreateChildNode(TEXT("ismasterclip"))->SetContent(bInMasterClip);
	ClipItemNode->CreateChildNode(TEXT("name"))->SetContent(InClipName);
	ClipItemNode->CreateChildNode(TEXT("enabled"))->SetContent(true);
	ClipItemNode->CreateChildNode(TEXT("duration"))->SetContent(Duration);

	if (!ConstructRateNode(ClipItemNode))
	{
		return false;
	}

	if (!bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("start"))->SetContent(StartFrame);
		ClipItemNode->CreateChildNode(TEXT("end"))->SetContent(EndFrame);
	}

	ClipItemNode->CreateChildNode(TEXT("in"))->SetContent(InFrame);
	ClipItemNode->CreateChildNode(TEXT("out"))->SetContent(OutFrame);

	if (bInMasterClip)
	{
		ClipItemNode->CreateChildNode(TEXT("anamorphic"))->SetContent(false);
		ClipItemNode->CreateChildNode(TEXT("pixelaspectratio"))->SetContent(FString(TEXT("square")));
		ClipItemNode->CreateChildNode(TEXT("fielddominance"))->SetContent(FString(TEXT("lower")));
	}

	if (!ConstructVideoFileNode(ClipItemNode, InClipMetadata.FileName, Duration, bInMasterClip))
	{
		return false;
	}

	return true;
}

bool FFCPXMLMetadataExportVisitor::GetMasterClipIdName(const FString& InClipName, FString& OutName)
{
	if (MasterClipIdMap.Num() > 0)
	{
		uint32* FoundId = MasterClipIdMap.Find(InClipName);
		if (FoundId != nullptr)
		{
			OutName = FString::Printf(TEXT("masterclip-%d"), *FoundId);
			return true;
		}
	}

	++MasterClipId;
	MasterClipIdMap.Add(InClipName, MasterClipId);
	OutName = FString::Printf(TEXT("masterclip-%d"), MasterClipId);

	return true;
}

bool FFCPXMLMetadataExportVisitor::GetFileIdName(const FString& InFileName, FString& OutFileIdName, bool& OutFileExists)
{
	if (FileIdMap.Num() > 0)
	{
		uint32* FoundFileId = FileIdMap.Find(InFileName);
		if (FoundFileId != nullptr)
		{
			OutFileIdName = FString::Printf(TEXT("file-%d"), *FoundFileId);
			OutFileExists = true;
			return true;
		}
	}

	++FileId;
	FileIdMap.Add(InFileName, FileId);
	OutFileIdName = FString::Printf(TEXT("file-%d"), FileId);
	OutFileExists = false;
	return true;
}

bool FFCPXMLMetadataExportVisitor::ConstructVideoTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicTrackData> InCinematicTrackData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData)
{
	if (!ExportData->IsExportDataValid() || !InCinematicTrackData.IsValid())
	{
		return false;
	}

	TSharedRef<FFCPXMLNode> TrackNode = InParentNode->CreateChildNode(TEXT("track"));
	
	for (TSharedPtr<FMovieSceneExportCinematicSectionData> CinematicSection : InCinematicTrackData->CinematicSections)
	{
		for (const FMovieSceneExportMetadataShot& Shot : MovieSceneExportMetadata->Shots)
		{
			UMovieSceneCinematicShotSection* ShotSection = Shot.MovieSceneShotSection.Get();
			if (ShotSection && ShotSection == CinematicSection->MovieSceneSection)
			{
				for (const TPair < FString, TMap<FString, FMovieSceneExportMetadataClip> >& Clip : Shot.Clips)
				{
					const TMap<FString, FMovieSceneExportMetadataClip>& ExtensionList = Clip.Value;
					if (ExtensionList.Num() > 0)
					{
						TArray<FString> Extensions;
						ExtensionList.GetKeys(Extensions);
						FString PreferredFormat = GetPreferredFormat(Extensions);
						check(ExtensionList.Contains(PreferredFormat));
						const FMovieSceneExportMetadataClip& ClipMetadata = ExtensionList[PreferredFormat];
						if (ClipMetadata.IsValid())
						{
							if (!ConstructVideoClipItemNode(TrackNode, Clip.Key, ClipMetadata, Shot.HandleFrames, false))
							{
								return false;
							}
						}
					}
				}
				break;
			}
		}
	}

	TSharedRef<FFCPXMLNode> EnabledNode = TrackNode->CreateChildNode(TEXT("enabled"));
	EnabledNode->SetContent(true);

	TSharedRef<FFCPXMLNode> LockedNode = TrackNode->CreateChildNode(TEXT("locked"));
	LockedNode->SetContent(false);

	return true;
}

bool FFCPXMLMetadataExportVisitor::ConstructVideoFileNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InFileName, int32 Duration, bool bInMasterClip)
{
	if (!ExportData->IsExportDataValid())
	{
		return false;
	}

	FString FileIdName{ TEXT("") };
	bool bFileExists = false;
	GetFileIdName(InFileName, FileIdName, bFileExists);

	// attributes
	TSharedRef<FFCPXMLNode> FileNode = InParentNode->CreateChildNode(TEXT("file"));
	FileNode->AddAttribute(TEXT("id"), FileIdName);

	if (!bFileExists)
	{
		FString FilePathName = SaveFilePath + TEXT("/") + InFileName;
		FString FilePathUrl = FString(TEXT("file://localhost/")) + FilePathName.Replace(TEXT(" "), TEXT("%20")).Replace(TEXT(":"), TEXT("%3a"));

		// required elements
		TSharedRef<FFCPXMLNode> NameNode = FileNode->CreateChildNode(TEXT("name"));
		NameNode->SetContent(InFileName);

		TSharedRef<FFCPXMLNode> PathUrlNode = FileNode->CreateChildNode(TEXT("pathurl"));
		PathUrlNode->SetContent(FilePathUrl);

		if (!ConstructRateNode(FileNode))
		{
			return false;
		}

		TSharedRef<FFCPXMLNode> DurationNode = FileNode->CreateChildNode(TEXT("duration"));
		DurationNode->SetContent(static_cast<int32>(Duration));

		if (!ConstructTimecodeNode(FileNode))
		{
			return false;
		}

		TSharedRef<FFCPXMLNode> MediaNode = FileNode->CreateChildNode(TEXT("media"));
		TSharedRef<FFCPXMLNode> VideoNode = MediaNode->CreateChildNode(TEXT("video"));

		if (!ConstructVideoSampleCharacteristicsNode(VideoNode, ExportData->GetResX(), ExportData->GetResY()))
		{
			return false;
		}
	}

	return true;
}
