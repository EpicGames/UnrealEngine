// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FCPXML/FCPXMLExport.h"
#include "MovieSceneExportMetadata.h"

/** The FFCPXMLMetadataExportVisitor class exports FCP 7 XML structure based on metadata from a sequencer movie export
 */

class FFCPXMLMetadataExportVisitor : public FFCPXMLExportVisitor
{
public:
	/** Constructor */
	FFCPXMLMetadataExportVisitor(FString InSaveFilename, TSharedRef<FMovieSceneExportData> InExportData, TSharedRef<FMovieSceneTranslatorContext> InExportContext, const FMovieSceneExportMetadata* InMovieSceneExportMetadata);
	/** Destructor */
	virtual ~FFCPXMLMetadataExportVisitor();

	/** Creates master video clip nodes. */
	virtual bool ConstructMasterVideoClipNodes(TSharedRef<FFCPXMLNode> InParentNode) override;
	virtual bool ConstructVideoTrackNode(TSharedRef<FFCPXMLNode> InParentNode, const TSharedPtr<FMovieSceneExportCinematicTrackData> InCinematicTrackData, const TSharedPtr<FMovieSceneExportCinematicData> InCinematicData) override;

	bool ConstructMasterClipNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InClipName, const FMovieSceneExportMetadataShot& InShotMetadata, const FMovieSceneExportMetadataClip& InClipMetadata, const int32 HandleFrames);
	bool ConstructVideoClipItemNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InClipName, const FMovieSceneExportMetadataClip& InClipMetadata, const int32 HandleFrames, bool bInMasterClip);
	bool GetMasterClipIdName(const FString& InClipName, FString& OutName);
	bool GetFileIdName(const FString& InFileName, FString& OutFileIdName, bool& OutFileExists);
	bool ConstructVideoFileNode(TSharedRef<FFCPXMLNode> InParentNode, const FString& InClipName, int32 Duration, bool bInMasterClip);

protected:
	const FMovieSceneExportMetadata* MovieSceneExportMetadata;

};
