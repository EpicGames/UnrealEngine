// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLMovieSceneTranslator.h"
#include "FCPXML/FCPXMLFile.h"
#include "FCPXML/FCPXMLImport.h"
#include "FCPXML/FCPXMLExport.h"
#include "FCPXML/FCPXMLMetadataExport.h"
#include "MovieScene.h"
#include "MovieSceneTranslator.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "XmlParser.h"

#define LOCTEXT_NAMESPACE "FCPXMLMovieSceneTranslator"

/* MovieSceneCapture FCP 7 XML Importer
*****************************************************************************/

FFCPXMLImporter::FFCPXMLImporter()
	: FMovieSceneImporter()
{
}

FFCPXMLImporter::~FFCPXMLImporter()
{
}

FText FFCPXMLImporter::GetFileTypeDescription() const
{
	return FText::FromString(TEXT("Final Cut Pro 7 XML (*.xml)|*.xml|"));
}

FText FFCPXMLImporter::GetDialogTitle() const
{
	return LOCTEXT("ImportFCPXML", "Import FCP 7 XML from...");
}

FText FFCPXMLImporter::GetTransactionDescription() const
{
	return LOCTEXT("ImportFCPXMLTransaction", "Import FCP 7 XML");
}

FName FFCPXMLImporter::GetMessageLogWindowTitle() const
{
	return FName(TEXT("Final Cut Pro 7 XML Import"));
}

FText FFCPXMLImporter::GetMessageLogLabel() const
{
	return LOCTEXT("FCPXMLImportLogLabel", "FCP 7 XML Import Log");
}

bool FFCPXMLImporter::Import(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InFilename, TSharedRef<FMovieSceneTranslatorContext> InContext)
{
	// Create intermediate structure to assist with import
	TSharedRef<FMovieSceneImportData> ImportData = MakeShared<FMovieSceneImportData>(InMovieScene, InContext);
	if (!ImportData->IsImportDataValid())
	{
		return false;
	}

	// Load file to string
	FString InString;
	if (!FFileHelper::LoadFileToString(InString, *InFilename))
	{
		return false;
	}

	// Construct XML from file string
	TSharedRef<FFCPXMLFile> FCPXMLFile = MakeShared<FFCPXMLFile>();
	bool bSuccess = FCPXMLFile->LoadFile(InString, EConstructMethod::ConstructFromBuffer);

	if (bSuccess && FCPXMLFile->IsValidFile())
	{
		// Import the loaded Xml structure into the Sequencer movie scene
		FFCPXMLImportVisitor ImportVisitor(ImportData, InContext);
		bSuccess = FCPXMLFile->Accept(ImportVisitor);

		// Flip rows
		if (ImportData->GetCinematicData(false).IsValid())
		{
			int32 MaxVideoTrackRowIndex = ImportVisitor.GetMaxVideoTrackRowIndex();
			for (TSharedPtr<FMovieSceneImportCinematicSectionData> SectionData : ImportData->GetCinematicData(false)->CinematicSections)
			{
				if (SectionData.IsValid())
				{
					UMovieSceneCinematicShotSection* CinematicSection = SectionData.Get()->CinematicSection;
					if (CinematicSection)
					{
						CinematicSection->SetRowIndex(MaxVideoTrackRowIndex - CinematicSection->GetRowIndex());
					}
				}
			}
		}
		if (ImportData->GetAudioData().IsValid())
		{
			int32 MaxAudioTrackRowIndex = ImportVisitor.GetMaxAudioTrackRowIndex();
			for (TSharedPtr<FMovieSceneImportAudioSectionData> SectionData : ImportData->GetAudioData()->AudioSections)
			{
				if (SectionData.IsValid())
				{
					UMovieSceneAudioSection* AudioSection = SectionData.Get()->AudioSection;
					if (AudioSection)
					{
						AudioSection->SetRowIndex(MaxAudioTrackRowIndex - AudioSection->GetRowIndex());
					}
				}
			}
		}
	}

	// add error message if one does not exist in the context
	if (!bSuccess && !InContext->ContainsMessageType(EMessageSeverity::Error))
	{
		FText ErrorMessage = FCPXMLFile->GetLastError();
		if (ErrorMessage.IsEmptyOrWhitespace())
		{
			ErrorMessage = LOCTEXT("FCPXMLImportGenericError", "Generic error occurred importing Final Cut Pro 7 XML file.");
		}
		InContext->AddMessage(EMessageSeverity::Error, ErrorMessage);
	}

	return bSuccess;
}

/* MovieSceneCapture FCP 7 XML Exporter
*****************************************************************************/

FFCPXMLExporter::FFCPXMLExporter()
	: FMovieSceneExporter()
{
}

FFCPXMLExporter::~FFCPXMLExporter()
{
}

FText FFCPXMLExporter::GetFileTypeDescription() const
{
	return FText::FromString(TEXT("Final Cut Pro 7 XML (*.xml)|*.xml|"));
}

FText FFCPXMLExporter::GetDialogTitle() const
{
	return LOCTEXT("ExportFCPXML", "Export FCP 7 XML to...");
}

FText FFCPXMLExporter::GetDefaultFileExtension() const
{
	return FText::FromString(TEXT("xml"));
}

FText FFCPXMLExporter::GetNotificationExportFinished() const
{
	return LOCTEXT("FCPXMLExportFinished", "FCP 7 XML Export finished");
}

FText FFCPXMLExporter::GetNotificationHyperlinkText() const
{
	return LOCTEXT("OpenFCPXMLExportFolder", "Open FCP 7 XML Export Folder...");
}

FName FFCPXMLExporter::GetMessageLogWindowTitle() const
{
	return FName(TEXT("Final Cut Pro 7 XML Export"));
}

FText FFCPXMLExporter::GetMessageLogLabel() const
{
	return LOCTEXT("FCPXMLExportLogLabel", "FCP 7 XML Export Log");
}

bool FFCPXMLExporter::Export(const UMovieScene* InMovieScene, FString InFilenameFormat, FFrameRate InFrameRate, uint32 InResX, uint32 InResY, int32 InHandleFrames, FString InSaveFilename, TSharedRef<FMovieSceneTranslatorContext> InContext, FString InMovieExtension, const FMovieSceneExportMetadata* InMetadata)
{
	// add warning message if filename format is not "{shot_name}"
	FString AcceptedFormat = TEXT("{shot_name}");
	if (InFilenameFormat != AcceptedFormat)
	{
		InContext->AddMessage(EMessageSeverity::Warning,
			FText::Format(LOCTEXT("FCPXMLExportFilenameFormatWarning", "FCP 7 XML export expects rendered video sequences with filename format '{0}' but most recent rendered filename format is '{1}'."),
				FText::FromString(AcceptedFormat),
				FText::FromString(InFilenameFormat)));
	}

	// Construct XML from file string
	TSharedRef<FFCPXMLFile> FCPXMLFile = MakeShared<FFCPXMLFile>();
	FCPXMLFile->ConstructFile(FPaths::GetBaseFilename(InSaveFilename, true));

	TSharedRef<FMovieSceneExportData> ExportData = MakeShared<FMovieSceneExportData>(InMovieScene, InFrameRate, InResX, InResY, InHandleFrames, InSaveFilename, InContext, InMovieExtension);

	// Export sequencer movie scene, merging with existing Xml structure.
	FFCPXMLExportVisitor* ExportVisitor;

	if (InMetadata)
	{
		ExportVisitor = new FFCPXMLMetadataExportVisitor(InSaveFilename, ExportData, InContext, InMetadata);
	}
	else
	{
		ExportVisitor = new FFCPXMLExportVisitor(InSaveFilename, ExportData, InContext);
	}

	bool bSuccess = FCPXMLFile->Accept(*ExportVisitor);
	if (bSuccess && FCPXMLFile->IsValidFile())
	{
		// Save the Xml structure to a file
		bSuccess = FCPXMLFile->Save(InSaveFilename);
	}

	// add error message if one was not created during the visitor traversal
	if (!bSuccess && !InContext->ContainsMessageType(EMessageSeverity::Error))
	{
		FText Message = FCPXMLFile->GetLastError();
		if (Message.IsEmptyOrWhitespace())
		{
			Message = LOCTEXT("FCPXMLExportGenericError", "Generic error occurred exporting Final Cut Pro 7 XML file.");
		}

		InContext->AddMessage(EMessageSeverity::Error, Message);
	}

	delete ExportVisitor;

	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
