// Copyright Epic Games, Inc. All Rights Reserved.

#include "FCPXML/FCPXMLNode.h"

#include "FCPXML/FCPXMLFile.h"
#include "HAL/PlatformCrt.h"
#include "Misc/CString.h"
#include "XmlNode.h"

#define LOCTEXT_NAMESPACE "FCPXMLImporter"
// DEFINE_LOG_CATEGORY_STATIC(LogStaticMeshEditorTools, Log, All);

namespace {

	/** Converts FString to float, returns false if string is not a float */
	bool FCPXMLStringToFloat(const FString& InValue, float& OutValue)
	{
		if (InValue.IsNumeric())
		{
			OutValue = FCString::Atof(*InValue);
			return true;
		}
		return false;
	}

	/** Converts FString to int32, returns false if string is not an int */
	bool FCPXMLStringToInt(const FString& InValue, int32& OutValue)
	{
		int32 Index;
		bool bIsInteger = InValue.IsNumeric() && !InValue.FindChar('.', Index);
		if (bIsInteger)
		{
			OutValue = FCString::Atoi(*InValue);
			return true;
		}

		return false;
	}

	/** Converts FString to bool, returns false if string is not a bool */
	bool FCPXMLStringToBool(const FString& InValue, bool& OutValue)
	{
		if (InValue == FString(TEXT("TRUE")))
		{
			OutValue = true;
			return true;
		}
		else if (InValue == FString(TEXT("FALSE")))
		{
			OutValue = false;
			return true;
		}

		return false;
	}
}

/** Gets the tag of the attribute */
const FString& FFCPXMLAttribute::GetTag() const
{
	return Tag;
}

/** Gets the value of the attribute */
const FString& FFCPXMLAttribute::GetValue() const
{
	return Value;
}

bool FFCPXMLAttribute::GetValue(FString& OutValue) const
{
	OutValue = Value;
	return true;
}

bool FFCPXMLAttribute::GetValue(float& OutValue) const
{
	return FCPXMLStringToFloat(Value, OutValue);
}

bool FFCPXMLAttribute::GetValue(int32& OutValue) const
{
	return FCPXMLStringToInt(Value, OutValue);
}

bool FFCPXMLAttribute::GetValue(bool& OutValue) const
{
	return FCPXMLStringToBool(Value, OutValue);
}

/** Constructor */
FFCPXMLNode::FFCPXMLNode(const FString InTag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile)
{
	Tag = InTag;
	Parent = InParent;
	ContainingFile = InFile;
}

/** Calls the appropriate visit method for a given visitor node. */
bool FFCPXMLNode::Accept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return DoAccept(NodeVisitor);
}

/** Calls the visitor for each of the node's children, or its reference node's children if applicable. */
bool FFCPXMLNode::VisitChildren(FFCPXMLNodeVisitor& NodeVisitor, bool VisitRefNodeChildren)
{
	for (TSharedPtr<FFCPXMLNode> Child : Children)
	{
		if (!Child->Accept(NodeVisitor)) 
		{ 
			return false; 
		}
	}

	if (VisitRefNodeChildren)
	{
		TSharedPtr<FFCPXMLNode> RefNode = GetReferenceNode();
		if (RefNode.IsValid())
		{ 
			for (TSharedPtr<FFCPXMLNode> RefChild : RefNode->Children)
			{
				// If current node does not have an element that exists 
				// in the ref node, traverse the ref node element.
				if (!GetChildNodeOnly(RefChild->Tag).IsValid())
				{
					if (!RefChild->Accept(NodeVisitor))
					{
						return false;
					}
				}
			}
		}
	}
	return true;
}

/** Recursive copy of data from XmlNode to this node */
void FFCPXMLNode::CopyFrom(FXmlNode* InNode)
{
	Tag = InNode->GetTag();
	Content = InNode->GetContent();

	TArray<FXmlAttribute> InAttributes = InNode->GetAttributes();
	for (const FXmlAttribute& Attr : InAttributes)
	{
		Attributes.Push(FFCPXMLAttribute(Attr.GetTag(), Attr.GetValue()));
	}

	TArray<FXmlNode*> InChildren = InNode->GetChildrenNodes();
	for (FXmlNode* Child : InChildren)
	{
		if (Child != nullptr)
		{
			TSharedPtr<FFCPXMLFile> FilePtr = ContainingFile.Pin();
			TSharedRef<FFCPXMLNode> ChildNode = FFCPXMLNode::CreateFFCPXMLNode(Child->GetTag(), AsShared(), FilePtr);
			ChildNode->CopyFrom(Child);
			Children.Push(ChildNode);
		}
	}
}

/** Constructs the Xml Buffer representing this node object */
void FFCPXMLNode::GetXmlBuffer(const FString& Indent, FString& Output) const
{
	// Write the tag
	Output += Indent + FString::Printf(TEXT("<%s"), *Tag);
	for (FFCPXMLAttribute Attribute : Attributes)
	{
		FString EscapedValue = Attribute.GetValue();
		EscapedValue.ReplaceInline(TEXT("&"), TEXT("&amp;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("\""), TEXT("&quot;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("'"), TEXT("&apos;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("<"), TEXT("&lt;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT(">"), TEXT("&gt;"), ESearchCase::CaseSensitive);
		Output += FString::Printf(TEXT(" %s=\"%s\""), *Attribute.GetTag(), *EscapedValue);
	}

	// Write the node children
	if (Children.Num() == 0)
	{
		if (Content.Len() == 0)
		{
			Output += TEXT(" />") LINE_TERMINATOR;
		}
		else
		{
			Output += TEXT(">") + Content + FString::Printf(TEXT("</%s>"), *Tag) + LINE_TERMINATOR;
		}
	}
	else
	{
		Output += TEXT(">") LINE_TERMINATOR;
		for (const TSharedPtr<FFCPXMLNode> ChildNode : Children)
		{
			ChildNode->GetXmlBuffer(Indent + TEXT("\t"), Output);
		}
		Output += Indent + FString::Printf(TEXT("</%s>"), *Tag) + LINE_TERMINATOR;
	}
}

const FString& FFCPXMLNode::GetTag() const
{
	return Tag;
}

const FString& FFCPXMLNode::GetContent() const
{
	return Content;
}

bool FFCPXMLNode::GetContent(FString& OutValue) const
{
	OutValue = Content;
	return true;
}

bool FFCPXMLNode::GetContent(float& OutValue) const
{
	return FCPXMLStringToFloat(Content, OutValue);
}

bool FFCPXMLNode::GetContent(int32& OutValue) const
{
	return FCPXMLStringToInt(Content, OutValue);
}

bool FFCPXMLNode::GetContent(bool& OutValue) const
{
	return FCPXMLStringToBool(Content, OutValue);
}

void FFCPXMLNode::SetContent(const FString& InContent)
{
	Content = InContent;
}

void FFCPXMLNode::SetContent(float InContent)
{
	Content = FString::Printf(TEXT("%f"), InContent);
}

void FFCPXMLNode::SetContent(int32 InContent)
{
	Content = FString::Printf(TEXT("%d"), InContent);
}

void FFCPXMLNode::SetContent(bool InContent)
{
	if (InContent)
	{
		Content = FString(TEXT("TRUE"));
	}
	else
	{
		Content = FString(TEXT("FALSE"));
	}
}

const TArray<TSharedRef<FFCPXMLNode>>& FFCPXMLNode::GetChildNodes() const
{
	return Children;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNode(const FString& InElement, ENodeInherit CheckInherit, ENodeReference CheckRefIds) const
{
	TSharedPtr<FFCPXMLNode> Node = GetChildNodeOnly(InElement);
	if (Node.IsValid())
	{
		return Node;
	}

	// look for element in references of this node
	if (CheckRefIds == ENodeReference::CheckReferences)
	{
		Node = GetChildNodeReference(InElement);
		if (Node.IsValid())
		{
			return Node;
		}
	}

	// look for element in inheritance hierarchy
	if (CheckInherit == ENodeInherit::CheckInherit)
	{
		TSharedPtr<FFCPXMLNode> LockedParent(Parent.Pin());
		if (LockedParent.IsValid())
		{
			return LockedParent->GetChildNodeInherit(InElement);
		}
	}

	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNode(const FString& InElement, const FString& InSubElement, ENodeInherit CheckInherit, ENodeReference CheckRefIds) const
{
	TSharedPtr<FFCPXMLNode> Node = GetChildNodeOnly(InElement);
	if (Node.IsValid())
	{
		TSharedPtr<FFCPXMLNode> SubElementNode = GetChildNodeOnly(InSubElement);
		if (SubElementNode.IsValid())
		{
			return SubElementNode;
		}
	}
	
	// look for element in reference of this node
	if (CheckRefIds == ENodeReference::CheckReferences)
	{
		Node = GetChildNodeReference(InElement, InSubElement);
		if (Node.IsValid())
		{
			return Node;
		}
	}

	// look for element in inheritance hierarchy
	if (CheckInherit == ENodeInherit::CheckInherit)
	{
		TSharedPtr<FFCPXMLNode> LockedParent(Parent.Pin());
		if (LockedParent.IsValid())
		{
			return LockedParent->GetChildNodeInherit(InElement, InSubElement);
		}
	}

	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeOnly(const FString& InElement) const
{
	for (TSharedPtr<FFCPXMLNode> Child : Children)
	{
		if (Child->GetTag() == InElement)
		{
			return Child;
		}
	}
	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeOnly(const FString& InElement, const FString& InSubElement) const
{
	for (TSharedPtr<FFCPXMLNode> Child : Children)
	{
		if (Child->GetTag() == InElement)
		{
			return Child->GetChildNodeOnly(InSubElement);
		}
	}
	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetReferenceNode() const
{
	FString Id;
	if (GetAttributeValue(TEXT("id"), Id))
	{
		return GetReference(Tag, Id);
	}

	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeReference(const FString& InElement) const
{
	TSharedPtr<FFCPXMLNode> RefNode = GetReferenceNode();
	if (RefNode.IsValid())
	{
		return RefNode->GetChildNodeOnly(InElement);
	}
	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeReference(const FString& InElement, const FString& InSubElement) const
{
	TSharedPtr<FFCPXMLNode> RefNode = GetReferenceNode();
	if (RefNode.IsValid())
	{
		TSharedPtr<FFCPXMLNode> ElementNode = RefNode->GetChildNodeOnly(InElement);
		if (ElementNode.IsValid())
		{
			return ElementNode->GetChildNodeOnly(InSubElement);
		}
	}
	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeInherit(const FString& InElement) const
{
	TSharedPtr<FFCPXMLNode> LockedParent(Parent.Pin());
	if (LockedParent.IsValid())
	{
		TSharedPtr<FFCPXMLNode> Node = LockedParent->GetChildNodeOnly(InElement);
		if (Node.IsValid())
		{
			return Node;
		}
		else
		{
			// keep looking up the tree
			return LockedParent->GetChildNodeInherit(InElement);
		}
	}

	return nullptr;
}

TSharedPtr<FFCPXMLNode> FFCPXMLNode::GetChildNodeInherit(const FString& InElement, const FString& InSubElement) const
{
	TSharedPtr<FFCPXMLNode> LockedParent(Parent.Pin());
	if (LockedParent.IsValid())
	{
		TSharedPtr<FFCPXMLNode> Node = LockedParent->GetChildNodeOnly(InElement, InSubElement);
		if (Node.IsValid())
		{
			return Node;
		}
		else
		{
			// keep looking up the tree
			return LockedParent->GetChildNodeInherit(InElement, InSubElement);
		}
	}

	return nullptr;
}

void FFCPXMLNode::AppendChildNode(TSharedRef<FFCPXMLNode> InNode)
{
	Children.Push(InNode);
}

TSharedRef<FFCPXMLNode> FFCPXMLNode::CreateChildNode(const FString& InTag)
{
	TSharedPtr<FFCPXMLFile> FilePtr = ContainingFile.Pin();
	TSharedRef<FFCPXMLNode> NewNode = FFCPXMLNode::CreateFFCPXMLNode(InTag, AsShared(), FilePtr);
	Children.Push(NewNode);
	return NewNode;
}

void FFCPXMLNode::AddAttribute(const FString &InTag, const FString &InValue)
{
	FFCPXMLAttribute Attr(InTag, InValue);
	Attributes.Push(Attr);
}

const TArray<FFCPXMLAttribute>& FFCPXMLNode::GetAttributes() const
{
	return Attributes;
}

bool FFCPXMLNode::GetAttribute(const FString& InTag, FFCPXMLAttribute& OutAttr) const
{
	for (const FFCPXMLAttribute& Attr : Attributes)
	{
		if (Attr.GetTag() == InTag)
		{
			OutAttr = Attr;
			return true;
		}
	}

	return false;
}

bool
FFCPXMLNode::AddReference(const FString& InElement, const FString &InId, TSharedPtr<FFCPXMLNode> InNode)
{
	TSharedPtr<FFCPXMLFile> LockedFile(ContainingFile.Pin());
	if (LockedFile.IsValid())
	{
		return LockedFile->AddReference(InElement, InId, InNode);
	}
	return false;
}

TSharedPtr<FFCPXMLNode>
FFCPXMLNode::GetReference(const FString &InElement, const FString &InId) const
{
	TSharedPtr<FFCPXMLFile> LockedFile(ContainingFile.Pin());
	if (LockedFile.IsValid())
	{
		return LockedFile->GetReference(InElement, InId);
	}
	return nullptr;
}

/** Factory method to create node object based on tag */
TSharedRef<FFCPXMLNode> FFCPXMLNode::CreateFFCPXMLNode(FString InTag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile)
{
	if (InTag == "xmeml")
	{
		TSharedRef<FFCPXMLXmemlNode> XmemlRef = MakeShared<FFCPXMLXmemlNode>(InParent, InFile);
		return XmemlRef;
	}
	else if (InTag == "sequence")
	{
		TSharedRef<FFCPXMLSequenceNode> SeqRef = MakeShared<FFCPXMLSequenceNode>(InParent, InFile);
		return SeqRef;
	}
	else if (InTag == "video")
	{
		TSharedRef<FFCPXMLVideoNode> VideoRef = MakeShared<FFCPXMLVideoNode>(InParent, InFile);
		return VideoRef;
	}
	else if (InTag == "audio")
	{
		TSharedRef<FFCPXMLAudioNode> AudioRef = MakeShared<FFCPXMLAudioNode>(InParent, InFile);
		return AudioRef;
	}
	else if (InTag == "track")
	{
		TSharedRef<FFCPXMLTrackNode> TrackRef = MakeShared<FFCPXMLTrackNode>(InParent, InFile);
		return TrackRef;
	}
	else if (InTag == "clip")
	{
		TSharedRef<FFCPXMLClipNode> ClipRef = MakeShared<FFCPXMLClipNode>(InParent, InFile);
		return ClipRef;
	}
	else if (InTag == "clipitem")
	{
		TSharedRef<FFCPXMLClipItemNode> ClipItemRef = MakeShared<FFCPXMLClipItemNode>(InParent, InFile);
		return ClipItemRef;
	}
	else if (InTag == "file")
	{
		TSharedRef<FFCPXMLFileNode> FileRef = MakeShared<FFCPXMLFileNode>(InParent, InFile);
		return FileRef;
	}

	TSharedRef<FFCPXMLBasicNode> BasicRef = MakeShared<FFCPXMLBasicNode>(InTag, InParent, InFile);
	return BasicRef;
}

FFCPXMLBasicNode::FFCPXMLBasicNode(FString InTag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(InTag, InParent, InFile) {}

bool FFCPXMLBasicNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLBasicNode>(AsShared()));
}

FFCPXMLXmemlNode::FFCPXMLXmemlNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("xmeml")), InParent, InFile) {}

bool FFCPXMLXmemlNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLXmemlNode>(AsShared()));
}

FFCPXMLSequenceNode::FFCPXMLSequenceNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("sequence")), InParent, InFile) {}

bool FFCPXMLSequenceNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLSequenceNode>(AsShared()));
}

FFCPXMLVideoNode::FFCPXMLVideoNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("video")), InParent, InFile) {}

bool FFCPXMLVideoNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLVideoNode>(AsShared()));
}

FFCPXMLAudioNode::FFCPXMLAudioNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("audio")), InParent, InFile) {}

bool FFCPXMLAudioNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLAudioNode>(AsShared()));
}

FFCPXMLClipNode::FFCPXMLClipNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("clip")), InParent, InFile) {}

bool FFCPXMLClipNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLClipNode>(AsShared()));
}

FFCPXMLClipItemNode::FFCPXMLClipItemNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("clipitem")), InParent, InFile) {}

bool FFCPXMLClipItemNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{ 
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLClipItemNode>(AsShared()));
}

FFCPXMLTrackNode::FFCPXMLTrackNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("track")), InParent, InFile) {}

bool FFCPXMLTrackNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor) 
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLTrackNode>(AsShared()));
}

FFCPXMLFileNode::FFCPXMLFileNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile) : FFCPXMLNode(FString(TEXT("file")), InParent, InFile) {}

bool FFCPXMLFileNode::DoAccept(FFCPXMLNodeVisitor& NodeVisitor)
{
	return NodeVisitor.VisitNode(StaticCastSharedRef<FFCPXMLFileNode>(AsShared()));
}

#undef LOCTEXT_NAMESPACE
