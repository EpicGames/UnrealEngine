// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FXmlNode;
/** The FFCPXMLNode pure virtual base class represents a node in the
	FCP 7 XML tree. There are derived classes for each node type, including
	a "Basic" node as default node type.

	NODE VISITOR CLASS
	The FFCPXMLNode includes support for a NodeVisitor class to easily traverse
	and modify the node structure. The FFCPXMLNode method Accepts() is called 
	to accept NodeVisitor objects. This implements the visitor pattern in order
	to allow double dispatch based on both the type of the node and the type 
	of the operation (e.g. Import, Export, File Setup).
	
	INHERITANCE
	Node element inheritance is supported, when specified, by searching for 
	inherited elements by walking up the node's parents, looking for peer elements
	with matching tags.

	REFERENCE IDS
	Certain node types support reference id attributes. When an id attribute
	is present, the node is added the reference id map in the FFCPXMLFile
	class. If the reference id already exists in the map, the current node
	inherits elements from the reference node if they are not defined in
	the current node.
*/

class FFCPXMLFile;
class FFCPXMLNodeVisitor;

enum class ENodeInherit : uint8
{
	NoInherit,
	CheckInherit
};

enum class ENodeReference : uint8
{
	NoReferences,
	CheckReferences
};

/** FCP XML attribute class. */
class FFCPXMLAttribute
{
public:
	/** Default constructor */
	FFCPXMLAttribute(const FString& InTag, const FString& InValue) : Tag(InTag), Value(InValue) {}

	/** Gets the tag of the attribute */
	const FString& GetTag() const;

	/** Gets the value of the attribute */
	const FString& GetValue() const;
	/** Gets the value of the node */
	bool GetValue(FString& OutValue) const;
	/** Gets the value of the node */
	bool GetValue(float& OutValue) const;
	/** Gets the value of the node */
	bool GetValue(int32& OutValue) const;
	/** Gets the value of the node */
	bool GetValue(bool& OutValue) const;

private:
	/** The tag string */
	FString Tag;

	/** The value string */
	FString Value;
};

/** FCP XML node class. This supports traversal by a visitor object which may modify the node tree's structure. */
class FFCPXMLNode : public TSharedFromThis<FFCPXMLNode>
{
	friend class FFCPXMLNodeVisitor;

public:
	/** Constructor */
	FFCPXMLNode(const FString InTag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

	/** Destructor */
	virtual ~FFCPXMLNode() = default;

private:

	/** No default constructor allowed */
	FFCPXMLNode() {}

 	/** No copy constructor allowed */
	FFCPXMLNode(const FFCPXMLNode& rhs) {}

public:
	/** Factory method to create new FCPXMLNode based on xml tag. */
	static TSharedRef<FFCPXMLNode> CreateFFCPXMLNode(FString tag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

	/** Recursive copy of data from XmlNode to this node */
	void CopyFrom(FXmlNode* InNode);

	/** Write data from this node into an Xml string */
	void GetXmlBuffer(const FString& Indent, FString& Output) const;

	/**
	* Calls the appropriate visit method for a given visitor node.
	* @param	NodeVisitor				Visitor object
	*/
	bool Accept(FFCPXMLNodeVisitor& NodeVisitor);

	/**
	* Visits the children of this node.
	* @param	NodeVisitor				Visitor object
	* @param    VisitRefIdChildren      When true, node will traverse refId node elements if they do not exist in current node 
	*/
	bool VisitChildren(FFCPXMLNodeVisitor& NodeVisitor, bool VisitRefNodeChildren = false);

public:

	/** Gets the tag of the node */
	const FString& GetTag() const;

	/** Gets the value of the node */
	const FString& GetContent() const;
	/** Gets the value of the node */
	bool GetContent(FString& OutValue) const;
	/** Gets the value of the node */
	bool GetContent(float& OutValue) const;
	/** Gets the value of the node */
	bool GetContent(int32& OutValue) const;
	/** Gets the value of the node */
	bool GetContent(bool& OutValue) const;

	/** Sets the new value of the node */
	void SetContent(const FString& InContent);
	/** Sets the new value of the node */
	void SetContent(float InContent);
	/** Sets the new value of the node */
	void SetContent(int32 InContent);
	/** Sets the new value of the node */
	void SetContent(bool InContent);

	/** Sets the new value of the node */
	FORCEINLINE void SetContent(const TCHAR* InContent)
	{
		// Force an FString conversion here, otherwise it calls the bool overload
		SetContent(FString(InContent));
	}

	/** Gets a list of child nodes */
	const TArray<TSharedRef<FFCPXMLNode>>& GetChildNodes() const;
	/** Gets number of child nodes */
	uint32 GetChildCount() { return Children.Num(); }

	/** Appends input node to children nodes */
	void AppendChildNode(TSharedRef<FFCPXMLNode> InNode);
	/** Creates a child node with the given tag and appends node to children. */
	TSharedRef<FFCPXMLNode> CreateChildNode(const FString& InTag);

	/** Retrieves child content, returning false if not found. */
	template <typename T>
	bool GetChildValue(const FString& InElement, T& OutValue, ENodeInherit CheckInherit = ENodeInherit::CheckInherit, ENodeReference CheckRefIds = ENodeReference::CheckReferences) const;

	/** Retrieves child content, returning false if not found. */
	template <typename T>
	bool GetChildSubValue(const FString& InElement, const FString& InSubElement, T& OutValue, ENodeInherit CheckInherit = ENodeInherit::CheckInherit, ENodeReference CheckRefIds = ENodeReference::CheckReferences) const;

	/** Retrieves child node, checking references and inheritance if specified. */
	TSharedPtr<FFCPXMLNode> GetChildNode(const FString& InElement, ENodeInherit CheckInherit = ENodeInherit::CheckInherit, ENodeReference CheckRefIds = ENodeReference::CheckReferences) const;

	/** Retrieves child node, checking references and inheritance if specified. */
	TSharedPtr<FFCPXMLNode> GetChildNode(const FString& InElement, const FString& InSubElement, ENodeInherit CheckInherit = ENodeInherit::CheckInherit, ENodeReference CheckRefIds = ENodeReference::CheckReferences) const;

	/** Adds an attribute */
	void AddAttribute(const FString &InTag, const FString &InValue);

	/**
	* Gets all of the attributes in this node
	* @return	List of attributes in this node
	*/
	const TArray<FFCPXMLAttribute>& GetAttributes() const;

	/** Gets an attribute that corresponds with the passed-in tag */
	bool GetAttribute(const FString& InTag, FFCPXMLAttribute& OutAttr) const;

	/** Retrieves attribute value, returning false if not found. */
	template <typename T>
	bool GetAttributeValue(const FString& InTag, T& OutValue) const;

public:
	/** Adds reference to file's reference map if it does not already exist */
	bool AddReference(const FString& InElement, const FString &InId, TSharedPtr<FFCPXMLNode> InNode);

	/** Gets reference from file's reference map */
	TSharedPtr<FFCPXMLNode> GetReference(const FString &InElement, const FString &InId) const;

protected:

	/** The list of children nodes */
	TArray<TSharedRef<FFCPXMLNode>> Children;
	/** Attributes of this node */
	TArray<FFCPXMLAttribute> Attributes;
	/** Weak pointer to parent node */
	TWeakPtr<FFCPXMLNode> Parent;
	/** Weak pointer to file object */
	TWeakPtr<FFCPXMLFile> ContainingFile;
	/** Tag of the node */
	FString Tag;
	/** Content of the node */
	FString Content;

private:

	/** Retrieves child node without checking references and inheritance. */
	TSharedPtr<FFCPXMLNode> GetChildNodeOnly(const FString& InElement) const;
	/** Retrieves child node without checking references and inheritance. */
	TSharedPtr<FFCPXMLNode> GetChildNodeOnly(const FString& InElement, const FString& InSubElement) const;
	/** Retrieves child node from a reference, if it exists. */
	TSharedPtr<FFCPXMLNode> GetChildNodeReference(const FString& InElement) const;
	/** Retrieves child node from a reference, if it exists. */
	TSharedPtr<FFCPXMLNode> GetChildNodeReference(const FString& InElement, const FString& InSubElement) const;
	/** Retrieves child node via inheritance, if it exists. */
	TSharedPtr<FFCPXMLNode> GetChildNodeInherit(const FString& InElement) const;
	/** Retrieves child node via inheritance, if it exists. */
	TSharedPtr<FFCPXMLNode> GetChildNodeInherit(const FString& InElement, const FString& InSubElement) const;

	/** Retrieves corresponding reference node, if it exists. */
	TSharedPtr<FFCPXMLNode> GetReferenceNode() const;

	/** Pure virtual method that accepts a visitor to visit this node */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) = 0;
};

/** Retrieves child value, returning false if not found. */
template <typename T>
bool FFCPXMLNode::GetChildValue(const FString& InElement, T& OutValue, ENodeInherit CheckInherit, ENodeReference CheckRefIds) const
{
	TSharedPtr<FFCPXMLNode> Node = GetChildNode(InElement, CheckInherit, CheckRefIds);
	if (Node.IsValid())
	{
		return Node->GetContent(OutValue);
	}

	return false;
}

/** Retrieves child value, returning false if not found. */
template <typename T>
bool FFCPXMLNode::GetChildSubValue(const FString& InElement, const FString& InSubElement, T& OutValue, ENodeInherit CheckInherit, ENodeReference CheckRefIds) const
{
	TSharedPtr<FFCPXMLNode> Node = GetChildNode(InElement, InSubElement, CheckInherit, CheckRefIds);
	if (Node.IsValid())
	{
		return Node->GetContent(OutValue);
	}

	return false;
}

template <typename T>
bool FFCPXMLNode::GetAttributeValue(const FString& InTag, T& OutValue) const
{
	for (const FFCPXMLAttribute& Attr : Attributes)
	{
		if (Attr.GetTag() == InTag)
		{
			return Attr.GetValue(OutValue);
		}
	}
	return false;
}

/** FCP XML Node class for any node type */
class FFCPXMLBasicNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLBasicNode(FString InTag, TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for the xmeml node which is the root of every XML file */
class FFCPXMLXmemlNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLXmemlNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for sequence nodes */
class FFCPXMLSequenceNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLSequenceNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for video nodes */
class FFCPXMLVideoNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLVideoNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for audio nodes */
class FFCPXMLAudioNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLAudioNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for track nodes */
class FFCPXMLTrackNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLTrackNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for clip nodes */
class FFCPXMLClipNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLClipNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for clipitem nodes */
class FFCPXMLClipItemNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLClipItemNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/** FCP XML Node class for file nodes */
class FFCPXMLFileNode : public FFCPXMLNode
{
public:
	/** Constructor */
	FFCPXMLFileNode(TSharedPtr<FFCPXMLNode> InParent, TSharedPtr<FFCPXMLFile> InFile);

private:
	/** Accepts a node visitor object */
	virtual bool DoAccept(FFCPXMLNodeVisitor& NodeVisitor) override final;
};

/**
* FCP XML node visitor abstract base class. This must contain a method
* for every node type that will be visiting the object. These inherited
* visitors can then be used to traverse the FCP XML node structure.
*/
class FFCPXMLNodeVisitor
{
public:
	/** Constructor */
	FFCPXMLNodeVisitor() {};
	/** Destructor */
	virtual ~FFCPXMLNodeVisitor() = default;

public:
	/** Visit anonymous node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLBasicNode> Node) = 0;
	/** Visit xmeml node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLXmemlNode> Node) = 0;
	/** Visit sequence node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLSequenceNode> Node) = 0;
	/** Visit video node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLVideoNode> Node) = 0;
	/** Visit audio node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLAudioNode> Node) = 0;
	/** Visit track node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLTrackNode> Node) = 0;
	/** Visit clip node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipNode> Node) = 0;
	/** Visit clip item node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLClipItemNode> Node) = 0;
	/** Visit file node. */
	virtual bool VisitNode(TSharedRef<FFCPXMLFileNode> Node) = 0;
};
