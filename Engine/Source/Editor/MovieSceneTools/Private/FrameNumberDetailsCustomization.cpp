// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameNumberDetailsCustomization.h"
#include "IDetailPropertyRow.h"
#include "Misc/FrameNumber.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/FrameRate.h"
#include "ISequencer.h"

#define LOCTEXT_NAMESPACE "TimeManagement.FrameNumber"

void FFrameNumberDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	using namespace UE::Sequencer;

	// Check for Min/Max metadata on the property itself and we'll apply it to the child when changed.
	const FString& MetaUIMinString = PropertyHandle->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = PropertyHandle->GetMetaData(TEXT("UIMax"));

	UIClampMin = TNumericLimits<int32>::Lowest();
	UIClampMax = TNumericLimits<int32>::Max();

	if (!MetaUIMinString.IsEmpty())
	{
		TTypeFromString<int32>::FromString(UIClampMin, *MetaUIMinString);
	}

	if (!MetaUIMaxString.IsEmpty())
	{
		TTypeFromString<int32>::FromString(UIClampMax, *MetaUIMaxString);
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (NumericTypeInterface == nullptr && Sequencer)
	{
		// Find the most relevant interface for this property
		int32 BestScore = MIN_int32;
		for (TSharedRef<FSequencerNumericTypeInterface> Interface : Sequencer->GetNumericTypeInterfaces())
		{
			const int32 Score = Interface->GetRelevancyScore(*Sequencer, PropertyHandle);
			if (Score > BestScore)
			{
				BestScore = Score;
				NumericTypeInterface = Interface->Interface;
			}
		}
	}

	TMap<FName, TSharedRef<IPropertyHandle>> CustomizedProperties;

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	// Add child properties to UI and pick out the properties which need customization
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		CustomizedProperties.Add(ChildHandle->GetProperty()->GetFName(), ChildHandle);
	}

	FrameNumberProperty = CustomizedProperties.FindChecked(GET_MEMBER_NAME_CHECKED(FFrameNumber, Value));

	ChildBuilder.AddCustomRow(LOCTEXT("TimeLabel", "Time"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(PropertyHandle->GetPropertyDisplayName())
			.ToolTipText(PropertyHandle->GetToolTipText())
			.Font(CustomizationUtils.GetRegularFont())
		]
		.ValueContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Fill)
			[
				SNew(SEditableTextBox)
				.Text(this, &FFrameNumberDetailsCustomization::OnGetTimeText)
				.ToolTipText(this, &FFrameNumberDetailsCustomization::OnGetTimeToolTipText)
				.OnTextCommitted(this, &FFrameNumberDetailsCustomization::OnTimeTextCommitted)
				.SelectAllTextWhenFocused(true)
				.ClearKeyboardFocusOnCommit(false)
				.RevertTextOnEscape(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Padding(FMargin(8.0f, 4.0f, 8.0f, 4.0f))
			]
		];
}

FText FFrameNumberDetailsCustomization::OnGetTimeText() const
{
	if (!NumericTypeInterface.IsValid())
	{
		return FText();
	}

	int32 CurrentValue = 0.0;
	FPropertyAccess::Result Result = FrameNumberProperty->GetValue(CurrentValue);

	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	return FText::FromString(NumericTypeInterface->ToString(CurrentValue));
}

FText FFrameNumberDetailsCustomization::OnGetTimeToolTipText() const
{
	const FText ToolTipText = LOCTEXT("TimeLabelWithDetailsTooltip", "Time field which takes timecode, frames and seconds formats (current: {0}).");

	int32 CurrentValue = 0;
	FPropertyAccess::Result Result = FrameNumberProperty->GetValue(CurrentValue);
	if (Result == FPropertyAccess::MultipleValues)
	{
		return FText::Format(ToolTipText, LOCTEXT("MultipleValues", "Multiple Values"));
	}

	// Since CurrentValue is an integer, we know that we won't have any subframe. So we only have to
	// display the tick number itself.
	FFrameTime DisplayTime = FFrameTime::FromDecimal(CurrentValue);
	FString DisplayTimeString = FString::Printf(TEXT("%d ticks"), DisplayTime.GetFrame().Value);
	return FText::Format(ToolTipText, FText::FromString(DisplayTimeString));
}

void FFrameNumberDetailsCustomization::OnTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (!NumericTypeInterface.IsValid())
	{
		return;
	}

	TArray<FString> PerObjectValueStrs;
	FrameNumberProperty->GetPerObjectValues(PerObjectValueStrs);

	for (FString& ValueStr : PerObjectValueStrs)
	{
		int32 ExistingValue = FCString::Atoi(*ValueStr);
		TOptional<double> TickResolution = NumericTypeInterface->FromString(InText.ToString(), ExistingValue);
		if (TickResolution.IsSet())
		{
			double ClampedValue = FMath::Clamp(TickResolution.GetValue(), (double)UIClampMin, (double)UIClampMax);
			ValueStr = FString::FromInt((int32)ClampedValue);
		}
	}
	FrameNumberProperty->SetPerObjectValues(PerObjectValueStrs);
}

#undef LOCTEXT_NAMESPACE
