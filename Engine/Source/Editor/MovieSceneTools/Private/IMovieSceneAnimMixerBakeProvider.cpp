// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMovieSceneAnimMixerBakeProvider.h"

FName IMovieSceneAnimMixerBakeProvider::GetModularFeatureName()
{
	static FName FeatureName(TEXT("MovieSceneAnimMixerBakeProvider"));
	return FeatureName;
}
