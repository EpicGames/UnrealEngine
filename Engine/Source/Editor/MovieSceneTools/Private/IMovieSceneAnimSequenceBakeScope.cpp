// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMovieSceneAnimSequenceBakeScope.h"

FName IMovieSceneAnimSequenceBakeScope::GetModularFeatureName()
{
	static const FName Name(TEXT("MovieSceneAnimSequenceBakeScope"));
	return Name;
}
