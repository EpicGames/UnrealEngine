// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetSequenceBinding.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Framework/Application/SlateApplication.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "PropertyCustomizationHelpers.h"
#include "MovieSceneSequence.h"
#include "ToolMenus.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "SGraphNode.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "GraphEditorSettings.h"
#include "Engine/Selection.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Editor.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetSequenceBinding)

static const FName OutputPinName(TEXT("Output"));
static const FName SequencePinName(TEXT("Sequence"));

#define LOCTEXT_NAMESPACE "UK2Node_GetSequenceBinding"

void EnsureFullyPreloaded(UObject* Object)
{
	if (!Object)
	{
		return;
	}

	Object->ConditionalPreload();

	// We only want to ensure that _loaded_ objects have RF_LoadCompleted set.
	// Some objects can be created during postload, so we don't need to verify RF_LoadCompleted in those cases.
	if (FLinkerLoad* Linker = Object->GetLinker())
	{
		check(Object->HasAnyFlags(RF_LoadCompleted));
	}

	TArray<UObject*> ObjectReferences;
	FReferenceFinder(ObjectReferences, nullptr, false, true, false, true).FindReferences(Object);

	for (UObject* Reference : ObjectReferences)
	{
		check(Reference);

		const bool bIsMovieSceneType =
			Reference->IsA<UMovieSceneSequence>() ||
			Reference->IsA<UMovieScene>() ||
			Reference->IsA<UMovieSceneTrack>() ||
			Reference->IsA<UMovieSceneSection>()
		;

		if (bIsMovieSceneType)
		{
			EnsureFullyPreloaded(Reference);
		}
	}
}

class FKCHandler_GetSequenceBinding : public FNodeHandlingFunctor
{
public:
	FKCHandler_GetSequenceBinding(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_GetSequenceBinding* BindingNode = CastChecked<UK2Node_GetSequenceBinding>(Node);
		
		for (UEdGraphPin* Pin : BindingNode->GetAllPins())
		{
			if (Pin->Direction == EGPD_Output && Pin->LinkedTo.Num())
			{
				FBPTerminal* Term = RegisterLiteral(Context, Pin);

				// Intentionally pass BindingPtr to both the data and defaults for ExportText to ensure that we always disable delta-serialization on the struct
				FMovieSceneObjectBindingID* BindingPtr = &BindingNode->Binding;
				FMovieSceneObjectBindingID::StaticStruct()->ExportText(Term->Name, BindingPtr, BindingPtr, nullptr, 0, nullptr);
			}
		}
	}
};

void UK2Node_GetSequenceBinding::SetSequence(UMovieSceneSequence* InSequence)
{
	SourceMovieSequence = InSequence;
}

void UK2Node_GetSequenceBinding::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const bool bConvertSoftToHardReference =
		Ar.IsLoading() && 
		((Ar.GetPortFlags() & PPF_Duplicate) == 0) &&
		!SourceSequence_DEPRECATED.IsNull()
	;

	if (bConvertSoftToHardReference)
	{
		SourceMovieSequence = Cast<UMovieSceneSequence>(SourceSequence_DEPRECATED.TryLoad());
		SourceSequence_DEPRECATED.Reset();
	}
}

void UK2Node_GetSequenceBinding::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UMovieScene* MovieScene = GetObjectMovieScene();
	if (!MovieScene)
	{
		const FText MessageText = LOCTEXT("InvalidSequenceBinding_NoSequence", "Invalid sequence binding specified on node @@ (could not find sequence).");
		MessageLog.Warning(*MessageText.ToString(), this);
	}
	else if (!MovieScene->FindPossessable(Binding.GetGuid()) && !MovieScene->FindSpawnable(Binding.GetGuid()))
	{
		const FText MessageText = LOCTEXT("InvalidSequenceBinding_Unresolved", "Invalid sequence binding specified on node @@.");
		MessageLog.Warning(*MessageText.ToString(), this);
	}
}

void UK2Node_GetSequenceBinding::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UMovieSceneSequence::StaticClass(), SequencePinName);

	// Result pin
	UEdGraphPin* ResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FMovieSceneObjectBindingID::StaticStruct(), UEdGraphSchema_K2::PN_ReturnValue);
	ResultPin->PinFriendlyName = LOCTEXT("SequenceBindingOutput", "Binding");

	Super::AllocateDefaultPins();
}

void UK2Node_GetSequenceBinding::PostPlacedNewNode()
{
	// Attempt to assign the sequence asset from our outer if this BP is contained within a sequence
	if (UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>())
	{
		SourceMovieSequence = OuterSequence;
	}
	Super::PostPlacedNewNode();
}

UMovieScene* UK2Node_GetSequenceBinding::GetObjectMovieScene() const
{
	if (SourceMovieSequence && Binding.IsValid())
	{
		FMovieSceneSequenceID SequenceID = Binding.GetRelativeSequenceID();
		if (SequenceID == MovieSceneSequenceID::Root)
		{
			// Look it up in the moviescene itself
			return SourceMovieSequence->GetMovieScene();
		}
		else
		{
			bool bHierarchyIsValid = true;

			// Ensure the hierarchy is valid (ie, the user hasn't changed a sub sequence for something else)
			FMovieSceneSequenceID CurrentSequenceID = SequenceID;
			while (CurrentSequenceID != MovieSceneSequenceID::Root)
			{
				const FMovieSceneSubSequenceData* SubData = SequenceHierarchyCache.FindSubData(CurrentSequenceID);
				const UMovieSceneSequence* SubSequence = SubData ? SubData->GetSequence() : nullptr;

				if (!SubSequence || SubSequence->GetSignature() != SequenceSignatureCache.FindRef(CurrentSequenceID))
				{
					bHierarchyIsValid = false;
					break;
				}

				const FMovieSceneSequenceHierarchyNode* Node = SequenceHierarchyCache.FindNode(CurrentSequenceID);
				if (!Node)
				{
					bHierarchyIsValid = false;
					break;
				}

				CurrentSequenceID = Node->ParentID;
			}

			// If it's not valid, it needs recompiling
			if (!bHierarchyIsValid)
			{
				// Recompile the hierarchy
				SequenceSignatureCache.Reset();
				SequenceHierarchyCache = FMovieSceneSequenceHierarchy();

				UMovieSceneCompiledDataManager::CompileHierarchy(SourceMovieSequence, &SequenceHierarchyCache, EMovieSceneServerClientMask::All);

				for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : SequenceHierarchyCache.AllSubSequenceData())
				{
					const UMovieSceneSequence* SubSequence = Pair.Value.GetSequence();
					if (ensure(SubSequence))
					{
						SequenceSignatureCache.Add(Pair.Key, SubSequence->GetSignature());
					}
				}
			}

			UMovieScene* MovieScene = nullptr;
			if (const FMovieSceneSubSequenceData* SubData = SequenceHierarchyCache.FindSubData(SequenceID))
			{
				UMovieSceneSequence* SubSequence = SubData->GetSequence();
				return SubSequence ? SubSequence->GetMovieScene() : nullptr;
			}
		}
	}

	return nullptr;
}

FNodeHandlingFunctor* UK2Node_GetSequenceBinding::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_GetSequenceBinding(CompilerContext);
}

void UK2Node_GetSequenceBinding::PreloadRequiredAssets()
{
	EnsureFullyPreloaded(SourceMovieSequence);
}

FText UK2Node_GetSequenceBinding::GetSequenceName() const
{
	return SourceMovieSequence ? FText::FromName(SourceMovieSequence->GetFName()) : LOCTEXT("NoSequence", "No Sequence");
}

FText UK2Node_GetSequenceBinding::GetBindingName() const
{
	UMovieScene* MovieScene = GetObjectMovieScene();
	return MovieScene ? MovieScene->GetObjectDisplayName(Binding.GetGuid()) : FText();
}

FText UK2Node_GetSequenceBinding::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText BindingName = GetBindingName();

	return BindingName.IsEmpty() ? LOCTEXT("NodeTitle", "Get Sequence Binding") : FText::Format(LOCTEXT("NodeTitle_Format", "Get Sequence Binding ({0})"), BindingName);
}

FText UK2Node_GetSequenceBinding::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Access an identifier for any object binding within a sequence");
}

FText UK2Node_GetSequenceBinding::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Sequencer|Player|Bindings");
}

FSlateIcon UK2Node_GetSequenceBinding::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.GetSequenceBinding");
	return Icon;
}

void UK2Node_GetSequenceBinding::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("K2NodeGetSequenceBinding", LOCTEXT("ThisNodeHeader", "This Node"));
		if (!Context->Pin)
		{
			Section.AddSubMenu(
				"SetSequence",
				LOCTEXT("SetSequence_Text", "Sequence"),
				LOCTEXT("SetSequence_ToolTip", "Sets the sequence to get a binding from"),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* SubMenu)
				{
					TArray<const UClass*> AllowedClasses({ UMovieSceneSequence::StaticClass() });

					TSharedRef<SWidget> MenuContent = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
						FAssetData(SourceMovieSequence),
						true /* bAllowClear */,
						AllowedClasses,
						PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
						FOnShouldFilterAsset(),
						FOnAssetSelected::CreateUObject(const_cast<UK2Node_GetSequenceBinding*>(this), &UK2Node_GetSequenceBinding::SetSequence),
						FSimpleDelegate());
					
					SubMenu->AddMenuEntry("Section", FToolMenuEntry::InitWidget("Widget", MenuContent, FText::GetEmpty(), false));
				}));
		}
	}
}

void UK2Node_GetSequenceBinding::SetSequence(const FAssetData& InAssetData)
{
	FSlateApplication::Get().DismissAllMenus();

	const FScopedTransaction Transaction(LOCTEXT("SetSequence", "Set Sequence"));
	Modify();

	SourceMovieSequence = Cast<UMovieSceneSequence>(InAssetData.GetAsset());
}

void UK2Node_GetSequenceBinding::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

TSharedPtr<SGraphNode> UK2Node_GetSequenceBinding::CreateVisualWidget()
{
	class SGraphNodeGetSequenceBinding : public SGraphNode, FMovieSceneObjectBindingIDPicker
	{
	public:
		SLATE_BEGIN_ARGS(SGraphNodeGetSequenceBinding){}
		SLATE_END_ARGS()

		SGraphNodeGetSequenceBinding()
			: FMovieSceneObjectBindingIDPicker(MovieSceneSequenceID::Root, nullptr)
		{}

		void Construct(const FArguments& InArgs, UK2Node_GetSequenceBinding* InNode)
		{
			bNeedsUpdate = false;
			GraphNode = InNode;
			Initialize();
			UpdateGraphNode();
		}

		virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
		{
			UK2Node_GetSequenceBinding* Node = CastChecked<UK2Node_GetSequenceBinding>(GraphNode);

			if (bNeedsUpdate || Node->SourceMovieSequence != LastSequence.Get())
			{
				Initialize();
				UpdateGraphNode();

				bNeedsUpdate = false;
			}

			LastSequence = Node->SourceMovieSequence;

			SGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		}

		virtual void CreateStandardPinWidget(UEdGraphPin* Pin) override
		{
			if (Pin->PinName == SequencePinName)
			{
				CreateDetailsPickers();
			}
			else
			{
				SGraphNode::CreateStandardPinWidget(Pin);
			}
		}

		void OnAssetSelectedFromPicker(const FAssetData& AssetData)
		{
			CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->SetSequence(AssetData);
			Initialize();
			UpdateGraphNode();
		}

		FText GetAssetName() const
		{
			return CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->GetSequenceName();
		}

		TSharedRef<SWidget> GenerateAssetPicker()
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			FAssetPickerConfig AssetPickerConfig;
			AssetPickerConfig.Filter.ClassPaths.Add(UMovieSceneSequence::StaticClass()->GetClassPathName());
			AssetPickerConfig.bAllowNullSelection = true;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SGraphNodeGetSequenceBinding::OnAssetSelectedFromPicker);
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.bAllowDragging = false;

			return SNew(SBox)
				.HeightOverride(300)
				.WidthOverride(300)
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("Menu.Background") )
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					]
				];
		}

		FReply UseSelectedAsset()
		{
			UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(GEditor->GetSelectedObjects()->GetTop(UMovieSceneSequence::StaticClass()));
			if (Sequence)
			{
				CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->SetSequence(Sequence);
				Initialize();
				UpdateGraphNode();
			}
			return FReply::Handled();
		}

		FReply BrowseToAsset()
		{
			UMovieSceneSequence* Sequence = CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->SourceMovieSequence;
			if (Sequence)
			{
				TArray<UObject*> Objects{ Sequence };
				GEditor->SyncBrowserToObjects(Objects);
			}
			return FReply::Handled();
		}

		void CreateDetailsPickers()
		{
			LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(Settings->GetInputPinPadding())
			[
				SNew(SHorizontalBox)

				// Asset Combo
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2,0)
				.MaxWidth(200.0f)
				[
					SNew(SComboButton)
					.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
					.ForegroundColor(this, &SGraphNodeGetSequenceBinding::OnGetComboForeground)
					.ButtonColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetBackground)
					.ContentPadding(FMargin(2,2,2,1))
					.MenuPlacement(MenuPlacement_BelowAnchor)
					.ButtonContent()
					[
						SNew(STextBlock)
						.ColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetComboForeground)
						.TextStyle( FAppStyle::Get(), "PropertyEditor.AssetClass" )
						.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
						.Text( this, &SGraphNodeGetSequenceBinding::GetAssetName )
					]
					.OnGetMenuContent(this, &SGraphNodeGetSequenceBinding::GenerateAssetPicker)
				]

				// Use button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1,0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.OnClicked(this, &SGraphNodeGetSequenceBinding::UseSelectedAsset)
					.ButtonColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetBackground)
					.ContentPadding(1.f)
					.ToolTipText(LOCTEXT("GraphNodeGetSequenceBinding_Use_Tooltip", "Use asset browser selection"))
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetForeground)
						.Image( FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")) )
					]
				]

				// Browse button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1,0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.OnClicked(this, &SGraphNodeGetSequenceBinding::BrowseToAsset)
					.ButtonColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetBackground)
					.ContentPadding(0)
					.ToolTipText(LOCTEXT("GraphNodeGetSequenceBinding_Browse_Tooltip", "Browse"))
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetForeground)
						.Image( FAppStyle::GetBrush(TEXT("Icons.Search")) )
					]
				]
			];

			LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(Settings->GetInputPinPadding())
			[
				SNew(SBox)
				.MaxDesiredWidth(200.0f)
				.Padding(FMargin(2,0))
				[
					SNew(SComboButton)
					.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
					.ToolTipText(this, &SGraphNodeGetSequenceBinding::GetToolTipText)
					.ForegroundColor(this, &SGraphNodeGetSequenceBinding::OnGetComboForeground)
					.ButtonColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetWidgetBackground)
					.ContentPadding(FMargin(2,2,2,1))
					.MenuPlacement(MenuPlacement_BelowAnchor)
					.ButtonContent()
					[
						GetCurrentItemWidget(
							SNew(STextBlock)
							.TextStyle( FAppStyle::Get(), "PropertyEditor.AssetClass" )
							.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
							.ColorAndOpacity(this, &SGraphNodeGetSequenceBinding::OnGetComboForeground)
						)
					]
					.OnGetMenuContent(this, &SGraphNodeGetSequenceBinding::GetPickerMenu)
				]
			];
		}

	private:

		virtual void SetCurrentValue(const FMovieSceneObjectBindingID& InBindingId) override
		{
			const FScopedTransaction Transaction(LOCTEXT("SetBindng", "Set Binding"));
			GraphNode->Modify();

			CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->Binding = InBindingId;
			bNeedsUpdate = true;
		}
		virtual FMovieSceneObjectBindingID GetCurrentValue() const override
		{
			return CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->Binding;
		}
		virtual UMovieSceneSequence* GetSequence() const override
		{
			return CastChecked<UK2Node_GetSequenceBinding>(GraphNode)->SourceMovieSequence;
		}

		FSlateColor OnGetComboForeground() const
		{
			return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, IsHovered() ? 1.f : 0.6f ) );
		}

		FSlateColor OnGetWidgetForeground() const
		{
			return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, IsHovered() ? 1.f : 0.15f ) );
		}

		FSlateColor OnGetWidgetBackground() const
		{
			return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, IsHovered() ? 0.8f : 0.4f ) );
		}

	private:
		TWeakObjectPtr<UMovieSceneSequence> LastSequence;
		bool bNeedsUpdate;
	};

	return SNew(SGraphNodeGetSequenceBinding, this);
}

#undef LOCTEXT_NAMESPACE
