// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceFBXInterop.h"

#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Exporters/Exporter.h"
#include "FbxExporter.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ISequencer.h"
#include "MovieSceneToolHelpers.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "SequencerExportTask.h"
#include "SequencerUtilities.h"
#include "UObject/UObjectIterator.h"
#include "UnrealExporter.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "LevelSequenceFBXInterop"

FLevelSequenceFBXInterop::FLevelSequenceFBXInterop(TSharedRef<ISequencer> InSequencer)
	: WeakSequencer(InSequencer)
{
}

void FLevelSequenceFBXInterop::ImportFBX()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	using namespace UE::Sequencer;

	TMap<FGuid, FString> ObjectBindingNameMap;

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();
	TSharedPtr<FOutlinerViewModel> OutlinerViewModel = EditorViewModel->GetOutliner();
	TParentFirstChildIterator<IObjectBindingExtension> ObjectBindingIt = OutlinerViewModel->GetRootItem()->GetDescendantsOfType<IObjectBindingExtension>();

	// Only visit the first object binding in a given sub-hierarchy so we get top-level object bindings.
	// This is done by skipping the branch on each iteration
	for ( ; ObjectBindingIt; ++ObjectBindingIt)
	{
		FGuid ObjectBinding = ObjectBindingIt->GetObjectGuid();
		ObjectBindingNameMap.Add(ObjectBinding, (*ObjectBindingIt).AsModel()->CastThisChecked<IOutlinerExtension>()->GetLabel().ToString());

		ObjectBindingIt.IgnoreCurrentChildren();
	}

	MovieSceneToolHelpers::ImportFBXWithDialog(Sequencer->GetFocusedMovieSceneSequence(), *Sequencer, ObjectBindingNameMap, TOptional<bool>());
}

void FLevelSequenceFBXInterop::ImportFBXOntoSelectedNodes()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	using namespace UE::Sequencer;

	// The object binding and names to match when importing from fbx
	TMap<FGuid, FString> ObjectBindingNameMap;

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();
	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : EditorViewModel->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		FGuid ObjectBinding = ObjectBindingNode->GetObjectGuid();
		ObjectBindingNameMap.Add(ObjectBinding, ObjectBindingNode->GetLabel().ToString());
	}

	MovieSceneToolHelpers::ImportFBXWithDialog(Sequencer->GetFocusedMovieSceneSequence(), *Sequencer, ObjectBindingNameMap, TOptional<bool>(false));
}

void FLevelSequenceFBXInterop::ExportFBX()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	using namespace UE::Sequencer;

	TArray<UExporter*> Exporters;
	TArray<FString> SaveFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bExportFileNamePicked = false;
	if ( DesktopPlatform != NULL )
	{
		FString FileTypes = "FBX document (*.fbx)|*.fbx";
		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (!It->IsChildOf(UExporter::StaticClass()) || It->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				continue;
			}

			UExporter* Default = It->GetDefaultObject<UExporter>();
			if (!Default->SupportsObject(Sequence))
			{
				continue;
			}

			for (int32 i = 0; i < Default->FormatExtension.Num(); ++i)
			{
				// We use force-lowercase here to be consistent with "File > Export Selected..."
				const FString& FormatExtension = Default->FormatExtension[i].ToLower();
				const FString& FormatDescription = Default->FormatDescription[i];

				if (FileTypes.Len() > 0)
				{
					FileTypes += TEXT("|");
				}

				FileTypes += FString::Printf(TEXT("%s (*.%s)|*.%s"), *FormatDescription, *FormatExtension, *FormatExtension);
			}

			Exporters.Add(Default);
		}

		bExportFileNamePicked = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT( "ExportLevelSequence", "Export Level Sequence" ).ToString(),
			*( FEditorDirectories::Get().GetLastDirectory( ELastDirectory::FBX ) ),
			TEXT( "" ),
			*FileTypes,
			EFileDialogFlags::None,
			SaveFilenames );
	}

	if ( bExportFileNamePicked )
	{
		FString ExportFilename = SaveFilenames[0];
		FEditorDirectories::Get().SetLastDirectory( ELastDirectory::FBX, FPaths::GetPath( ExportFilename ) ); // Save path as default for next time.

		// Make sure external selection is up to date since export could happen on tracks that have been right clicked but not have their underlying bound objects selected yet since that happens on mouse up.
		FSequencerUtilities::SynchronizeExternalSelectionWithSequencerSelection(Sequencer.ToSharedRef());
		
		// Select selected nodes if there are selected nodes
		TArray<FGuid> Bindings;
		TArray<UMovieSceneTrack*> Tracks;
		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();
		for (FViewModelPtr Node : EditorViewModel->GetSelection()->Outliner)
		{
			if (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode = Node.ImplicitCast())
			{
				Bindings.Add(ObjectBindingNode->GetObjectGuid());

				for (TViewModelPtr<IOutlinerExtension> DescendantNode : Node->GetDescendantsOfType<IOutlinerExtension>())
				{
					if (!EditorViewModel->GetSelection()->Outliner.IsSelected(DescendantNode) && DescendantNode.AsModel()->IsA<IObjectBindingExtension>())
					{
						IObjectBindingExtension* DescendantObjectBindingNode = DescendantNode.AsModel()->CastThisChecked<IObjectBindingExtension>();
						Bindings.Add(DescendantObjectBindingNode->GetObjectGuid());
					}
				}
			}
			else if (TViewModelPtr<ITrackExtension> TrackNode = Node.ImplicitCast())
			{
				UMovieSceneTrack* Track = TrackNode->GetTrack();
				if (Track && MovieScene->ContainsTrack(*Track))
				{
					Tracks.Add(Track);
				}
			}
		}

		FString FileExtension = FPaths::GetExtension(ExportFilename);
		if (FileExtension == TEXT("fbx"))
		{
			ExportFBXInternal(ExportFilename, Bindings, (Bindings.Num() + Tracks.Num()) > 0 ? Tracks : MovieScene->GetTracks());
		}
		else
		{
			for (UExporter* Exporter : Exporters)
			{
				if (Exporter->FormatExtension.Contains(FileExtension))
				{
					USequencerExportTask* ExportTask = NewObject<USequencerExportTask>();
					TStrongObjectPtr<USequencerExportTask> ExportTaskGuard(ExportTask);
					ExportTask->Object = Sequencer->GetFocusedMovieSceneSequence();
					ExportTask->Exporter = nullptr;
					ExportTask->Filename = ExportFilename;
					ExportTask->bSelected = false;
					ExportTask->bReplaceIdentical = true;
					ExportTask->bPrompt = false;
					ExportTask->bUseFileArchive = false;
					ExportTask->bWriteEmptyFiles = false;
					ExportTask->bAutomated = false;
					ExportTask->Exporter = NewObject<UExporter>(GetTransientPackage(), Exporter->GetClass());

					ExportTask->SequencerContext = Sequencer->GetPlaybackContext();

					UExporter::RunAssetExportTask(ExportTask);

					ExportTask->Object = nullptr;
					ExportTask->Exporter = nullptr;
					ExportTask->SequencerContext = nullptr;

					break;
				}
			}
		}
	}
}

void FLevelSequenceFBXInterop::ExportFBXInternal(const FString& ExportFilename, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();
	//Show the fbx export dialog options
	bool ExportCancel = false;
	bool ExportAll = false;
	Exporter->FillExportOptions(false, true, ExportFilename, ExportCancel, ExportAll);
	if (!ExportCancel)
	{
		UMovieSceneSequence* MovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
		UMovieSceneSequence* RootMovieSceneSequence = Sequencer->GetRootMovieSceneSequence();
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		UObject* PlaybackContext = Sequencer->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
		UnFbx::FFbxExporter::FLevelSequenceNodeNameAdapter NodeNameAdapter(MovieScene, Sequencer.Get(), Template);

		{
			FAllSpawnableRestoreState SpawnableRestoreState(Sequencer->GetSharedPlaybackState().ToSharedPtr());
			if (SpawnableRestoreState.bWasChanged)
			{
				// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
				Sequencer->SetLocalTimeDirectly(UE::MovieScene::DiscreteInclusiveLower(FSequencerUtilities::GetTimeBounds(Sequencer.ToSharedRef())));
			}

			FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransform();
			FAnimExportSequenceParameters AESP;
			AESP.Player = Sequencer.Get();
			AESP.RootToLocalTransform = RootToLocalTransform;
			AESP.MovieSceneSequence = MovieSceneSequence;
			AESP.RootMovieSceneSequence = RootMovieSceneSequence;
			if (MovieSceneToolHelpers::ExportFBX(World, AESP, Bindings, Tracks, NodeNameAdapter, Template, ExportFilename))
			{
				FNotificationInfo Info(NSLOCTEXT("Sequencer", "ExportFBXSucceeded", "FBX Export Succeeded."));
				Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString InFilename) { FPlatformProcess::ExploreFolder(*InFilename); }, ExportFilename);
				Info.HyperlinkText = FText::FromString(ExportFilename);
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Success);
			}
			else
			{
				FNotificationInfo Info(NSLOCTEXT("Sequencer", "ExportFBXFailed", "FBX Export Failed."));
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}

		Sequencer->ForceEvaluate();
	}
}

#undef LOCTEXT_NAMESPACE
