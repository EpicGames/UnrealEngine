// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/CameraCutTrackModel.h"
#include "MVVM/ViewModels/CinematicShotTrackModel.h"
#include "MVVM/ViewModels/BindingLifetimeTrackModel.h"
#include "MVVM/ViewModels/TimeWarpChannelModel.h"
#include "MVVM/ViewModels/ScalingAnchorsModel.h"


namespace UE
{
namespace Sequencer
{

// Model types
UE_SEQUENCER_DEFINE_CASTABLE(FCameraCutTrackModel);
UE_SEQUENCER_DEFINE_CASTABLE(FCinematicShotTrackModel);
UE_SEQUENCER_DEFINE_CASTABLE(FBindingLifetimeTrackModel);
UE_SEQUENCER_DEFINE_CASTABLE(FTimeWarpChannelModel);
UE_SEQUENCER_DEFINE_CASTABLE(FScalingAnchorsModel);

} // namespace Sequencer
} // namespace UE

