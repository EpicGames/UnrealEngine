// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/BindingLifetimeTrackModel.h"

#include "MovieSceneTrack.h"
#include "Templates/Casts.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"

namespace UE
{
	namespace Sequencer
	{

		TSharedPtr<FTrackModel> FBindingLifetimeTrackModel::CreateTrackModel(UMovieSceneTrack* Track)
		{
			if (UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(Track))
			{
				return MakeShared<FBindingLifetimeTrackModel>(BindingLifetimeTrack);
			}
			return nullptr;
		}

		FBindingLifetimeTrackModel::FBindingLifetimeTrackModel(UMovieSceneBindingLifetimeTrack* Track)
			: FTrackModel(Track)
		{
		}

		void FBindingLifetimeTrackModel::OnConstruct()
		{
			FTrackModel::OnConstruct();
			RecalculateInverseLifetimeRange();
		}

		void FBindingLifetimeTrackModel::RecalculateInverseLifetimeRange()
		{
			TArray<FFrameNumberRange> SectionRanges;
			Algo::Transform(GetSectionModels().IterateSubList<FSectionModel>(), SectionRanges, [](const TViewModelPtr<FSectionModel>& Item) { return Item->GetRange(); });
			
			InverseLifetimeRange = UMovieSceneBindingLifetimeTrack::CalculateInverseLifetimeRange(SectionRanges);
		}

		FSortingKey FBindingLifetimeTrackModel::GetSortingKey() const
		{
			FSortingKey SortingKey;
			if (UMovieSceneTrack* Track = GetTrack())
			{
				SortingKey.CustomOrder = Track->GetSortingOrder();
			}
			return SortingKey.PrioritizeBy(4);
		}

		void FBindingLifetimeTrackModel::OnDeferredModifyFlush()
		{
			// Do the child update
			FTrackModel::OnDeferredModifyFlush();

			// Recalculate our valid range
			RecalculateInverseLifetimeRange();
		}


	} // namespace Sequencer
} // namespace UE

