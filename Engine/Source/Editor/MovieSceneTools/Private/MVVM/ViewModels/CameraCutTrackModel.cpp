// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/CameraCutTrackModel.h"

#include "MovieSceneTrack.h"
#include "Templates/Casts.h"
#include "Tracks/MovieSceneCameraCutTrack.h"

namespace UE
{
namespace Sequencer
{

TSharedPtr<FTrackModel> FCameraCutTrackModel::CreateTrackModel(UMovieSceneTrack* Track)
{
	if (UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(Track))
	{
		return MakeShared<FCameraCutTrackModel>(CameraCutTrack);
	}
	return nullptr;
}

FCameraCutTrackModel::FCameraCutTrackModel(UMovieSceneCameraCutTrack* Track)
	: FTrackModel(Track)
{
}

FSortingKey FCameraCutTrackModel::GetSortingKey() const
{
	FSortingKey SortingKey;
	if (UMovieSceneTrack* Track = GetTrack())
	{
		SortingKey.CustomOrder = Track->GetSortingOrder();
	}
	return SortingKey.PrioritizeBy(4);
}

} // namespace Sequencer
} // namespace UE

