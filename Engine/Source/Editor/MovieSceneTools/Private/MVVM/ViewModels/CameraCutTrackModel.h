// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "Templates/SharedPointer.h"

#define UE_API MOVIESCENETOOLS_API

class UMovieSceneCameraCutTrack;
class UMovieSceneTrack;
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }

namespace UE
{
namespace Sequencer
{

class FCameraCutTrackModel
	: public FTrackModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FCameraCutTrackModel, FTrackModel);

	static UE_API TSharedPtr<FTrackModel> CreateTrackModel(UMovieSceneTrack* Track);

	UE_API explicit FCameraCutTrackModel(UMovieSceneCameraCutTrack* Track);

	UE_API FSortingKey GetSortingKey() const override;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
