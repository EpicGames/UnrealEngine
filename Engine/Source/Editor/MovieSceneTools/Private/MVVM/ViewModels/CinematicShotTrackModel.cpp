// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/CinematicShotTrackModel.h"

#include "MovieSceneTrack.h"
#include "Templates/Casts.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"

namespace UE
{
namespace Sequencer
{

TSharedPtr<FTrackModel> FCinematicShotTrackModel::CreateTrackModel(UMovieSceneTrack* Track)
{
	if (UMovieSceneCinematicShotTrack* CinematicShotTrack = Cast<UMovieSceneCinematicShotTrack>(Track))
	{
		return MakeShared<FCinematicShotTrackModel>(CinematicShotTrack);
	}
	return nullptr;
}

FCinematicShotTrackModel::FCinematicShotTrackModel(UMovieSceneCinematicShotTrack* Track)
	: FTrackModel(Track)
{
}

FSortingKey FCinematicShotTrackModel::GetSortingKey() const
{
	FSortingKey SortingKey;
	if (UMovieSceneTrack* Track = GetTrack())
	{
		SortingKey.CustomOrder = Track->GetSortingOrder();
	}
	return SortingKey.PrioritizeBy(5);
}

} // namespace Sequencer
} // namespace UE

