// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ScalingAnchorsModel.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MVVM/Extensions/IViewSpaceClientExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewSpace.h"
#include "SequencerToolMenuContext.h"
#include "ToolMenus.h"
#include "Decorations/MovieSceneScalingAnchors.h"


namespace UE::Sequencer
{

void FScalingAnchorsModel::InitializeObject(TWeakObjectPtr<> InWeakObject)
{
	WeakAnchors = CastChecked<UMovieSceneScalingAnchors>(InWeakObject.Get());

	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("Sequencer.SectionContextMenu");
	ToolMenu->AddDynamicSection(
		NAME_None,
		FNewToolMenuDelegate::CreateSP(this, &FScalingAnchorsModel::ExtendSectionMenu),
		FToolMenuInsert(NAME_None, EToolMenuInsertType::After)
	);
}

UObject* FScalingAnchorsModel::GetObject() const
{
	return WeakAnchors.Get();
}

void FScalingAnchorsModel::ExtendSectionMenu(UToolMenu* InMenu)
{
	USequencerToolMenuContext* ContextObject = InMenu->FindContext<USequencerToolMenuContext>();
	TWeakPtr<ISequencer>       WeakSequencer = ContextObject ? ContextObject->WeakSequencer : nullptr;
	TSharedPtr<ISequencer>     Sequencer     = WeakSequencer.Pin();

	if (!Sequencer || Sequencer->GetViewModel()->GetRootModel() != FViewModelPtr(FindAncestorOfType<FSequenceModel>()))
	{
		return;
	}

	FToolMenuSection& ScalingSection = InMenu->AddSection(TEXT("Scaling"));

	ScalingSection.AddMenuEntry(
		NAME_None,
		NSLOCTEXT("FScalingAnchorsModel", "CreateScalingGroup", "Create Scaling Group"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FScalingAnchorsModel::CreateScalingGroup, WeakSequencer) 
		)
	);
}

void FScalingAnchorsModel::CreateScalingGroup(TWeakPtr<ISequencer> InWeakSequencer)
{
	UMovieSceneScalingAnchors* Anchors   = WeakAnchors.Get();
	TSharedPtr<ISequencer>     Sequencer = InWeakSequencer.Pin();

	if (Sequencer && Anchors)
	{
		FGuid NewGuid = FGuid::NewGuid();

		FMovieSceneAnchorsScalingGroup& Group = Anchors->GetOrCreateScalingGroup(NewGuid);
		for (TSharedPtr<FSectionModel> Section : Sequencer->GetViewModel()->GetSelection()->TrackArea.Filter<FSectionModel>())
		{
			Group.Sections.Add(Section->GetSection());
		}
	}
}

void FScalingAnchorsModel::UpdateViewSpaces(FTrackAreaViewModel& TrackAreaViewModel)
{
	using namespace UE::MovieScene;

	FSectionModelStorageExtension* SectionModelStorage = TrackAreaViewModel.GetEditor()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();

	UMovieSceneScalingAnchors* Anchors = WeakAnchors.Get();
	if (Anchors)
	{
		class FAnchorsSpace : public FTrackAreaViewSpace
		{
		public:
			FAnchorsSpace(TWeakObjectPtr<UMovieSceneScalingAnchors> InWeakAnchors, TWeakPtr<ISequencer> InWeakSequencer)
				: WeakAnchors(InWeakAnchors)
				, WeakSequencer(InWeakSequencer)
			{
			}

#if 0
			UMovieSceneTimeWarpGetter* FindTimeWarp() const
			{
				UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
				if (!Linker || AnchorEntity.IsEmpty() || !AnchorEntity.IsValid(Linker->EntityManager))
				{
					TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
					if (!Sequencer)
					{
						WeakLinker = nullptr;
						AnchorEntity = FEntityHandle();
						return nullptr;
					}

					FMovieSceneRootEvaluationTemplateInstance& Root = Sequencer->GetEvaluationTemplate();
					Linker = Root.GetEntitySystemLinker();

					const FSequenceInstance* SequenceInstance = Root.FindInstance(Sequencer->GetFocusedTemplateID());
					if (SequenceInstance)
					{
						FMovieSceneEntityID Entity = SequenceInstance->FindEntity(WeakAnchors.Get(), UMovieSceneScalingAnchors::DefaultTimeWarpEntityID);
						if (Entity)
						{
							AnchorEntity = Linker->EntityManager.GetEntityHandle(Entity);
							WeakLinker = Linker;
						}
					}
				}

				if (AnchorEntity.IsValid(Linker->EntityManager))
				{
					TOptionalComponentReader<TObjectPtr<UMovieSceneTimeWarpGetter>> TimeWarp = Linker->EntityManager.ReadComponent(AnchorEntity.GetEntityID(), FBuiltInComponentTypes::Get()->TimeWarp);
					return TimeWarp ? *TimeWarp : TObjectPtr<UMovieSceneTimeWarpGetter>();
				}
				return nullptr;
			}
#endif

			virtual double SourceToView(double SourceTime) const
			{
				using namespace UE::MovieScene;

				UMovieSceneScalingAnchors* Anchors = WeakAnchors.Get();
				UMovieSceneTimeWarpGetter* DefaultTimeWarp = Anchors;
				if (!DefaultTimeWarp)
				{
					return SourceTime;
				}

				FFrameRate TickResolution = Anchors->GetTypedOuter<UMovieScene>()->GetTickResolution();

				TOptional<FFrameTime> Time = DefaultTimeWarp->InverseRemapTimeCycled(SourceTime * TickResolution, 0, FInverseTransformTimeParams());
				return Time ? Time.GetValue() / TickResolution : SourceTime;
			}

			virtual double ViewToSource(double ViewTime) const
			{
				UMovieSceneScalingAnchors* Anchors = WeakAnchors.Get();
				UMovieSceneTimeWarpGetter* DefaultTimeWarp = Anchors;
				if (!DefaultTimeWarp)
				{
					return ViewTime;
				}
				FFrameRate TickResolution = Anchors->GetTypedOuter<UMovieScene>()->GetTickResolution();
				return DefaultTimeWarp->RemapTime(ViewTime * TickResolution) / TickResolution;
			}

			mutable TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
			TWeakObjectPtr<UMovieSceneScalingAnchors> WeakAnchors;
			TWeakPtr<ISequencer> WeakSequencer;
			mutable FEntityHandle AnchorEntity;
		};


		TSharedPtr<ISequencer> Sequencer = TrackAreaViewModel.GetEditor()->CastThis<FSequencerEditorViewModel>()->GetSequencer();
		TSharedPtr<FTrackAreaViewSpace> DefaultViewSpace = MakeShared<FAnchorsSpace>(WeakAnchors, Sequencer);
		TrackAreaViewModel.AddViewSpace(FGuid(), DefaultViewSpace);

#if 0
		if (SectionModelStorage)
		{
			for (const TPair<FGuid, FMovieSceneAnchorsScalingGroup>& Pair : Anchors->GetScalingGroups())
			{
				// @todo: make a way for the view space to be well defined
				TSharedPtr<FTrackAreaViewSpace> NewViewSpace = MakeShared<FAnchorsSpace>(WeakAnchors, Sequencer);
				TrackAreaViewModel.AddViewSpace(Pair.Key, NewViewSpace);

				for (TObjectPtr<UMovieSceneSection> Section : Pair.Value.Sections)
				{
					TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(Section);
					if (SectionModel)
					{
						FViewSpaceClientExtensionShim& ViewSpaceExt = SectionModel->AddDynamicExtension<FViewSpaceClientExtensionShim>(Pair.Key);
						ViewSpaceExt.SetViewSpaceID(Pair.Key);
					}
				}
			}
		}
#endif
	}
}

} // namespace UE::Sequencer

