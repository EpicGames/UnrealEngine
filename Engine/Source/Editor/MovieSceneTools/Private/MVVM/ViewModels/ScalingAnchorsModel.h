// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IObjectModelExtension.h"
#include "MVVM/Extensions/ITrackAreaViewSpaceProviderExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MOVIESCENETOOLS_API

class ISequencer;
class UMovieSceneScalingAnchors;
class UToolMenu;

namespace UE::Sequencer
{

class FTrackAreaViewModel;

class FScalingAnchorsModel
	: public FViewModel
	, public IObjectModelExtension
	, public ITrackAreaViewSpaceProviderExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FScalingAnchorsModel, FViewModel, IObjectModelExtension, ITrackAreaViewSpaceProviderExtension);

	/*~ Begin IObjectModelExtension */
	UE_API virtual void InitializeObject(TWeakObjectPtr<> InWeakObject) override;
	UE_API virtual UObject* GetObject() const override;
	/*~ End IObjectModelExtension */

	/*~ Begin ITrackAreaViewSpaceProviderExtension */
	UE_API virtual void UpdateViewSpaces(FTrackAreaViewModel& TrackAreaViewModel) override;
	/*~ End ITrackAreaViewSpaceProviderExtension */

private:

	UE_API void ExtendSectionMenu(UToolMenu* InMenu);
	UE_API void CreateScalingGroup(TWeakPtr<ISequencer> InWeakSequencer);

private:

	TWeakObjectPtr<UMovieSceneScalingAnchors> WeakAnchors;
};


} // namespace UE::Sequencer

#undef UE_API
