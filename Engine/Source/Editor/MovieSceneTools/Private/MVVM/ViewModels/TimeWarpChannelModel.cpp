// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TimeWarpChannelModel.h"

#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Views/ViewUtilities.h"
#include "SequencerUtilities.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "TimeWarpChannelModel"

namespace UE::Sequencer
{

FTimeWarpChannelModel::FTimeWarpChannelModel(FName InChannelName, TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel)
	: FChannelModel(InChannelName, InSection, InChannel)
{}


TSharedPtr<SWidget> FTimeWarpChannelModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	TViewModelPtr<IOutlinerExtension> OutlinerItem = GetLinkedOutlinerItem();

	if (InColumnName == FCommonOutlinerNames::Add)
	{
		return MakeButton(
			LOCTEXT("ChangeTimeWarpToolTip", "Change Time Warp to utilize a different curve type"),
			FAppStyle::GetBrush("Sequencer.Outliner.Indicators.TimeWarp"),
			FOnGetContent::CreateSP(this, &FTimeWarpChannelModel::BuildReplaceTimeWarpSubMenu),
			OutlinerItem.AsModel());
	}

	return nullptr;
}


void FTimeWarpChannelModel::BuildContextMenu(FMenuBuilder& MenuBuilder, TViewModelPtr<FChannelGroupOutlinerModel> GroupOwner)
{
	TViewModelPtr<ITrackExtension> Track = GetLinkedOutlinerItem().AsModel()->FindAncestorOfType<ITrackExtension>();
	if (Track)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ReplaceTimeWarp", "Replace With"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FTimeWarpChannelModel::PopulateReplaceTimeWarpSubMenu)
		);
	}
}


TSharedRef<SWidget> FTimeWarpChannelModel::BuildReplaceTimeWarpSubMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	PopulateReplaceTimeWarpSubMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}


void FTimeWarpChannelModel::PopulateReplaceTimeWarpSubMenu(FMenuBuilder& MenuBuilder)
{
	TViewModelPtr<ISectionOwnerExtension> SectionOwner = GetLinkedOutlinerItem().AsModel()->FindAncestorOfType<ISectionOwnerExtension>();
	if (SectionOwner)
	{
		FSequencerUtilities::PopulateTimeWarpChannelSubMenu(MenuBuilder, SectionOwner.AsWeak());
	}
}


} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE