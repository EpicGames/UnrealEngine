// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

#include "MVVM/ViewModels/ChannelModel.h"


struct FMovieSceneChannelHandle;

class FName;
class ISequencerSection;


namespace UE::Sequencer
{

struct FTimeWarpChannelModel : FChannelModel
{
	UE_SEQUENCER_DECLARE_CASTABLE(FTimeWarpChannelModel, FChannelModel);

	FTimeWarpChannelModel(FName InChannelName, TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel);

	/*~ Begin FChannelModel API */
	TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	void BuildContextMenu(FMenuBuilder& MenuBuilder, TViewModelPtr<FChannelGroupOutlinerModel> GroupOwner) override;
	/*~ End FChannelModel API */

private:

	TSharedRef<SWidget> BuildReplaceTimeWarpSubMenu();
	void PopulateReplaceTimeWarpSubMenu(FMenuBuilder& MenuBuilder);
};

} // namespace UE::Sequencer

