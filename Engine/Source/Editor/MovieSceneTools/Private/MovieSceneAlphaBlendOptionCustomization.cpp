// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAlphaBlendOptionCustomization.h"
#include "MovieSceneBuiltInEasingFunctionGridWidget.h"

#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailChildrenBuilder.h"
#include "MovieSceneBuiltInEasingFunctionCustomization.h"

void FAlphaBlendPropertyCustomization::SetAlphaBlendOption(const EAlphaBlendOption NewBlendOption)
{
	FScopedTransaction Transaction(NSLOCTEXT("AlphaBlendCustomization", "SetBlendMode", "Set Blend Mode"));

	TypeProperty->NotifyPreChange();

	TArray<void*> RawData;
	TypeProperty->AccessRawData(RawData);

	for (void* Ptr : RawData)
	{
		*((EAlphaBlendOption*)Ptr) = NewBlendOption;
	}

	ButtonContent->SetType(ConvertToEasingType(NewBlendOption));

	TypeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	TypeProperty->NotifyFinishedChangingProperties();
}

EAlphaBlendOption FAlphaBlendPropertyCustomization::GetAlphaBlendOption() const
{
	if (TypeProperty)
	{
		TArray<void*> RawData;
		TypeProperty->AccessRawData(RawData);

		if (RawData.Num() > 0)
		{
			return *((EAlphaBlendOption*)RawData[0]);
		}
	}

	return EAlphaBlendOption::Linear;
}

void FAlphaBlendPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TypeProperty = StructPropertyHandle;

	// Exclude all unsupported curve types
	TSet<EMovieSceneBuiltInEasing> FilterExclude;
	FilterExclude.Add(EMovieSceneBuiltInEasing::SinIn);
	FilterExclude.Add(EMovieSceneBuiltInEasing::SinOut);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuadIn);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuadOut);
	FilterExclude.Add(EMovieSceneBuiltInEasing::CubicIn);
	FilterExclude.Add(EMovieSceneBuiltInEasing::CubicOut);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuartIn);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuartOut);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuintIn);
	FilterExclude.Add(EMovieSceneBuiltInEasing::QuintOut);

	ButtonContent = SNew(SBuiltInFunctionVisualizerWithText, ConvertToEasingType(FAlphaBlendPropertyCustomization::GetAlphaBlendOption()));

	TSharedRef<SComboButton> ComboBox = SNew(SComboButton)
		.ToolTipText(StructPropertyHandle->GetToolTipText())
		// Show the currently used curve type inside the combo box
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		// The grid of curve widgets, that appears after clicking the combo box
		.MenuContent()
		[
			SNew(SEasingFunctionGridWidget)
			.FilterExclude(FilterExclude)
			.OnTypeChanged(this, &FAlphaBlendPropertyCustomization::SetType)
		];

	FDetailWidgetRow& Row = ChildBuilder.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(StructPropertyHandle->GetPropertyDisplayName())
			.ToolTipText(StructPropertyHandle->GetToolTipText())
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			ComboBox
		];
}

void FAlphaBlendPropertyCustomization::SetType(EMovieSceneBuiltInEasing NewType)
{
	SetAlphaBlendOption(ConvertFromEasingType(NewType));
}

EMovieSceneBuiltInEasing FAlphaBlendPropertyCustomization::ConvertToEasingType(EAlphaBlendOption in) const
{
	switch (in)
	{
	case EAlphaBlendOption::Linear: return EMovieSceneBuiltInEasing::Linear;
	case EAlphaBlendOption::Cubic: return EMovieSceneBuiltInEasing::Cubic;
	case EAlphaBlendOption::CubicInOut: return EMovieSceneBuiltInEasing::CubicInOut;
	case EAlphaBlendOption::HermiteCubic: return EMovieSceneBuiltInEasing::HermiteCubicInOut;
	case EAlphaBlendOption::Sinusoidal: return EMovieSceneBuiltInEasing::SinInOut;
	case EAlphaBlendOption::QuadraticInOut: return EMovieSceneBuiltInEasing::QuadInOut;
	case EAlphaBlendOption::QuarticInOut: return EMovieSceneBuiltInEasing::QuartInOut;
	case EAlphaBlendOption::QuinticInOut: return EMovieSceneBuiltInEasing::QuintInOut;
	case EAlphaBlendOption::CircularIn: return EMovieSceneBuiltInEasing::CircIn;
	case EAlphaBlendOption::CircularOut: return EMovieSceneBuiltInEasing::CircOut;
	case EAlphaBlendOption::CircularInOut: return EMovieSceneBuiltInEasing::CircInOut;
	case EAlphaBlendOption::ExpIn: return EMovieSceneBuiltInEasing::ExpoIn;
	case EAlphaBlendOption::ExpOut: return EMovieSceneBuiltInEasing::ExpoOut;
	case EAlphaBlendOption::ExpInOut: return EMovieSceneBuiltInEasing::ExpoInOut;
	case EAlphaBlendOption::Custom: return EMovieSceneBuiltInEasing::Custom;
	}

	return EMovieSceneBuiltInEasing::Linear;
}

EAlphaBlendOption FAlphaBlendPropertyCustomization::ConvertFromEasingType(EMovieSceneBuiltInEasing in) const
{
	switch (in)
	{
	case EMovieSceneBuiltInEasing::Linear: return EAlphaBlendOption::Linear;
	case EMovieSceneBuiltInEasing::Cubic: return EAlphaBlendOption::Cubic;
	case EMovieSceneBuiltInEasing::CubicInOut: return EAlphaBlendOption::CubicInOut;
	case EMovieSceneBuiltInEasing::HermiteCubicInOut: return EAlphaBlendOption::HermiteCubic;
	case EMovieSceneBuiltInEasing::SinInOut: return EAlphaBlendOption::Sinusoidal;
	case EMovieSceneBuiltInEasing::QuadInOut: return EAlphaBlendOption::QuadraticInOut;
	case EMovieSceneBuiltInEasing::QuartInOut: return EAlphaBlendOption::QuarticInOut;
	case EMovieSceneBuiltInEasing::QuintInOut: return EAlphaBlendOption::QuinticInOut;
	case EMovieSceneBuiltInEasing::CircIn: return EAlphaBlendOption::CircularIn;
	case EMovieSceneBuiltInEasing::CircOut: return EAlphaBlendOption::CircularOut;
	case EMovieSceneBuiltInEasing::CircInOut: return EAlphaBlendOption::CircularInOut;
	case EMovieSceneBuiltInEasing::ExpoIn: return EAlphaBlendOption::ExpIn;
	case EMovieSceneBuiltInEasing::ExpoOut: return EAlphaBlendOption::ExpOut;
	case EMovieSceneBuiltInEasing::ExpoInOut: return EAlphaBlendOption::ExpInOut;
	case EMovieSceneBuiltInEasing::Custom: return EAlphaBlendOption::Custom;
	}

	return EAlphaBlendOption::Linear;
}