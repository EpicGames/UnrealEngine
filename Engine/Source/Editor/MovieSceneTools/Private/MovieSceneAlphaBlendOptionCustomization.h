// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "Runtime/Engine/Public/AlphaBlend.h"

class SBuiltInFunctionVisualizerWithText;

class FAlphaBlendPropertyCustomization : public IPropertyTypeCustomization
{
public:
	virtual ~FAlphaBlendPropertyCustomization() = default;

	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override final {}
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	void SetAlphaBlendOption(const EAlphaBlendOption NewBlendOption);
	EAlphaBlendOption GetAlphaBlendOption() const;

private:
	void SetType(EMovieSceneBuiltInEasing NewType);
	EMovieSceneBuiltInEasing ConvertToEasingType(EAlphaBlendOption In) const;
	EAlphaBlendOption ConvertFromEasingType(EMovieSceneBuiltInEasing In) const;

	TSharedPtr<IPropertyHandle> TypeProperty;
	TSharedPtr<SBuiltInFunctionVisualizerWithText> ButtonContent;
};