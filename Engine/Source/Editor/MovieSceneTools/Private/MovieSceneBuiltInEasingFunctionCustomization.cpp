// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBuiltInEasingFunctionCustomization.h"
#include "MovieSceneBuiltInEasingFunctionGridWidget.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailChildrenBuilder.h"

class FActiveTimerHandle;
class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FPointerEvent;
struct FSlateBrush;

void FMovieSceneBuiltInEasingFunctionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneBuiltInEasingFunction, Type));

	TSet<EMovieSceneBuiltInEasing> FilterExclude;
	FilterExclude.Add(EMovieSceneBuiltInEasing::Custom);
	FilterExclude.Add(EMovieSceneBuiltInEasing::Cubic);

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Easing");
	DetailBuilder.HideProperty(TypeProperty);
	FDetailWidgetRow& Row = Category.AddCustomRow(FText());
	Row.WholeRowContent()
	[
		SNew(SEasingFunctionGridWidget)
		.FilterExclude(FilterExclude)
		.OnTypeChanged(this, &FMovieSceneBuiltInEasingFunctionCustomization::SetType)
	];
}

void FMovieSceneBuiltInEasingFunctionCustomization::SetType(EMovieSceneBuiltInEasing NewType)
{
	FScopedTransaction Transaction(NSLOCTEXT("EasingFunctionCustomization", "SetEasingType", "Set Easing Type"));	

	TypeProperty->NotifyPreChange();

	TArray<void*> RawData;
	TypeProperty->AccessRawData(RawData);

	for (void* Ptr : RawData)
	{
		*((EMovieSceneBuiltInEasing*)Ptr) = NewType;
	}

	TypeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	TypeProperty->NotifyFinishedChangingProperties();
}

///////////////////////////////////////////////////////////////////////////////

void SBuiltInFunctionVisualizerWithText::Construct(const FArguments& InArgs, EMovieSceneBuiltInEasing InValue)
{
	const UEnum* EasingEnum = StaticEnum<EMovieSceneBuiltInEasing>();
	check(EasingEnum);

	FunctionVisualizer = SNew(SBuiltInFunctionVisualizer, InValue);
	FunctionName = SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(EasingEnum->GetDisplayNameTextByValue((int64)InValue));

	ChildSlot
		[
			SNew(SVerticalBox)

			// Add curve
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(100.0f)
				.HeightOverride(50.0f)
				[
					FunctionVisualizer.ToSharedRef()
				]
			]

			// Add curve type name
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			[
				FunctionName.ToSharedRef()
			]
		];
}

void SBuiltInFunctionVisualizerWithText::SetType(EMovieSceneBuiltInEasing InValue)
{
	FunctionVisualizer->SetType(InValue);

	const UEnum* EasingEnum = StaticEnum<EMovieSceneBuiltInEasing>();
	check(EasingEnum);
	FunctionName->SetText(EasingEnum->GetDisplayNameTextByValue((int64)InValue));
}