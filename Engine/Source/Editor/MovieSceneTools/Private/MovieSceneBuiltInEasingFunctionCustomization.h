// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "MovieSceneBuiltInEasingFunctionVisualizer.h"

class IDetailLayoutBuilder;

enum class EMovieSceneBuiltInEasing : uint8;
class FReply;
class IPropertyHandle;

class FMovieSceneBuiltInEasingFunctionCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void SetType(EMovieSceneBuiltInEasing NewType);

private:
	TSharedPtr<IPropertyHandle> TypeProperty;
};

/** Widget showing the curve graph, and the curve name below it */
class SBuiltInFunctionVisualizerWithText : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBuiltInFunctionVisualizerWithText) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, EMovieSceneBuiltInEasing InValue);
	void SetType(EMovieSceneBuiltInEasing InValue);

private:
	TSharedPtr<SBuiltInFunctionVisualizer> FunctionVisualizer;
	TSharedPtr<STextBlock> FunctionName;
};