// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBuiltInEasingFunctionGridWidget.h"
#include "MovieSceneBuiltInEasingFunctionVisualizer.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailChildrenBuilder.h"

class FActiveTimerHandle;
class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FPointerEvent;
struct FSlateBrush;

SEasingFunctionGridWidget::FGroup& SEasingFunctionGridWidget::FindOrAddGroup(TArray<FGroup>& Groups, const FString& GroupName)
{
	for (int32 Index = Groups.Num() - 1; Index >= 0; --Index)
	{
		if (Groups[Index].GroupName == GroupName)
		{
			return Groups[Index];
		}
	}

	Groups.Emplace();
	Groups.Last().GroupName = GroupName;
	return Groups.Last();
};

TArray<SEasingFunctionGridWidget::FGroup> SEasingFunctionGridWidget::ConstructGroups(const TSet<EMovieSceneBuiltInEasing>& FilterExclude)
{
	const UEnum* EasingEnum = StaticEnum<EMovieSceneBuiltInEasing>();
	check(EasingEnum)

	TArray<FGroup> Groups;

	for (int32 NameIndex = 0; NameIndex < EasingEnum->NumEnums() - 1; ++NameIndex)
	{
		const FString& Grouping = EasingEnum->GetMetaData(TEXT("Grouping"), NameIndex);
		EMovieSceneBuiltInEasing Value = (EMovieSceneBuiltInEasing)EasingEnum->GetValueByIndex(NameIndex);

		if (FilterExclude.IsEmpty() || FilterExclude.Find(Value) == nullptr)
		{
			FindOrAddGroup(Groups, Grouping).Values.Add(Value);
		}
	}

	return Groups;
}

FReply SEasingFunctionGridWidget::OnTypeButtonClicked(EMovieSceneBuiltInEasing type)
{
	OnClickedDelegate.ExecuteIfBound(type);
	return FReply::Handled();
}

void SEasingFunctionGridWidget::Construct(const FArguments& InArgs)
{
	FilterExcludeAttribute = InArgs._FilterExclude;
	OnClickedDelegate = InArgs._OnTypeChanged;

	const UEnum* EasingEnum = StaticEnum<EMovieSceneBuiltInEasing>();
	check(EasingEnum)

	TArray<FGroup> Groups = ConstructGroups(FilterExcludeAttribute.Get());

	TSharedRef<SGridPanel> Grid = SNew(SGridPanel);

	int32 RowIndex = 0;
	for (const FGroup& Group : Groups)
	{
		for (int32 ColumnIndex = 0; ColumnIndex < Group.Values.Num(); ++ColumnIndex)
		{
			EMovieSceneBuiltInEasing Value = Group.Values[ColumnIndex];

			Grid->AddSlot(ColumnIndex, RowIndex)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &SEasingFunctionGridWidget::OnTypeButtonClicked, Value)
					[
						SNew(SBox)
						.WidthOverride(100.f)
						.HeightOverride(50.f)
						[
							SNew(SBuiltInFunctionVisualizer, Value)
						]
					]
				];

			Grid->AddSlot(ColumnIndex, RowIndex + 1)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(EasingEnum->GetDisplayNameTextByValue((int64)Value))
				];
		}

		RowIndex += 2;
	}

	ChildSlot
		[
			Grid
		];
}