// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "Widgets/SCompoundWidget.h"

class IDetailLayoutBuilder;

enum class EMovieSceneBuiltInEasing : uint8;
class FReply;
class IPropertyHandle;

DECLARE_DELEGATE_OneParam(FEasingFunctionGridWidget_OnClicked, EMovieSceneBuiltInEasing);

/** Widget showing a grid of curves including their names.
 * Curves are grouped within rows, e.g. Exponential in, out, in-out in one row.
 * A filter can be used to exclude several curves from the grid.
 */
class SEasingFunctionGridWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEasingFunctionGridWidget) {}

	/** The easing curve filter containing all curve types that should be excluded.In case the filter is empty, all curve types will be shown. */
	SLATE_ATTRIBUTE(TSet<EMovieSceneBuiltInEasing>, FilterExclude)
	SLATE_EVENT(FEasingFunctionGridWidget_OnClicked, OnTypeChanged)
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	struct FGroup
	{
		FString GroupName;
		TArray<EMovieSceneBuiltInEasing, TInlineAllocator<3>> Values;
	};
	
	FGroup& FindOrAddGroup(TArray<FGroup>& Groups, const FString& GroupName);
	TArray<FGroup> ConstructGroups(const TSet<EMovieSceneBuiltInEasing>& FilterExclude);

	FReply OnTypeButtonClicked(EMovieSceneBuiltInEasing type);

	FEasingFunctionGridWidget_OnClicked OnClickedDelegate;
	TAttribute<TSet<EMovieSceneBuiltInEasing>> FilterExcludeAttribute;
};