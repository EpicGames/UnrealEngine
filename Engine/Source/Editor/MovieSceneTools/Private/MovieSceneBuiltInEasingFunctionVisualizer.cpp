// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBuiltInEasingFunctionVisualizer.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailChildrenBuilder.h"

class FActiveTimerHandle;
class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FPointerEvent;
struct FSlateBrush;

void SBuiltInFunctionVisualizer::SetType(EMovieSceneBuiltInEasing InValue)
{
	InterpValue = FVector2D::ZeroVector;

	UMovieSceneBuiltInEasingFunction* DefaultObject = GetMutableDefault<UMovieSceneBuiltInEasingFunction>();
	EMovieSceneBuiltInEasing DefaultType = DefaultObject->Type;

	DefaultObject->Type = InValue;

	Samples.Reset();
	float Interp = 0.f;
	while (Interp <= 1.f)
	{
		Samples.Add(FVector2D(Interp, DefaultObject->Evaluate(Interp)));
		Interp += 0.01f;
	}

	DefaultObject->Type = DefaultType;
	EasingType = InValue;
}

void SBuiltInFunctionVisualizer::Construct(const FArguments& InArgs, EMovieSceneBuiltInEasing InValue)
{
	SetType(InValue);

	ChildSlot
		[
			SNew(SOverlay)
		];
}

void SBuiltInFunctionVisualizer::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!TimerHandle.IsValid())
	{
		MouseOverTime = FSlateApplication::Get().GetCurrentTime();
		TimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SBuiltInFunctionVisualizer::TickInterp));
	}
}

void SBuiltInFunctionVisualizer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (TimerHandle.IsValid())
	{
		InterpValue = FVector2D::ZeroVector;
		UnRegisterActiveTimer(TimerHandle.ToSharedRef());
		TimerHandle = nullptr;
	}
}

EActiveTimerReturnType SBuiltInFunctionVisualizer::TickInterp(const double InCurrentTime, const float InDeltaTime)
{
	static float InterpInPad = .25f, InterpOutPad = .5f;

	float TotalInterpTime = InterpInPad + AnimationDuration + InterpOutPad;
	InterpValue.X = FMath::Clamp((FMath::Fmod(float(InCurrentTime - MouseOverTime), TotalInterpTime) - InterpInPad) / AnimationDuration, 0.f, 1.f);

	UMovieSceneBuiltInEasingFunction* DefaultObject = GetMutableDefault<UMovieSceneBuiltInEasingFunction>();

	EMovieSceneBuiltInEasing DefaultType = DefaultObject->Type;
	DefaultObject->Type = EasingType;

	InterpValue.Y = DefaultObject->Evaluate(InterpValue.X);
	DefaultObject->Type = DefaultType;

	return EActiveTimerReturnType::Continue;
}

int32 SBuiltInFunctionVisualizer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	float VerticalPad = 0.2f;
	FVector2D InverseVerticalSize(AllottedGeometry.Size.X, -AllottedGeometry.Size.Y);

	const float VerticalBottom = AllottedGeometry.Size.Y - AllottedGeometry.Size.Y * VerticalPad * .5f;
	const float CurveHeight = AllottedGeometry.Size.Y * (1.f - VerticalPad);
	const float CurveWidth = AllottedGeometry.Size.X - 5.f;

	TArray<FVector2D> Points;
	for (FVector2D Sample : Samples)
	{
		FVector2D Offset(5.f, VerticalBottom);
		Points.Add(Offset + FVector2D(
			CurveWidth * Sample.X,
			-CurveHeight * Sample.Y
		));
	}

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		Points,
		ESlateDrawEffect::None);

	if (TimerHandle.IsValid())
	{
		FVector2D PointOffset(0.f, VerticalBottom - CurveHeight * InterpValue.Y - 4.f);

		static const FSlateBrush* InterpPointBrush = FAppStyle::GetBrush("Sequencer.InterpLine");
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.MakeChild(
				FVector2D(AllottedGeometry.Size.X, 7.f),
				FSlateLayoutTransform(PointOffset)
			).ToPaintGeometry(),
			InterpPointBrush,
			ESlateDrawEffect::None,
			FLinearColor::Green
		);
	}

	return LayerId + 1;
}