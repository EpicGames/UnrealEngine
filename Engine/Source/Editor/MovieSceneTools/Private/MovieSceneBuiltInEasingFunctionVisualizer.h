// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class IDetailLayoutBuilder;

enum class EMovieSceneBuiltInEasing : uint8;
class FReply;
class IPropertyHandle;

/** Widget showing the curve graph */
class SBuiltInFunctionVisualizer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBuiltInFunctionVisualizer) {}
	SLATE_END_ARGS();

	virtual ~SBuiltInFunctionVisualizer() = default;

	void Construct(const FArguments& InArgs, EMovieSceneBuiltInEasing InValue);
	
	void SetType(EMovieSceneBuiltInEasing InValue);
	void SetAnimationDuration(float TimeInSeconds) { AnimationDuration = TimeInSeconds; }

private:
	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override final;
	void OnMouseLeave(const FPointerEvent& MouseEvent) override final;

	EActiveTimerReturnType TickInterp(const double InCurrentTime, const float InDeltaTime);

	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	TSharedPtr<FActiveTimerHandle> TimerHandle;
	double MouseOverTime;
	EMovieSceneBuiltInEasing EasingType;

	FVector2D InterpValue;
	TArray<FVector2D> Samples;
	float AnimationDuration = 0.5f; /** Preview animation duration in seconds. */
};