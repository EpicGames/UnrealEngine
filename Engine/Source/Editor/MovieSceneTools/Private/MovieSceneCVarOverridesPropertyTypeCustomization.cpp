// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCVarOverridesPropertyTypeCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailWidgetRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Sections/MovieSceneCVarSection.h"
#include "Styling/AppStyle.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SMultiLineEditableText.h"

#define LOCTEXT_NAMESPACE "FCVarOverridesPropertyTypeCustomization"

namespace UE
{
namespace MovieScene
{

TSharedRef<IPropertyTypeCustomization> FCVarOverridesPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FCVarOverridesPropertyTypeCustomization>();
}

void FCVarOverridesPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(TOptional<float>())
	[
		SNew(SBox)
		.MinDesiredHeight(150.f)
		[
			SNew(SMultiLineEditableText)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.OnTextCommitted(this, &FCVarOverridesPropertyTypeCustomization::OnCVarsCommitted)
			.OnTextChanged(this, &FCVarOverridesPropertyTypeCustomization::OnCVarsChanged)
			.Text(this, &FCVarOverridesPropertyTypeCustomization::GetCVarText)
			.HintText(LOCTEXT("CVarSectionHint", "Add CVar values"))
		]
	];
}

void FCVarOverridesPropertyTypeCustomization::OnCVarsCommitted(const FText& NewText, ETextCommit::Type)
{
	OnCVarsChanged(NewText);
}

void FCVarOverridesPropertyTypeCustomization::OnCVarsChanged(const FText& NewText)
{
	FString NewValue = NewText.ToString();

	PropertyHandle->NotifyPreChange();

	TArray<void*> RawPtrs;
	PropertyHandle->AccessRawData(RawPtrs);

	for (void* Ptr : RawPtrs)
	{
		static_cast<FMovieSceneCVarOverrides*>(Ptr)->SetFromString(NewValue);
	}

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

FText FCVarOverridesPropertyTypeCustomization::GetCVarText() const
{
	TArray<const void*> RawPtrs;
	PropertyHandle->AccessRawData(RawPtrs);

	if (RawPtrs.Num() == 0)
	{
		return FText();
	}

	if (RawPtrs.Num() == 1)
	{
		return FText::FromString(static_cast<const FMovieSceneCVarOverrides*>(RawPtrs[0])->GetString());
	}

	return LOCTEXT("Multiple Values", "<<Multiple Values>>");
}

} // namespace MovieScene
} // namespace UE

#undef LOCTEXT_NAMESPACE