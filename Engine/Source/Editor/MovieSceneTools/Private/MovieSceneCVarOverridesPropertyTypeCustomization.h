// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

namespace UE
{
namespace MovieScene
{

struct FCVarOverridesPropertyTypeCustomization : public IPropertyTypeCustomization
{
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	void OnCVarsCommitted(const FText& NewText, ETextCommit::Type);
	void OnCVarsChanged(const FText& NewText);
	FText GetCVarText() const;

	TSharedPtr<IPropertyHandle> PropertyHandle;
};


} // namespace MovieScene
} // namespace UE