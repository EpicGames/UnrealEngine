// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDirectorBlueprintEndpointCustomization.h"
#include "Modules/ModuleManager.h"

#include "UObject/UnrealType.h"
#include "Algo/Find.h"
#include "ISequencerModule.h"
#include "MovieSceneSequence.h"
#include "ScopedTransaction.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SExpanderArrow.h"

#include "PropertyHandle.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include "K2Node_Variable.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "SGraphActionMenu.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintFunctionNodeSpawner.h"

#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"

#define LOCTEXT_NAMESPACE "MovieSceneDirectorBlueprintEndpointCustomization"

namespace UE::Sequencer
{

TFunction<bool(const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)> MakeRejectAnyIncompatibleReturnValuesFilter(FProperty* ReturnProperty)
{
	if (!ReturnProperty)
	{
		return [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction) { return false; };
	}

	FStructProperty* ReturnStructProperty = CastField<FStructProperty>(ReturnProperty);
	FObjectPropertyBase* ReturnObjectProperty = CastField<FObjectPropertyBase>(ReturnProperty);
	FBoolProperty* ReturnBoolProperty = CastField<FBoolProperty>(ReturnProperty);
	auto RejectAnyIncompatibleReturnValues = [=](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		const UFunction* Function = BlueprintAction.GetAssociatedFunction();
		const FProperty* FunctionReturnProperty = Function->GetReturnProperty();
		if (ReturnProperty->SameType(FunctionReturnProperty))
		{
			if (ReturnStructProperty && ReturnStructProperty->Struct == CastField<FStructProperty>(FunctionReturnProperty)->Struct)
			{
				return false;
			}
			else if (ReturnObjectProperty && ReturnObjectProperty->PropertyClass == CastField<FObjectPropertyBase>(FunctionReturnProperty)->PropertyClass)
			{
				return false;
			}
			else if (ReturnBoolProperty)
			{
				return false;
			}
		}
		return true;
	};
	return RejectAnyIncompatibleReturnValues;
};

} // namespace UE::Sequencer

void FMovieSceneDirectorBlueprintEndpointCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FMovieSceneDirectorBlueprintEndpointCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	PropertyRawData.Empty();
	PropertyHandle->AccessRawData(PropertyRawData);

	ChildBuilder.AddCustomRow(FText())
	.NameContent()
	[
		SNew(STextBlock)
		.Font(CustomizationUtils.GetRegularFont())
		.Text(LOCTEXT("EndpointValueText", "Endpoint"))
	]
	.ValueContent()
	.MinDesiredWidth(200.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.OnGetMenuContent(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetMenuContent)
			.CollapseMenuOnParentFocus(true)
			.ContentPadding(FMargin(4.f, 0.f))
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetEndpointIcon)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(CustomizationUtils.GetRegularFont())
					.Text(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetEndpointName)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::NavigateToDefinition), LOCTEXT("NavigateToDefinition_Tip", "Navigate to this endpoint's definition"))
		]
	];

	const bool bAnyBoundEndpoints = GetAllValidEndpoints().Num() != 0;
	if (bAnyBoundEndpoints)
	{
		FText CallInEditorText    = LOCTEXT("CallInEditor_Label",   "Call In Editor");
		FText CallInEditorTooltip = LOCTEXT("CallInEditor_Tooltip", "When checked, this endpoint will be triggered in the Editor outside of PIE.\n\nBEWARE: ANY CHANGES MADE AS A RESULT OF THIS ENDPOINT BEING CALLED MAY END UP BEING SAVED IN THE CURRENT LEVEL OR ASSET.");

		ChildBuilder.AddCustomRow(CallInEditorText)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(CallInEditorText)
			.ToolTipText(CallInEditorTooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.ToolTipText(CallInEditorTooltip)
			.IsChecked(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetCallInEditorCheckState)
			.OnCheckStateChanged(this, &FMovieSceneDirectorBlueprintEndpointCustomization::OnSetCallInEditorCheckState)
		];
	}

	UK2Node* CommonEndpoint = GetCommonEndpoint();
	if (!CommonEndpoint)
	{
		return;
	}

	TArray<FWellKnownParameterCandidates> WellKnownParameterCandidates;
	GetWellKnownParameterCandidates(CommonEndpoint, WellKnownParameterCandidates);
	for (int32 ParamIndex = 0; ParamIndex < WellKnownParameterCandidates.Num(); ++ParamIndex)
	{
		const FWellKnownParameterCandidates& Candidates = WellKnownParameterCandidates[ParamIndex];
		const FWellKnownParameterMetadata& ParamMetadata = Candidates.Metadata;

		if (!Candidates.bShowUnmatchedParameters && Candidates.CandidatePinNames.IsEmpty())
		{
			continue;
		}

		ChildBuilder.AddCustomRow(ParamMetadata.PickerLabel)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(ParamMetadata.PickerLabel)
			.ToolTipText(ParamMetadata.PickerTooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(ParamMetadata.PickerTooltip)
			.OnGetMenuContent(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinMenuContent, ParamIndex)
			.CollapseMenuOnParentFocus(true)
			.ContentPadding(FMargin(4.f, 0.f))
			.ButtonContent()
			[
				SNew(SBox)
				.HeightOverride(21.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinIcon, ParamIndex)
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(CustomizationUtils.GetRegularFont())
						.Text(this, &FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinText, ParamIndex)
					]
				]
			]
		];
	}

	UFunction* CommonFunction = nullptr;
	UBlueprint* Blueprint = CommonEndpoint->HasValidBlueprint() ? CommonEndpoint->GetBlueprint() : nullptr;
	if (Blueprint)
	{
		if (UK2Node_Event* Event = Cast<UK2Node_Event>(CommonEndpoint))
		{
			CommonFunction = Blueprint->SkeletonGeneratedClass ?
				Blueprint->SkeletonGeneratedClass->FindFunctionByName(Event->GetFunctionName()) :
				nullptr;
		}
		else if (UK2Node_FunctionEntry* EndpointEntry = Cast<UK2Node_FunctionEntry>(CommonEndpoint))
		{
			CommonFunction = EndpointEntry->FindSignatureFunction();
		}
		else
		{
			// @todo: Error not supported
		}

		Blueprint->OnCompiled().AddSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::OnBlueprintCompiled);
	}

	if (CommonFunction)
	{
		IDetailCategoryBuilder& DetailCategoryBuilder = CreateNewCategoryForPayloadVariables() ? 
			ChildBuilder.GetParentCategory()
			.GetParentLayout()
			.EditCategory("Payload", LOCTEXT("PayloadLabel", "Payload"), ECategoryPriority::Uncommon) :
			ChildBuilder.GetParentCategory();

		bool bPayloadUpToDate = true;

		TArray<UObject*> EditObjects;
		GetEditObjects(EditObjects);

		FPayloadVariableMap PayloadVariables;
		TArray<FName> WellKnownParameters;
		TArray<FName, TInlineAllocator<8>> AllValidNames;

		const FString* WorldContextParamName = CommonFunction->FindMetaData(FBlueprintMetadata::MD_WorldContext);

		for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
		{
			AllValidNames.Empty();
			PayloadVariables.Empty();
			WellKnownParameters.Empty();

			GetPayloadVariables(EditObjects[Index], PropertyRawData[Index], PayloadVariables);
			GetWellKnownParameterPinNames(EditObjects[Index], PropertyRawData[Index], WellKnownParameters);

			TSharedPtr<FStructOnScope> StructData = MakeShared<FStructOnScope>(CommonFunction);

			StructData->SetPackage(Blueprint->GetPackage());

			for (FProperty* Field : TFieldRange<FProperty>(CommonFunction))
			{
				// Ignore parameters we can't use as inputs.
				if (Field->HasAnyPropertyFlags(CPF_OutParm | CPF_ReturnParm | CPF_ReferenceParm))
				{
					continue;
				}

				// Ignore "well-known parameters", i.e. parameters for which the system
				// will pass specific values not meant to be defined by the user.
				if (WellKnownParameters.Contains(Field->GetFName()))
				{
					continue;
				}

				// Ignore the world context parameter.
				if (WorldContextParamName && Field->GetName() == *WorldContextParamName)
				{
					continue;
				}

				const FMovieSceneDirectorBlueprintVariableValue* PayloadVariable = PayloadVariables.Find(Field->GetFName());
				if (PayloadVariable)
				{
					AllValidNames.Add(Field->GetFName());

					// We have an override for this variable
					bool bImportSuccess = false;
					if (PayloadVariable->ObjectValue.IsValid())
					{
						bImportSuccess = FBlueprintEditorUtils::PropertyValueFromString(Field, PayloadVariable->ObjectValue.ToString(), StructData->GetStructMemory());
					}
					if (!bImportSuccess)
					{
						bImportSuccess = FBlueprintEditorUtils::PropertyValueFromString(Field, *PayloadVariable->Value, StructData->GetStructMemory());
					}
					if (!bImportSuccess)
					{
						// @todo: error
					}
				}
				else if (UEdGraphPin* Pin = CommonEndpoint->FindPin(Field->GetFName(), EGPD_Output))
				{
					bool bImportSuccess = false;
					if (Pin->DefaultObject)
					{
						bImportSuccess = FBlueprintEditorUtils::PropertyValueFromString(Field, Pin->DefaultObject->GetPathName(), StructData->GetStructMemory());
					}
					if (!bImportSuccess && !Pin->DefaultValue.IsEmpty())
					{
						bImportSuccess = FBlueprintEditorUtils::PropertyValueFromString(Field, Pin->DefaultValue, StructData->GetStructMemory());
					}
					if (!bImportSuccess)
					{
						// @todo: error
					}
				}

				IDetailPropertyRow* ExternalRow = CreateNewCategoryForPayloadVariables() ? 
				(DetailCategoryBuilder.AddExternalStructureProperty(StructData.ToSharedRef(), Field->GetFName(), EPropertyLocation::Default, FAddPropertyParams().ForceShowProperty()))
				: (ChildBuilder.AddExternalStructureProperty(StructData.ToSharedRef(), Field->GetFName(), FAddPropertyParams().ForceShowProperty()));

				TSharedPtr<IPropertyHandle> LocalVariableProperty = ExternalRow->GetPropertyHandle();
				FSimpleDelegate Delegate = FSimpleDelegate::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::OnPayloadVariableChanged, StructData.ToSharedRef(), LocalVariableProperty);

				LocalVariableProperty->SetOnPropertyValueChanged(Delegate);
				LocalVariableProperty->SetOnChildPropertyValueChanged(Delegate);
			}

			bPayloadUpToDate &= (AllValidNames.Num() == PayloadVariables.Num());
		}

		if (!bPayloadUpToDate)
		{
			(CreateNewCategoryForPayloadVariables() ? DetailCategoryBuilder.AddCustomRow(FText()) : ChildBuilder.AddCustomRow(FText()))
			.WholeRowContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FMargin(5.f, 0.f, 5.f, 0.f))
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "Log.Warning")
					.Font(FAppStyle::GetFontStyle("FontAwesome.10"))
					.Text(FEditorFontGlyphs::Exclamation_Triangle)
				]

				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 5.f, 0.f))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "Log.Warning")
					.Text(LOCTEXT("PayloadOutOfDateError", "Payload variables may be out-of-date. Please compile the blueprint."))
				]
			];
		}
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::OnPayloadVariableChanged(TSharedRef<FStructOnScope> InStructData, TSharedPtr<IPropertyHandle> LocalVariableProperty)
{
	// This function should only ever be bound if all the entry points call the same function
	FProperty* Property = LocalVariableProperty->GetProperty();
	if (!Property)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetPayloadValue", "Set payload value"));

	bool bChangedAnything = false;

	FString NewValueString;

	const bool bSuccessfulTextExport = FBlueprintEditorUtils::PropertyValueToString(Property, InStructData->GetStructMemory(), NewValueString, nullptr);
	if (!bSuccessfulTextExport)
	{
		// @todo: error
		return;
	}

	TObjectPtr<UObject> NewValueObject;
	if (FPackageName::IsValidPath(NewValueString))
	{
		NewValueObject = FindObject<UObject>(nullptr, *NewValueString);
	}

#if WITH_EDITORONLY_DATA
	// Fixup redirectors
	while (Cast<UObjectRedirector>(NewValueObject) != nullptr)
	{
		NewValueObject = Cast<UObjectRedirector>(NewValueObject)->DestinationObject;
		if (NewValueObject)
		{
			NewValueString = NewValueObject->GetPathName();
		}
	}
#endif

	FMovieSceneDirectorBlueprintVariableValue NewPayloadVariable{ NewValueObject, NewValueString };

	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	if (!ensure(PropertyRawData.Num() == EditObjects.Num()))
	{
		return;
	}

	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		bChangedAnything |= SetPayloadVariable(EditObjects[Index], PropertyRawData[Index], Property->GetFName(), NewPayloadVariable);
	}

	if (bChangedAnything)
	{
		UK2Node* CommonEndpoint = GetCommonEndpoint();

		UBlueprint* BP = (CommonEndpoint && CommonEndpoint->HasValidBlueprint()) ? CommonEndpoint->GetBlueprint() : nullptr;
		if (BP)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
		}
	}
	else
	{
		Transaction.Cancel();
	}
}

ECheckBoxState FMovieSceneDirectorBlueprintEndpointCustomization::GetCallInEditorCheckState() const
{
	ECheckBoxState CheckState = ECheckBoxState::Undetermined;
	for (UK2Node* Endpoint : GetAllValidEndpoints())
	{
		bool bCallInEditor = false;
		if (UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Endpoint))
		{
			bCallInEditor = CustomEvent->bCallInEditor;
		}
		else if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Endpoint))
		{
			bCallInEditor = FunctionEntry->MetaData.bCallInEditor;
		}

		if (CheckState == ECheckBoxState::Undetermined)
		{
			CheckState = bCallInEditor ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		else if (bCallInEditor != (CheckState == ECheckBoxState::Checked) )
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return CheckState;
}

void FMovieSceneDirectorBlueprintEndpointCustomization::OnSetCallInEditorCheckState(ECheckBoxState NewCheckedState)
{
	FScopedTransaction Transaction(LOCTEXT("SetCallInEditor", "Set Call in Editor"));

	const bool bCallInEditor = (NewCheckedState == ECheckBoxState::Checked);

	TSet<UBlueprint*> Blueprints;

	for (UK2Node* Endpoint : GetAllValidEndpoints())
	{
		if (UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Endpoint))
		{
			CustomEvent->Modify();
			CustomEvent->bCallInEditor = bCallInEditor;

			if (CustomEvent->HasValidBlueprint())
			{
				Blueprints.Add(CustomEvent->GetBlueprint());
			}
		}
		else if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Endpoint))
		{
			FunctionEntry->Modify();
			FunctionEntry->MetaData.bCallInEditor = bCallInEditor;

			if (FunctionEntry->HasValidBlueprint())
			{
				Blueprints.Add(FunctionEntry->GetBlueprint());
			}
		}
	}

	for (UBlueprint* Blueprint : Blueprints)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::OnBlueprintCompiled(UBlueprint*)
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}

UMovieSceneSequence* FMovieSceneDirectorBlueprintEndpointCustomization::GetCommonSequence() const
{
	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	UMovieSceneSequence* CommonSequence = nullptr;

	for (UObject* Obj : EditObjects)
	{
		UMovieSceneSequence* ThisSequence = Obj ? Obj->GetTypedOuter<UMovieSceneSequence>() : nullptr;
		if (CommonSequence && CommonSequence != ThisSequence)
		{
			return nullptr;
		}

		CommonSequence = ThisSequence;
	}
	return CommonSequence;
}

void FMovieSceneDirectorBlueprintEndpointCustomization::IterateEndpoints(TFunctionRef<bool(UK2Node*)> Callback) const
{
	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	if (!ensure(PropertyRawData.Num() == EditObjects.Num()))
	{
		return;
	}

	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		UK2Node* Endpoint = FindEndpoint(EditObjects[Index], PropertyRawData[Index]);
		if (Endpoint)
		{
			if (!Callback(Endpoint))
			{
				return;
			}
		}
	}
}

TArray<UK2Node*> FMovieSceneDirectorBlueprintEndpointCustomization::GetAllValidEndpoints() const
{
	TArray<UK2Node*> Endpoints;

	IterateEndpoints(
		[&Endpoints](UK2Node* Endpoint)
		{
			if (Endpoint)
			{
				Endpoints.Add(Endpoint);
			}
			return true;
		}
	);

	return Endpoints;
}

UK2Node* FMovieSceneDirectorBlueprintEndpointCustomization::GetCommonEndpoint() const
{
	UK2Node* CommonEndpoint = nullptr;

	IterateEndpoints(
		[&CommonEndpoint](UK2Node* Endpoint)
		{
			if (CommonEndpoint && Endpoint != CommonEndpoint)
			{
				CommonEndpoint = nullptr;
				return false;
			}

			CommonEndpoint = Endpoint;
			return true;
		}
	);

	return CommonEndpoint;
}

void FMovieSceneDirectorBlueprintEndpointCustomization::GetEditObjects(TArray<UObject*>& OutObjects) const
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->GetOuterObjects(OutObjects);
	}
}

TSharedRef<SWidget> FMovieSceneDirectorBlueprintEndpointCustomization::GetMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

	UMovieSceneSequence* Sequence = GetCommonSequence();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CreateEndpoint_Text",    "Create New Endpoint"),
		LOCTEXT("CreateEndpoint_Tooltip", "Creates a new endpoint in this sequence's blueprint."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateEventBinding"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::CreateEndpoint)
		)
	);

	const bool bAnyBoundEndpoints = GetAllValidEndpoints().Num() != 0;
	if (!bAnyBoundEndpoints && Sequence)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("CreateQuickBinding_Text",    "Quick Bind"),
			LOCTEXT("CreateQuickBinding_Tooltip", "Shows a list of functions on this object binding that can be bound directly to this endpoint."),
			FNewMenuDelegate::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::PopulateQuickBindSubMenu, Sequence, FOnQuickBindActionSelected::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::HandleQuickBindActionSelected)),
			false /* bInOpenSubMenuOnClick */,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateQuickBinding"),
			false /* bInShouldWindowAfterMenuSelection */
		);
	}
	else
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("Rebind_Text",         "Rebind To"),
			LOCTEXT("Rebind_Text_Tooltip", "Rebinds this endpoint to a different function call or event node."),
			FNewMenuDelegate::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::PopulateRebindSubMenu, Sequence),
			false /* bInOpenSubMenuOnClick */,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.CreateQuickBinding"),
			false /* bInShouldCloseWindowAfterMenuSelection */
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearEndpoint_Text",    "Clear"),
			LOCTEXT("ClearEndpoint_Tooltip", "Unbinds this endpoint from its current binding."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.ClearEventBinding"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::ClearEndpoint)
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinMenuContent(int32 ParameterIndex)
{
	UK2Node* CommonEndpoint = GetCommonEndpoint();
	if (!CommonEndpoint)
	{
		return SNew(STextBlock).Text(LOCTEXT("WellKnownParameterPinError_MultipleEndpoints", "Cannot choose a parameter pin with multiple endpoints selected"));
	}

	TArray<FWellKnownParameterCandidates> WellKnownParameters;
	GetWellKnownParameterCandidates(CommonEndpoint, WellKnownParameters);
	if (!WellKnownParameters.IsValidIndex(ParameterIndex) ||
			WellKnownParameters[ParameterIndex].CandidatePinNames.IsEmpty())
	{
		return SNew(STextBlock).Text(LOCTEXT("WellKnownParameterPinError_NoPins", "No compatible pins were found."));
	}

	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);
	for (FName CandidatePinName : WellKnownParameters[ParameterIndex].CandidatePinNames)
	{
		FText PinText = FText::FromName(CandidatePinName);
		MenuBuilder.AddMenuEntry(
			PinText,
			FText::Format(LOCTEXT("SetWellKnownParameterPin_Tooltip", "When calling the endpoint, pass this parameter through pin {0}."), FText::FromName(CandidatePinName)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::SetWellKnownParameterPinName, ParameterIndex, CandidatePinName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::CompareWellKnownParameterPinName, ParameterIndex, CandidatePinName)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearWellKnownParameterPin_Label", "Clear"),
		LOCTEXT("ClearWellKnownParameterPin_Tooltip", "Clears the parameter binding, meaning that this parameter won't be passed to the endpoint anymore."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FMovieSceneDirectorBlueprintEndpointCustomization::SetWellKnownParameterPinName, ParameterIndex, FName())
		)
	);

	return MenuBuilder.MakeWidget();
}

bool FMovieSceneDirectorBlueprintEndpointCustomization::CompareWellKnownParameterPinName(int32 ParameterIndex, FName InPinName) const
{
	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	check(EditObjects.Num() == PropertyRawData.Num());

	TArray<FName> WellKnownParameters;
	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		WellKnownParameters.Empty();
		GetWellKnownParameterPinNames(EditObjects[Index], PropertyRawData[Index], WellKnownParameters);
		if (WellKnownParameters.IsValidIndex(ParameterIndex) &&
				WellKnownParameters[ParameterIndex] == InPinName)
		{
			return true;
		}
	}

	return false;
}

void FMovieSceneDirectorBlueprintEndpointCustomization::SetWellKnownParameterPinName(int32 ParameterIndex, FName InNewBoundPinName)
{
	FScopedTransaction Transaction(LOCTEXT("SetWellKnownParameterPinTransaction", "Set Bound Object Pin"));

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->NotifyPreChange();
	}

	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	check(EditObjects.Num() == PropertyRawData.Num());

	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		EditObjects[Index]->Modify();

		const bool bWasSet = SetWellKnownParameterPinName(EditObjects[Index], PropertyRawData[Index], ParameterIndex, InNewBoundPinName);
		ensureMsgf(bWasSet, TEXT("Trying to set bound pin %s for parameter %d but sub-class did not handle it."), 
				*InNewBoundPinName.ToString(), ParameterIndex);
	}

	if (PropertyHandle.IsValid())
	{
		// Ensure that anything listening for property changed notifications are notified of the new binding
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	FSequenceDataMap AllSequenceData;
	GatherSequenceData(AllSequenceData);
	for (const TPair<UMovieSceneSequence*, FSequenceData>& SequenceData : AllSequenceData)
	{
		FKismetEditorUtilities::CompileBlueprint(SequenceData.Value.Blueprint);
	}

	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}

const FSlateBrush* FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinIcon(int32 ParameterIndex) const
{
	const bool bIsUnbound = CompareWellKnownParameterPinName(ParameterIndex, NAME_None);
	if (bIsUnbound)
	{
		return nullptr;
	}

	return FAppStyle::GetBrush("Graph.Pin.Disconnected_VarA");
}

FText FMovieSceneDirectorBlueprintEndpointCustomization::GetWellKnownParameterPinText(int32 ParameterIndex) const
{
	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	check(EditObjects.Num() == PropertyRawData.Num());

	FName CommonPinName;
	TArray<FName> WellKnownParameters;

	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		WellKnownParameters.Empty();
		GetWellKnownParameterPinNames(EditObjects[Index], PropertyRawData[Index], WellKnownParameters);
		if (WellKnownParameters.IsValidIndex(ParameterIndex))
		{
			FName PinName = WellKnownParameters[ParameterIndex];
			if (CommonPinName == NAME_None)
			{
				CommonPinName = PinName;
			}
			else if (CommonPinName != PinName)
			{
				return LOCTEXT("MultiplePinValues", "Multiple Values");
			}
		}
	}

	return FText::FromName(CommonPinName);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSequence* Sequence, FOnQuickBindActionSelected InOnQuickBindActionSelected)
{
	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	if (!SequenceEditor)
	{
		return;
	}

	UBlueprint* Blueprint = SequenceEditor->GetOrCreateDirectorBlueprint(Sequence);
	if (!Blueprint)
	{
		return;
	}

	FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition = GenerateEndpointDefinition(Sequence);

	TSharedRef<SGraphActionMenu> ActionMenu = SNew(SGraphActionMenu)
		.OnCreateCustomRowExpander_Static([](const FCustomExpanderData& Data) -> TSharedRef<SExpanderArrow> { return SNew(SExpanderArrow, Data.TableRow); })
		.OnCollectAllActions(this, &FMovieSceneDirectorBlueprintEndpointCustomization::CollectQuickBindActions, Blueprint, EndpointDefinition)
		.OnActionSelected(FOnActionSelected::CreateLambda([this, Blueprint, EndpointDefinition, InOnQuickBindActionSelected](const TArray< TSharedPtr<FEdGraphSchemaAction> >& Actions, ESelectInfo::Type Type)
			{
				// We call the passed in delegate first as we may need to set something up before running the action
				InOnQuickBindActionSelected.ExecuteIfBound(Actions, Type, Blueprint, EndpointDefinition);
			}));

	ActionMenu->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(
		[FilterTextBox = ActionMenu->GetFilterTextBox()](double, float)
		{
			FSlateApplication::Get().SetKeyboardFocus(FilterTextBox);
			return EActiveTimerReturnType::Stop;
		}
	));

	MenuBuilder.AddWidget(
		SNew(SBox)
		.WidthOverride(300.f)
		.MaxDesiredHeight(500.f)
		[
			ActionMenu
		],
		FText()
	);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::CollectQuickBindActions(FGraphActionListBuilderBase& OutAllActions, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition)
{
	// Build up the context object
	auto RejectAnyNonFunctions = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		const UFunction* Function = BlueprintAction.GetAssociatedFunction();
		return Function == nullptr;
	};
	auto RejectAnyNonPureFunctions = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		const UFunction* Function = BlueprintAction.GetAssociatedFunction();
		return Function == nullptr || Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
	};
	auto RejectAnyUnboundActions = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		return BlueprintAction.GetBindings().Num() <= 0;
	};

	FProperty* ReturnProperty = EndpointDefinition.EndpointSignature ? EndpointDefinition.EndpointSignature->GetReturnProperty() : nullptr;
	auto RejectAnyIncompatibleReturnValues = UE::Sequencer::MakeRejectAnyIncompatibleReturnValuesFilter(ReturnProperty);

	FBlueprintActionMenuBuilder ContextMenuBuilder;

	if (EndpointDefinition.PossibleCallTargetClass)
	{
		// Add actions that are relevant to the bound object from the pin class
		FBlueprintActionFilter CallOnMemberFilter(FBlueprintActionFilter::BPFILTER_RejectGlobalFields | FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);
		CallOnMemberFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
		CallOnMemberFilter.Context.Blueprints.Add(Blueprint);

		for (FObjectProperty* ObjectProperty : TFieldRange<FObjectProperty>(EndpointDefinition.PossibleCallTargetClass))
		{
			if (ObjectProperty->HasAnyPropertyFlags(CPF_BlueprintVisible) && (ObjectProperty->HasMetaData(FBlueprintMetadata::MD_ExposeFunctionCategories) || FBlueprintEditorUtils::IsSCSComponentProperty(ObjectProperty)))
			{
				CallOnMemberFilter.Context.SelectedObjects.Add(ObjectProperty);
				FBlueprintActionFilter::AddUnique(CallOnMemberFilter.TargetClasses, ObjectProperty->PropertyClass);
			}
		}

		FBlueprintActionFilter::AddUnique(CallOnMemberFilter.TargetClasses, EndpointDefinition.PossibleCallTargetClass);

		// This removes duplicate entries (ie. Set Static Mesh and Set Static Mesh (StaticMeshComponent)), 
		// but also prevents displaying functions on BP components. Comment out for now.
		//CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(RejectAnyUnboundActions));
		
		CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(RejectAnyNonPureFunctions));

		if (ReturnProperty)
		{
			CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateLambda(RejectAnyIncompatibleReturnValues));
		}

		ContextMenuBuilder.AddMenuSection(CallOnMemberFilter, FText::FromName(EndpointDefinition.PossibleCallTargetClass->GetFName()), 0);
	}

	{
		// Add all actions that are relevant to the sequence director BP itself
		FBlueprintActionFilter MenuFilter(FBlueprintActionFilter::BPFILTER_RejectGlobalFields | FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);

		MenuFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());

		MenuFilter.Context.Blueprints.Add(Blueprint);
		MenuFilter.Context.Graphs.Append(Blueprint->UbergraphPages);
		MenuFilter.Context.Graphs.Append(Blueprint->FunctionGraphs);

		if (Blueprint->SkeletonGeneratedClass)
		{
			FBlueprintActionFilter::AddUnique(MenuFilter.TargetClasses, Blueprint->SkeletonGeneratedClass);
		}
		MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(RejectAnyNonFunctions));
		
		if (ReturnProperty)
		{
			MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateLambda(RejectAnyIncompatibleReturnValues));
		}

		ContextMenuBuilder.AddMenuSection(MenuFilter, LOCTEXT("SequenceDirectorMenu", "This Sequence"), 0);
	}

	{
		OnCollectQuickBindActions(Blueprint, ContextMenuBuilder);
	}

	ContextMenuBuilder.RebuildActionList();

	OutAllActions.Append(ContextMenuBuilder);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::HandleQuickBindActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition)
{
	if (InSelectionType != ESelectInfo::OnMouseClick && InSelectionType != ESelectInfo::OnKeyPress)
	{
		return;
	}

	for (TSharedPtr<FEdGraphSchemaAction> Action : SelectedAction)
	{
		if (Action->GetTypeId() == FBlueprintActionMenuItem::StaticGetTypeId())
		{
			const UBlueprintFunctionNodeSpawner* FunctionNodeSpawner = Cast<UBlueprintFunctionNodeSpawner>(static_cast<FBlueprintActionMenuItem*>(Action.Get())->GetRawAction());
			const UFunction* FunctionToCall = FunctionNodeSpawner ? FunctionNodeSpawner->GetFunction() : nullptr;

			if (FunctionToCall)
			{
				UClass* OuterClass = CastChecked<UClass>(FunctionToCall->GetOuter());
				if (OuterClass->ClassGeneratedBy == Blueprint && Blueprint->SkeletonGeneratedClass)
				{
					// Attempt to locate a custom event or a function graph of this name on the blueprint
					for (UEdGraph* Graph : Blueprint->UbergraphPages)
					{
						for (UEdGraphNode* Node : Graph->Nodes)
						{
							UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Node);
							if (CustomEvent && Blueprint->SkeletonGeneratedClass->FindFunctionByName(CustomEvent->GetFunctionName()) == FunctionToCall)
							{
								// Use this custom event
								SetEndpoint(EndpointDefinition, CustomEvent, CustomEvent, EAutoCreatePayload::Variables);
								return;
							}
						}
					}

					for (UEdGraph* Graph : Blueprint->FunctionGraphs)
					{
						if (Blueprint->SkeletonGeneratedClass->FindFunctionByName(Graph->GetFName()) != FunctionToCall)
						{
							continue;
						}

						// Use this function graph for the event endpoint
						for (UEdGraphNode* Node : Graph->Nodes)
						{
							UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node);
							if (FunctionEntry)
							{
								SetEndpoint(EndpointDefinition, FunctionEntry, FunctionEntry, EAutoCreatePayload::Variables);
								return;
							}
						}
					}
				}
			}
		}

		if (Action)
		{
			Blueprint->Modify();
			UK2Node* NewEndpoint = FMovieSceneDirectorBlueprintUtils::CreateEndpoint(Blueprint, EndpointDefinition);

			UEdGraphPin* EndpointThenPin = NewEndpoint->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
			UEdGraphPin* CallTargetPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEndpoint, EndpointDefinition.PossibleCallTargetClass);

			FVector2f NodePosition(NewEndpoint->NodePosX + 400.f, NewEndpoint->NodePosY + 100.f);
			UEdGraphNode* NewNode = Action->PerformAction(NewEndpoint->GetGraph(), CallTargetPin ? CallTargetPin : EndpointThenPin, NodePosition);

			if (NewNode)
			{
				// If the new node has an exec pin, connect it. It may not have one if it's a BlueprintPure function.
				UEdGraphPin* NewNodeExecPin = NewNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
				if (EndpointThenPin && NewNodeExecPin)
				{
					EndpointThenPin->MakeLinkTo(NewNodeExecPin);
				}

				TArray<UK2Node_FunctionResult*> ResultNodes;
				NewNode->GetGraph()->GetNodesOfClass(ResultNodes);
				if (ResultNodes.Num() > 0)
				{
					// If there is a result node, move it past the endpoint call and connect it.
					ResultNodes[0]->NodePosX = NodePosition.X + 400.f;

					UEdGraphPin* NewNodeThenPin = NewNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
					UEdGraphPin* ResultExecPin = ResultNodes[0]->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
					if (NewNodeThenPin && ResultExecPin)
					{
						NewNodeThenPin->MakeLinkTo(ResultExecPin);
					}

					// If the new node has a return value, and if the endpoint has one too, try to connect them together.
					UEdGraphPin* OutputPin = ResultNodes[0]->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Input);
					UEdGraphPin* NewNodeReturnValuePin = NewNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);
					if (OutputPin && NewNodeReturnValuePin)
					{
						// Connect the nodes.
						NewNodeReturnValuePin->MakeLinkTo(OutputPin);
					}
				}
			}

			SetEndpoint(EndpointDefinition, NewEndpoint, Cast<UK2Node>(NewNode), EAutoCreatePayload::Pins | EAutoCreatePayload::Variables);
		}
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::SetPropertyHandle(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;
	if (PropertyHandle.IsValid())
	{
		PropertyRawData.Empty();
		PropertyHandle->AccessRawData(PropertyRawData);
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::PopulateRebindSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSequence* Sequence)
{
	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	if (!SequenceEditor)
	{
		return;
	}

	UBlueprint* Blueprint = SequenceEditor->GetOrCreateDirectorBlueprint(Sequence);
	if (!Blueprint)
	{
		return;
	}

	FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition = GenerateEndpointDefinition(Sequence);

	TSharedRef<SGraphActionMenu> ActionMenu = SNew(SGraphActionMenu)
		.OnCreateCustomRowExpander_Static([](const FCustomExpanderData& Data) -> TSharedRef<SExpanderArrow> { return SNew(SExpanderArrow, Data.TableRow); })
		.OnCollectAllActions(this, &FMovieSceneDirectorBlueprintEndpointCustomization::CollectAllRebindActions, Blueprint, EndpointDefinition)
		.OnActionSelected(this, &FMovieSceneDirectorBlueprintEndpointCustomization::HandleRebindActionSelected, Blueprint, EndpointDefinition);

	ActionMenu->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(
		[FilterTextBox = ActionMenu->GetFilterTextBox()](double, float)
		{
			FSlateApplication::Get().SetKeyboardFocus(FilterTextBox);
			return EActiveTimerReturnType::Stop;
		}
	));

	MenuBuilder.AddWidget(
		SNew(SBox)
		.WidthOverride(300.f)
		.MaxDesiredHeight(500.f)
		[
			ActionMenu
		],
		FText()
	);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::CollectAllRebindActions(FGraphActionListBuilderBase& OutAllActions, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition)
{
	// Build up the context object
	auto RejectAnyForeignFunctions = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction, UBlueprint* Blueprint)
	{
		const UBlueprintFunctionNodeSpawner* FunctionNodeSpawner = Cast<UBlueprintFunctionNodeSpawner>(BlueprintAction.NodeSpawner);
		const UFunction* FunctionToCall = FunctionNodeSpawner ? FunctionNodeSpawner->GetFunction() : nullptr;

		if (!FunctionToCall || FunctionToCall->HasAnyFunctionFlags(FUNC_BlueprintPure))
		{
			return true;
		}

		UClass* OuterClass = CastChecked<UClass>(FunctionToCall->GetOuter());
		return OuterClass->ClassGeneratedBy != Blueprint;
	};

	FProperty* ReturnProperty = EndpointDefinition.EndpointSignature ? EndpointDefinition.EndpointSignature->GetReturnProperty() : nullptr;
	auto RejectAnyIncompatibleReturnValues = UE::Sequencer::MakeRejectAnyIncompatibleReturnValuesFilter(ReturnProperty);

	FBlueprintActionMenuBuilder ContextMenuBuilder;

	{
		FBlueprintActionFilter MenuFilter(FBlueprintActionFilter::BPFILTER_RejectGlobalFields | FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);
		MenuFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());

		MenuFilter.Context.Blueprints.Add(Blueprint);

		MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(RejectAnyForeignFunctions, Blueprint));

		if (ReturnProperty)
		{
			MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateLambda(RejectAnyIncompatibleReturnValues));
		}

		if (Blueprint->SkeletonGeneratedClass)
		{
			FBlueprintActionFilter::AddUnique(MenuFilter.TargetClasses, Blueprint->SkeletonGeneratedClass);
		}

		ContextMenuBuilder.AddMenuSection(MenuFilter, LOCTEXT("SequenceDirectorMenu", "This Sequence"), 0);
	}

	{
		OnCollectAllRebindActions(Blueprint, ContextMenuBuilder);
	}

	ContextMenuBuilder.RebuildActionList();

	OutAllActions.Append(ContextMenuBuilder);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::HandleRebindActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition)
{
	if (InSelectionType != ESelectInfo::OnMouseClick && InSelectionType != ESelectInfo::OnKeyPress)
	{
		return;
	}

	if (!Blueprint->SkeletonGeneratedClass)
	{
		return;
	}

	for (TSharedPtr<FEdGraphSchemaAction> Action : SelectedAction)
	{
		if (Action->GetTypeId() != FBlueprintActionMenuItem::StaticGetTypeId())
		{
			continue;
		}

		const UBlueprintFunctionNodeSpawner* FunctionNodeSpawner = Cast<UBlueprintFunctionNodeSpawner>(static_cast<FBlueprintActionMenuItem*>(Action.Get())->GetRawAction());
		const UFunction* FunctionToCall = FunctionNodeSpawner ? FunctionNodeSpawner->GetFunction() : nullptr;
		if (!FunctionToCall)
		{
			continue;
		}

		// Give the opportunity for our sub-class implementation to do custom rebinding.
		FSequenceDataMap AllSequenceData;
		GatherSequenceData(AllSequenceData);

		for (const TPair<UMovieSceneSequence*, FSequenceData>& Pair : AllSequenceData)
		{
			const bool bDidRebind = OnRebindEndpoint(
					Pair.Key, Pair.Value.Blueprint, Pair.Value.EditObjects, Pair.Value.RawData,
					EndpointDefinition, Action);
			if (bDidRebind)
			{
				return;
			}
		}

		// Default implementation only rebinds to an existing endpoint in our current director blueprint.
		UClass* OuterClass = CastChecked<UClass>(FunctionToCall->GetOuter());
		if (OuterClass->ClassGeneratedBy != Blueprint)
		{
			continue;
		}

		// Attempt to locate a custom event or a function graph of this name on the blueprint
		for (UEdGraph* Graph : Blueprint->UbergraphPages)
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Node);
				if (CustomEvent && Blueprint->SkeletonGeneratedClass->FindFunctionByName(CustomEvent->GetFunctionName()) == FunctionToCall)
				{
					// Use this custom event
					SetEndpoint(EndpointDefinition, CustomEvent, CustomEvent, EAutoCreatePayload::Variables);
					return;
				}
			}
		}

		for (UEdGraph* Graph : Blueprint->FunctionGraphs)
		{
			if (Blueprint->SkeletonGeneratedClass->FindFunctionByName(Graph->GetFName()) != FunctionToCall)
			{
				continue;
			}

			// Use this function graph for the event endpoint
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node);
				if (FunctionEntry)
				{
					SetEndpoint(EndpointDefinition, FunctionEntry, FunctionEntry, EAutoCreatePayload::Variables);
					return;
				}
			}
		}
	}

	ensureMsgf(false, TEXT("Unknown blueprint action type encountered for rebinding"));
}

const FSlateBrush* FMovieSceneDirectorBlueprintEndpointCustomization::GetEndpointIcon() const
{
	UK2Node* CommonEndpoint = GetCommonEndpoint();

	if (CommonEndpoint)
	{
		FLinearColor Color;
		FSlateIcon EndpointIcon = CommonEndpoint->GetIconAndTint(Color);
		return EndpointIcon.GetIcon();
	}
	else
	{
		if (PropertyRawData.Num() > 1)
		{
			return FAppStyle::GetBrush("Sequencer.MultipleEvents");
		}
	}

	return FAppStyle::GetBrush("Sequencer.UnboundEvent");
}

FText FMovieSceneDirectorBlueprintEndpointCustomization::GetEndpointName() const
{
	UEdGraphNode* CommonEndpoint = GetCommonEndpoint();

	if (CommonEndpoint)
	{
		return CommonEndpoint->GetNodeTitle(ENodeTitleType::MenuTitle);
	}
	else
	{
		if (PropertyRawData.Num() != 1)
		{
			return LOCTEXT("MultipleValuesText", "Multiple Values");
		}
	}

	return LOCTEXT("UnboundText", "Unbound");
}

void FMovieSceneDirectorBlueprintEndpointCustomization::GatherSequenceData(FSequenceDataMap& AllSequenceData)
{
	TArray<UObject*> EditObjects;
	GetEditObjects(EditObjects);

	check(EditObjects.Num() == PropertyRawData.Num());

	for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
	{
		UMovieSceneSequence*       Sequence           = EditObjects[Index]->GetTypedOuter<UMovieSceneSequence>();
		FMovieSceneSequenceEditor* SequenceEditor     = FMovieSceneSequenceEditor::Find(Sequence);
		UBlueprint*                SequenceDirectorBP = SequenceEditor ? SequenceEditor->GetOrCreateDirectorBlueprint(Sequence) : nullptr;

		FSequenceData& SequenceData = AllSequenceData.FindOrAdd(Sequence);
		ensure(SequenceData.Blueprint == nullptr || SequenceData.Blueprint == SequenceDirectorBP);
		SequenceData.Blueprint = SequenceDirectorBP;
		SequenceData.EditObjects.Add(EditObjects[Index]);
		SequenceData.RawData.Add(PropertyRawData[Index]);
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::SetEndpoint(const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint, UK2Node* PayloadTemplate, EAutoCreatePayload AutoCreatePayload)
{
	FScopedTransaction Transaction(LOCTEXT("SetDirectorBlueprintEndpoint", "Set Director Blueprint Endpoint"));

	// Modify and assign the blueprint for outer sections
	FSequenceDataMap AllSequenceData;
	GatherSequenceData(AllSequenceData);

	// If we're assigning a new valid endpoint, it must reside within the same blueprint as everything we're assigning it to.
	// Anything else must be implemented as a call function node connected to a custom event node or function graph.
	UMovieSceneSequence* Sequence = nullptr;
	UBlueprint* Blueprint = (NewEndpoint && NewEndpoint->HasValidBlueprint()) ? NewEndpoint->GetBlueprint() : nullptr;
	for (const TPair<UMovieSceneSequence*, FSequenceData>& Pair : AllSequenceData)
	{
		if (NewEndpoint == nullptr)
		{
			Blueprint = Pair.Value.Blueprint;
		}
		if (Sequence == nullptr)
		{
			Sequence = Pair.Key;
		}

		if (!ensureAlwaysMsgf(
				Pair.Value.Blueprint == Blueprint && Pair.Key == Sequence, 
				TEXT("Attempting to assign an endpoint to objects with different Sequence Director Blueprints.")))
		{
			Transaction.Cancel();
			return;
		}

		Blueprint->Modify();
	}

	if (!ensure(Sequence))
	{
		Transaction.Cancel();
		return;
	}

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->NotifyPreChange();
	}

	UEdGraphPin* CallTargetPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEndpoint, EndpointDefinition.PossibleCallTargetClass);

	// Map of the Payload Variable Names to their Default Values as Strings
	TMap<FName, FMovieSceneDirectorBlueprintVariableValue> PayloadVariables;
	if (PayloadTemplate && EnumHasAnyFlags(AutoCreatePayload, EAutoCreatePayload::Variables | EAutoCreatePayload::Pins))
	{
		UFunction* PayloadTemplateFunction = nullptr;
		if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(PayloadTemplate))
		{
			PayloadTemplateFunction = EventNode->FindEventSignatureFunction();
		}
		else if (UK2Node_FunctionEntry* FunctionEntryNode = Cast<UK2Node_FunctionEntry>(PayloadTemplate))
		{
			PayloadTemplateFunction = FunctionEntryNode->FindSignatureFunction();
		}
		else if (UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(PayloadTemplate))
		{
			PayloadTemplateFunction = CallFunctionNode->GetTargetFunction();
		}

		TSet<FName> NonPayloadPins;
		if (PayloadTemplateFunction)
		{
			const FString* WorldContextParamName = PayloadTemplateFunction->FindMetaData(FBlueprintMetadata::MD_WorldContext);
			if (WorldContextParamName)
			{
				NonPayloadPins.Add(FName(*WorldContextParamName));
			}
		}

		UK2Node_EditablePinBase* EditableNode = Cast<UK2Node_EditablePinBase>(NewEndpoint);

		for (UEdGraphPin* PayloadPin : PayloadTemplate->Pins)
		{
			if (PayloadPin != CallTargetPin && 
					PayloadPin->LinkedTo.Num() == 0 && 
					PayloadPin->Direction == EGPD_Input && 
					PayloadPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec && 
					PayloadPin->PinName != UEdGraphSchema_K2::PN_Self &&
					!NonPayloadPins.Contains(PayloadPin->PinName))
			{
				// Make a payload variable for this pin
				if (EnumHasAnyFlags(AutoCreatePayload, EAutoCreatePayload::Variables))
				{
					PayloadVariables.Add(PayloadPin->PinName, FMovieSceneDirectorBlueprintVariableValue{ PayloadPin->DefaultObject, PayloadPin->DefaultValue });
				}

				// Make a matching user pin on the endpoint node
				if (EditableNode && EnumHasAnyFlags(AutoCreatePayload, EAutoCreatePayload::Pins))
				{
					// Pins for ref parameters for functions default to bIsReference but the payload cannot be by reference.
					PayloadPin->PinType.bIsReference = false;

					UEdGraphPin* NewPin = EditableNode->CreateUserDefinedPin(PayloadPin->PinName, PayloadPin->PinType, EGPD_Output);
					if (PayloadTemplate != NewEndpoint && NewPin)
					{
						NewPin->MakeLinkTo(PayloadPin);
					}
				}
			}
		}
	}

	// Create payload variables for new parameters, remove payload variables for parameters
	// that don't exist anymore.
	const FMovieSceneDirectorBlueprintVariableValue EmptyValue;
	for (const TPair<UMovieSceneSequence*, FSequenceData>& Pair : AllSequenceData)
	{
		const TArray<void*>& RawData = Pair.Value.RawData;
		const TArray<UObject*>& EditObjects = Pair.Value.EditObjects;

		for (int32 Index = 0; Index < RawData.Num(); ++Index)
		{
			FPayloadVariableMap OldPayloadVariables;
			GetPayloadVariables(EditObjects[Index], RawData[Index], OldPayloadVariables);

			for (const TPair<FName, FMovieSceneDirectorBlueprintVariableValue>& PayloadVar : PayloadVariables)
			{
				if (!OldPayloadVariables.Contains(PayloadVar.Key))
				{
					SetPayloadVariable(EditObjects[Index], RawData[Index], PayloadVar.Key, PayloadVar.Value);
				}
			}

			for (const TPair<FName, FMovieSceneDirectorBlueprintVariableValue>& PayloadVar : OldPayloadVariables)
			{
				if (!PayloadVariables.Contains(PayloadVar.Key))
				{
					SetPayloadVariable(EditObjects[Index], RawData[Index], PayloadVar.Key, EmptyValue);
				}
			}
		}

		OnSetEndpoint(Sequence, Blueprint, EditObjects, RawData, EndpointDefinition, NewEndpoint);
	}

	// Ensure that anything listening for property changed notifications are notified of the new binding
	if (PropertyHandle.IsValid())
	{
		// Ensure that anything listening for property changed notifications are notified of the new binding
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	// Compile the blueprint now that clients have had a chance to update underlying data (we do this after to ensure we are compiling the correct data)
	if (Blueprint)
	{
		FKismetEditorUtilities::CompileBlueprint(Blueprint);
	}

	// Forcibly update the panel now that our endpoint has changed
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}

	if (NewEndpoint)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewEndpoint, false);
	}
}

void FMovieSceneDirectorBlueprintEndpointCustomization::CreateEndpoint()
{
	struct FSequenceData
	{
		TArray<void*> RawData;
		TArray<UObject*> EditObjects;
	};

	TSortedMap<UMovieSceneSequence*, FSequenceData> PerSequenceData;

	// Populate all the sequences represented by this customization
	{
		TArray<UObject*> EditObjects;
		GetEditObjects(EditObjects);

		check(PropertyRawData.Num() == EditObjects.Num());

		for (int32 Index = 0; Index < PropertyRawData.Num(); ++Index)
		{
			FSequenceData& SequenceData = PerSequenceData.FindOrAdd(EditObjects[Index]->GetTypedOuter<UMovieSceneSequence>());
			SequenceData.RawData.Add(PropertyRawData[Index]);
			SequenceData.EditObjects.Add(EditObjects[Index]);
		}
	}

	// Create user facing endpoint
	FScopedTransaction Transaction(LOCTEXT("CreateEndpoint", "Create Endpoint"));

	if (PropertyHandle.IsValid())
	{
		PropertyHandle->NotifyPreChange();
	}

	UK2Node* LastNewEndpoint = nullptr;
	TArray<UBlueprint*> BlueprintsToRecompile;

	for (const TPair<UMovieSceneSequence*, FSequenceData>& SequencePair : PerSequenceData)
	{
		FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(SequencePair.Key);
		if (!SequenceEditor)
		{
			continue;
		}

		UBlueprint* SequenceDirectorBP = SequenceEditor->GetOrCreateDirectorBlueprint(SequencePair.Key);
		if (!SequenceDirectorBP)
		{
			continue;
		}

		FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition = GenerateEndpointDefinition(SequencePair.Key);

		SequenceDirectorBP->Modify();
		BlueprintsToRecompile.Add(SequenceDirectorBP);

		UK2Node* NewEndpoint = FMovieSceneDirectorBlueprintUtils::CreateEndpoint(SequenceDirectorBP, EndpointDefinition);
		if (!NewEndpoint)
		{
			continue;
		}

		OnCreateEndpoint(SequencePair.Key, SequenceDirectorBP, SequencePair.Value.EditObjects, SequencePair.Value.RawData, EndpointDefinition, NewEndpoint);

		FBlueprintEditorUtils::MarkBlueprintAsModified(SequenceDirectorBP);

		LastNewEndpoint = NewEndpoint;
	}

	if (PropertyHandle.IsValid())
	{
		// Ensure that anything listening for property changed notifications are notified of the new binding
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	// Compile the blueprint now that clients have had a chance to update underlying data (we do this after to ensure we are compiling the correct data)
	for (UBlueprint* Blueprint : BlueprintsToRecompile)
	{
		FKismetEditorUtilities::CompileBlueprint(Blueprint);
	}

	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}

	// Focus the first created endpoint
	if (LastNewEndpoint)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(LastNewEndpoint, false);
	}
}

UK2Node* FMovieSceneDirectorBlueprintEndpointCustomization::FindEndpoint(UObject* EditObject, void* RawData) const
{
	if (!EditObject || !RawData)
	{
		return nullptr;
	}

	UMovieSceneSequence* Sequence = EditObject->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	if (!SequenceEditor)
	{
		return nullptr;
	}

	UBlueprint* SequenceDirectorBP = SequenceEditor->FindDirectorBlueprint(Sequence);
	if (!SequenceDirectorBP)
	{
		return nullptr;
	}
	
	return FindEndpoint(Sequence, SequenceDirectorBP, EditObject, RawData);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::ClearEndpoint()
{
	FMovieSceneDirectorBlueprintEndpointDefinition EmptyDefinition;
	SetEndpoint(EmptyDefinition, nullptr, nullptr, EAutoCreatePayload::None);
}

void FMovieSceneDirectorBlueprintEndpointCustomization::NavigateToDefinition()
{
	UEdGraphNode* CommonEndpoint = GetCommonEndpoint();
	if (CommonEndpoint)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(CommonEndpoint, false);
	}
}

#undef LOCTEXT_NAMESPACE

