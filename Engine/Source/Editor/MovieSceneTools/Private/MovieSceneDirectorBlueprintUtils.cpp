// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDirectorBlueprintUtils.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_MakeArray.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"

#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSpawnable.h"

#define LOCTEXT_NAMESPACE "MovieSceneDirectorBlueprintUtils"

FString FMovieSceneDirectorBlueprintEndpointParameter::SanitizePinName(const FString& InPinName)
{
	FString SanitizedPinName(InPinName);
	for (TCHAR& Char : SanitizedPinName)
	{
		if (FCString::Strchr(INVALID_NAME_CHARACTERS, Char) != nullptr)
		{
			Char = '_';
		}
	}
	return SanitizedPinName;
}

UK2Node* FMovieSceneDirectorBlueprintUtils::CreateEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition)
{
	switch (EndpointDefinition.EndpointType)
	{
		case EMovieSceneDirectorBlueprintEndpointType::Event:
			return CreateEventEndpoint(Blueprint, EndpointDefinition);
		case EMovieSceneDirectorBlueprintEndpointType::Function:
			return CreateFunctionEndpoint(Blueprint, EndpointDefinition);
		default:
			return nullptr;
	}
}

UK2Node_CustomEvent* FMovieSceneDirectorBlueprintUtils::CreateEventEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition)
{
	// Create a custom event node to replace the original event node imported from text
	UEdGraph* SequenceEventGraph = FindOrCreateEventGraph(Blueprint, EndpointDefinition.GraphName);
	UK2Node_CustomEvent* CustomEventNode = NewObject<UK2Node_CustomEvent>(SequenceEventGraph);

	FString EndpointName = FMovieSceneDirectorBlueprintEndpointParameter::SanitizePinName(EndpointDefinition.EndpointName);
	check(EndpointName.Len() != 0);
	CustomEventNode->CustomFunctionName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, EndpointName);

	// Ensure that it is editable
	CustomEventNode->bIsEditable = true;

	// Setup defaults
	CustomEventNode->CreateNewGuid();
	CustomEventNode->PostPlacedNewNode();
	CustomEventNode->AllocateDefaultPins();

	const FVector2D NewPosition = SequenceEventGraph->GetGoodPlaceForNewNode();
	CustomEventNode->NodePosX = NewPosition.X;
	CustomEventNode->NodePosY = NewPosition.Y;

	// Add extra pins and invoke custom setup
	for (const FMovieSceneDirectorBlueprintEndpointParameter& ExtraPin : EndpointDefinition.ExtraPins)
	{
		FEdGraphPinType PinType;
		PinType.PinCategory = ExtraPin.PinTypeCategory;
		PinType.PinSubCategoryObject = ExtraPin.PinTypeClass;
		FString PinName = FMovieSceneDirectorBlueprintEndpointParameter::SanitizePinName(ExtraPin.PinName);
		CustomEventNode->CreateUserDefinedPin(FName(PinName), PinType, ExtraPin.PinDirection);
	}

	SequenceEventGraph->AddNode(CustomEventNode, false, false);

	FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, CustomEventNode->CustomFunctionName);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	return CustomEventNode;
}

UK2Node_FunctionEntry* FMovieSceneDirectorBlueprintUtils::CreateFunctionEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition)
{
	// Create the new function graph
	FString EndpointName = FMovieSceneDirectorBlueprintEndpointParameter::SanitizePinName(EndpointDefinition.EndpointName);
	check(EndpointName.Len() != 0);
	FName FuncGraphName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, EndpointName);
	UEdGraph* FuncGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FuncGraphName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	const bool bIsUserCreated = true;
	FBlueprintEditorUtils::AddFunctionGraph<UFunction>(Blueprint, FuncGraph, bIsUserCreated, EndpointDefinition.EndpointSignature);

	// Find the entry node.
	TArray<UK2Node_FunctionEntry*> EntryNodes;
	FuncGraph->GetNodesOfClass(EntryNodes);
	check(EntryNodes.Num() == 1);

	// Add extra pins and invoke custom setup
	for (const FMovieSceneDirectorBlueprintEndpointParameter& ExtraPin : EndpointDefinition.ExtraPins)
	{
		FEdGraphPinType PinType;
		PinType.PinCategory = ExtraPin.PinTypeCategory;
		PinType.PinSubCategoryObject = ExtraPin.PinTypeClass;
		FString PinName = FMovieSceneDirectorBlueprintEndpointParameter::SanitizePinName(ExtraPin.PinName);
		EntryNodes[0]->CreateUserDefinedPin(FName(PinName), PinType, ExtraPin.PinDirection);
	}

	FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, EntryNodes[0]->CustomGeneratedFunctionName);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	return EntryNodes[0];
}

UEdGraph* FMovieSceneDirectorBlueprintUtils::FindOrCreateEventGraph(UBlueprint* Blueprint, const FString& GraphName)
{
	UEdGraph* SequenceEventGraph = FindObject<UEdGraph>(Blueprint, *GraphName);
	if (!SequenceEventGraph)
	{
		SequenceEventGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FName(GraphName), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

		SequenceEventGraph->GraphGuid = FGuid::NewGuid();
		Blueprint->UbergraphPages.Add(SequenceEventGraph);
	}
	return SequenceEventGraph;
}

FMovieSceneDirectorBlueprintEntrypointResult FMovieSceneDirectorBlueprintUtils::GenerateEntryPoint(const FMovieSceneDirectorBlueprintEndpointCall& EndpointCall, FKismetCompilerContext* Compiler)
{
	check(Compiler && EndpointCall.Endpoint);

	UEdGraphNode* Endpoint = EndpointCall.Endpoint;
	UBlueprint* Blueprint = Compiler->Blueprint;

	UFunction* EndpointFunction = nullptr;

	// Find the function that we need to call on the skeleton class
	{
		if (UK2Node_Event* Event = Cast<UK2Node_Event>(Endpoint))
		{
			EndpointFunction = Blueprint->SkeletonGeneratedClass ? Blueprint->SkeletonGeneratedClass->FindFunctionByName(Event->GetFunctionName()) : nullptr;
		}
		else if (UK2Node_FunctionEntry* EndpointEntry = Cast<UK2Node_FunctionEntry>(Endpoint))
		{
			EndpointFunction = EndpointEntry->FindSignatureFunction();
		}
		else
		{
			Compiler->MessageLog.Error(*LOCTEXT("InvalidEndpoint_Error", "Sequencer event is bound to an invalid endpoint node @@").ToString(), Endpoint);
		}
	}

	if (EndpointFunction == nullptr)
	{
		return FMovieSceneDirectorBlueprintEntrypointResult();
	}

	// @todo: use a more destriptive name for event entry points?
	static FString DefaultEventEntryName = TEXT("SequenceEvent__ENTRYPOINT");
	UEdGraph* EntryPointGraph = Compiler->SpawnIntermediateFunctionGraph(DefaultEventEntryName + Blueprint->GetName());
	check(EntryPointGraph->Nodes.Num() == 1);

	const UEdGraphSchema*  Schema        = EntryPointGraph->GetSchema();
	UK2Node_FunctionEntry* FunctionEntry = CastChecked<UK2Node_FunctionEntry>(EntryPointGraph->Nodes[0]);

	FMovieSceneDirectorBlueprintEntrypointResult Result;
	Result.Entrypoint = FunctionEntry;
	Result.CompiledFunctionName = FunctionEntry->GetGraph()->GetFName();

	// -------------------------------------------------------------------------------------
	// Locate and initialize the function entry node
	{
		int32 ExtraFunctionFlags = ( FUNC_BlueprintEvent | FUNC_Public );
		FunctionEntry->AddExtraFlags(ExtraFunctionFlags);
		FunctionEntry->bIsEditable = false;
		FunctionEntry->MetaData.Category = LOCTEXT("DefaultCategory", "Sequencer Event Endpoints");
		FunctionEntry->MetaData.bCallInEditor = EndpointFunction->GetBoolMetaData(FBlueprintMetadata::MD_CallInEditor);
	}

	// -------------------------------------------------------------------------------------
	// Create a function call node to invoke the endpoint function itself
	UK2Node_CallFunction* CallFunctionNode = NewObject<UK2Node_CallFunction>(EntryPointGraph);
	{
		CallFunctionNode->CreateNewGuid();
		CallFunctionNode->PostPlacedNewNode();

		CallFunctionNode->FunctionReference.SetSelfMember(EndpointFunction->GetFName());

		CallFunctionNode->ReconstructNode();

		CallFunctionNode->NodePosX = FunctionEntry->NodePosX + 400.f;
		CallFunctionNode->NodePosY = FunctionEntry->NodePosY - 16.f;

		EntryPointGraph->AddNode(CallFunctionNode, false, false);
	}

	// -------------------------------------------------------------------------------------
	// If the endpoint function has a return value, create a return node for it
	UK2Node_FunctionResult* ResultNode = nullptr;
	if (FProperty* ReturnParam = EndpointFunction->GetReturnProperty())
	{
		ResultNode = NewObject<UK2Node_FunctionResult>(EntryPointGraph);

		ResultNode->CreateNewGuid();
		ResultNode->PostPlacedNewNode();

		ResultNode->FunctionReference.SetSelfMember(EndpointFunction->GetFName());

		ResultNode->ReconstructNode();

		ResultNode->NodePosX = CallFunctionNode->NodePosX + 400.f;
		ResultNode->NodePosY = CallFunctionNode->NodePosY - 16.f;

		EntryPointGraph->AddNode(ResultNode, false, false);
	}

	// -------------------------------------------------------------------------------------
	// Reserve pins that want to be assigned a value at runtime
	for (const FName& ExposedPinName : EndpointCall.ExposedPinNames)
	{
		UEdGraphPin* ExposedPin = CallFunctionNode->FindPin(ExposedPinName, EGPD_Input);
		if (ExposedPin)
		{
			UEdGraphPin* BoundParamPinInput = FunctionEntry->CreateUserDefinedPin(ExposedPin->PinName, ExposedPin->PinType, EGPD_Output, true);
			if (BoundParamPinInput)
			{
				BoundParamPinInput->MakeLinkTo(ExposedPin);
			}
		}
	}

	// -------------------------------------------------------------------------------------
	// Wire up the function entry 'then' pin to the call function 'execute' pin
	// If there's a result node, wire up the call function's result to its 'Return Value' pin
	{
		// Wire up the exec nodes
		UEdGraphPin* ThenPin = FunctionEntry->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
		UEdGraphPin* ExecPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);

		if (ensure(ThenPin && ExecPin))
		{
			ThenPin->MakeLinkTo(ExecPin);
		}

		if (ResultNode)
		{
			UEdGraphPin* CallResultPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);
			UEdGraphPin* OutputResultPin = ResultNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Input);

			if (ensure(CallResultPin && OutputResultPin))
			{
				CallResultPin->MakeLinkTo(OutputResultPin);
			}

			ThenPin = CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
			ExecPin = ResultNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);

			if (ensure(ThenPin && ExecPin))
			{
				ThenPin->MakeLinkTo(ExecPin);
			}
		}
	}

	// -------------------------------------------------------------------------------------
	// Set pin defaults for each of the relevant pins on the call function node according to the payload
	{
		TArray<FName, TInlineAllocator<8>> ValidPinNames;

		// Gather all input pins for the function call
		for (int32 PinIndex = 0; PinIndex < CallFunctionNode->Pins.Num(); ++PinIndex)
		{
			// Only consider input pins that are data, and not already connected
			UEdGraphPin* Pin = CallFunctionNode->Pins[PinIndex];
			if (Pin->Direction != EGPD_Input || Pin->PinName == UEdGraphSchema_K2::PN_Execute || Pin->PinName == UEdGraphSchema_K2::PN_Self || Pin->LinkedTo.Num() != 0)
			{
				continue;
			}

			UClass* PinObjectType = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get());
			const bool bIsArrayPin = Pin->PinType.IsArray();
			const bool bIsRawActorPin = Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object && PinObjectType && PinObjectType->IsChildOf(AActor::StaticClass());

			if (!ensureMsgf(!bIsArrayPin, TEXT("Array parameters aren't supported")))
			{
				continue;
			}

			const FMovieSceneDirectorBlueprintVariableValue* PayloadVariable = EndpointCall.PayloadVariables.Find(Pin->PinName);
			if (!PayloadVariable)
			{
				continue;
			}

			bool bSuccess = false;
			if (bIsRawActorPin)
			{
				bSuccess = GenerateEntryPointRawActorParameter(Compiler, EntryPointGraph, Endpoint, CallFunctionNode, Pin, *PayloadVariable);
			}
			else
			{
				bool bMarkAsModified = false;
				if (PayloadVariable->ObjectValue.IsValid())
				{
					if (UObject* ResolvedObject = PayloadVariable->ObjectValue.ResolveObject())
					{
						Schema->TrySetDefaultObject(*Pin, ResolvedObject, bMarkAsModified);
					}
					else
					{
						Schema->TrySetDefaultValue(*Pin, PayloadVariable->ObjectValue.ToString(), bMarkAsModified);
					}
				}
				else if (!PayloadVariable->Value.IsEmpty())
				{
					Schema->TrySetDefaultValue(*Pin, PayloadVariable->Value, bMarkAsModified);
				}
				bSuccess = true;
			}

			if (bSuccess)
			{
				ValidPinNames.Add(Pin->PinName);
			}
		}

		// Remove any invalid payload variable names
		if (ValidPinNames.Num() != EndpointCall.PayloadVariables.Num())
		{
			const FText ErrorFmt = LOCTEXT("PayloadParameterRemoved", "Stale Sequencer endpoint payload parameter {0} has been removed.");
			for (auto It = EndpointCall.PayloadVariables.CreateConstIterator(); It; ++It)
			{
				if (!ValidPinNames.Contains(It->Key))
				{
					Compiler->MessageLog.Note(*FText::Format(ErrorFmt, FText::FromName(It->Key)).ToString());
					Result.StalePayloadVariables.Add(It->Key);
				}
			}
		}
	}

	// -------------------------------------------------------------------------------------
	// Generate any other custom pins and nodes
	if (EndpointCall.CustomizeEndpointCall.IsBound())
	{
		FMovieSceneCustomizeDirectorBlueprintEndpointCallParams Params;
		Params.FunctionGraph = EntryPointGraph;
		Params.FunctionEntryNode = FunctionEntry;
		Params.CallFunctionNode = CallFunctionNode;
		Params.FunctionResultNode = ResultNode;
		EndpointCall.CustomizeEndpointCall.Execute(Params);
	}
	// WARNING: After this call, we have to be careful about looking for stuff in the function graph,
	//		    since there could be any number of new nodes and pins.

	return Result;
}

bool FMovieSceneDirectorBlueprintUtils::GenerateEntryPointRawActorParameter(
			FKismetCompilerContext* Compiler, UEdGraph* Graph, UEdGraphNode* Endpoint, 
			UK2Node* OriginNode, UEdGraphPin* DestPin, const FMovieSceneDirectorBlueprintVariableValue& PayloadValue)
{
	// Raw actor properties are represented as soft object ptrs under the hood so that we can still serialize them
	// To make this work, we have to make the following graph:
	// MakeSoftObjectPath('<ActorPath>') -> Conv_SoftObjPathToSoftObjRef() -> Conv_SoftObjectReferenceToObject() -> Cast<ActorType>()
	UK2Node_CallFunction* MakeSoftObjectPathNode = Compiler->SpawnIntermediateNode<UK2Node_CallFunction>(OriginNode, Graph);
	MakeSoftObjectPathNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeSoftObjectPath), UKismetSystemLibrary::StaticClass());
	MakeSoftObjectPathNode->AllocateDefaultPins();

	UK2Node_CallFunction* ConvertToSoftObjectRef = Compiler->SpawnIntermediateNode<UK2Node_CallFunction>(OriginNode, Graph);
	ConvertToSoftObjectRef->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_SoftObjPathToSoftObjRef), UKismetSystemLibrary::StaticClass());
	ConvertToSoftObjectRef->AllocateDefaultPins();

	UK2Node_CallFunction* ConvertToObjectFunc = Compiler->SpawnIntermediateNode<UK2Node_CallFunction>(OriginNode, Graph);
	ConvertToObjectFunc->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_SoftObjectReferenceToObject), UKismetSystemLibrary::StaticClass());
	ConvertToObjectFunc->AllocateDefaultPins();

	UClass* PinObjectType = Cast<UClass>(DestPin->PinType.PinSubCategoryObject.Get());

	UK2Node_DynamicCast* CastNode = Compiler->SpawnIntermediateNode<UK2Node_DynamicCast>(OriginNode, Graph);
	CastNode->SetPurity(true);
	CastNode->TargetType = PinObjectType;
	CastNode->AllocateDefaultPins();

	UEdGraphPin* PathInput     = MakeSoftObjectPathNode->FindPin(FName("PathString"));
	UEdGraphPin* PathOutput    = MakeSoftObjectPathNode->GetReturnValuePin();
	UEdGraphPin* SoftRefInput  = ConvertToSoftObjectRef->FindPin(FName("SoftObjectPath"));
	UEdGraphPin* SoftRefOutput = ConvertToSoftObjectRef->GetReturnValuePin();
	UEdGraphPin* ConvertInput  = ConvertToObjectFunc->FindPin(FName("SoftObject"));
	UEdGraphPin* ConvertOutput = ConvertToObjectFunc->GetReturnValuePin();
	UEdGraphPin* CastInput     = CastNode->GetCastSourcePin();
	UEdGraphPin* CastOutput    = CastNode->GetCastResultPin();

	if (!PathInput || !PathOutput || !SoftRefInput || !SoftRefOutput || !ConvertInput || !ConvertOutput || !CastInput || !CastOutput)
	{
		Compiler->MessageLog.Error(*LOCTEXT("ActorFacadeError", "GenerateEntryPoint: Failed to generate soft-ptr facade for AActor payload property property @@. @@").ToString(), *DestPin->PinName.ToString(), Endpoint);
		return false;
	}

	const UEdGraphSchema*  Schema = Graph->GetSchema();

	// Set the default value for the path string/object
	const bool bMarkAsModified = false;
	if (PayloadValue.ObjectValue.IsValid())
	{
		Schema->TrySetDefaultValue(*PathInput, PayloadValue.ObjectValue.ToString(), bMarkAsModified);
	}
	else if (!PayloadValue.Value.IsEmpty())
	{
		Schema->TrySetDefaultValue(*PathInput, PayloadValue.Value, bMarkAsModified);
	}

	bool bSuccess = true;
	bSuccess &= Schema->TryCreateConnection(PathOutput, SoftRefInput);
	bSuccess &= Schema->TryCreateConnection(SoftRefOutput, ConvertInput);
	bSuccess &= Schema->TryCreateConnection(ConvertOutput, CastInput);
	bSuccess &= Schema->TryCreateConnection(CastOutput, DestPin);

	if (!bSuccess)
	{
		Compiler->MessageLog.Error(*LOCTEXT("ActorFacadeConnectionError", "GenerateEntryPoint: Failed to connect nodes for soft-ptr facade in AActor payload property @@. @@").ToString(), *DestPin->PinName.ToString(), Endpoint);
	}

	return bSuccess;
}

UEdGraphPin* FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(UK2Node* InEndpoint, UClass* CallTargetClass)
{
	if (!CallTargetClass)
	{
		return nullptr;
	}

	for (UEdGraphPin* Pin : InEndpoint->Pins)
	{
		if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object && Pin->PinType.PinSubCategoryObject == CallTargetClass)
		{
			return Pin;
		}
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
