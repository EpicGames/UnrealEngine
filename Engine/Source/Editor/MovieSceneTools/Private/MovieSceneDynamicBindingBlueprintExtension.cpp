// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDynamicBindingBlueprintExtension.h"

#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneDynamicBinding.h"
#include "MovieSceneDynamicBindingUtils.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDynamicBindingBlueprintExtension)

class UEdGraphNode;

void UMovieSceneDynamicBindingBlueprintExtension::BindTo(TWeakObjectPtr<UMovieSceneSequence> InMovieSceneSequence)
{
	WeakMovieSceneSequences.AddUnique(InMovieSceneSequence);
}

void UMovieSceneDynamicBindingBlueprintExtension::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (WeakMovieSceneSequence_DEPRECATED.IsValid())
		{
			WeakMovieSceneSequences.Add(WeakMovieSceneSequence_DEPRECATED);
			WeakMovieSceneSequence_DEPRECATED.Reset();
		}
	}
}

void UMovieSceneDynamicBindingBlueprintExtension::PostLoad()
{
	WeakMovieSceneSequences.Remove(nullptr);
	Super::PostLoad();
}

void UMovieSceneDynamicBindingBlueprintExtension::HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint)
{
	for (TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence : WeakMovieSceneSequences)
	{
		if (UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get())
		{
			UBlueprint::ForceLoad(MovieSceneSequence);
			if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
			{
				UBlueprint::ForceLoad(MovieScene);
			}
		}
	}
}

void UMovieSceneDynamicBindingBlueprintExtension::HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext)
{
	for (TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence : WeakMovieSceneSequences)
	{
		UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get();
		HandleGenerateFunctionGraphs(CompilerContext, MovieSceneSequence);
	}
}

void UMovieSceneDynamicBindingBlueprintExtension::HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext, UMovieSceneSequence* MovieSceneSequence)
{
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	ensureMsgf(!MovieScene->HasAnyFlags(RF_NeedLoad), TEXT("Attempting to generate entry point functions before a movie scene has been loaded"));

	// Generate a function graph for each endpoint used by the bound sequence. This function graph is simply a call
	// to the endpoint function with the payload variables set as the call parameters.
	auto GenerateFunctionGraphs = 
		[CompilerContext]
		(const FGuid& ObjectBinding, FMovieSceneDynamicBinding& DynamicBinding)
	{
		UEdGraphNode* Endpoint = Cast<UK2Node>(DynamicBinding.WeakEndpoint.Get());
		if (Endpoint)
		{
			// Set up the endpoint call, with our payload variables.
			FMovieSceneDirectorBlueprintEndpointCall EndpointCall;
			EndpointCall.Endpoint = Endpoint;
			if (!DynamicBinding.ResolveParamsPinName.IsNone())
			{
				EndpointCall.ExposedPinNames.Add(DynamicBinding.ResolveParamsPinName);
			}
			for (auto& Pair : DynamicBinding.PayloadVariables)
			{
				EndpointCall.PayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
			}

			// Create the endpoint call, and clean-up stale payload variables.
			FMovieSceneDirectorBlueprintEntrypointResult EntrypointResult = FMovieSceneDirectorBlueprintUtils::GenerateEntryPoint(EndpointCall, CompilerContext);
			DynamicBinding.CompiledFunctionName = EntrypointResult.CompiledFunctionName;
			EntrypointResult.CleanUpStalePayloadVariables(DynamicBinding.PayloadVariables);
		}
	};
	FMovieSceneDynamicBindingUtils::IterateDynamicBindings(MovieScene, GenerateFunctionGraphs);

	// This callback sets the generated function calls back onto the custom director blueprint bindings in the sequence,
	// and keeps a pointer to any needed arguments on this function, so that we can pass special values later
	// at runtime.
	TWeakObjectPtr<UMovieSceneSequence> WeakMovieSceneSequence(MovieSceneSequence);
	auto OnFunctionListGenerated = [WeakMovieSceneSequence](FKismetCompilerContext* CompilerContext)
	{
		UMovieSceneSequence* MovieSceneSequence = WeakMovieSceneSequence.Get();
		UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
		if (!ensureMsgf(MovieSceneSequence, TEXT("A movie scene was garbage-collected while its director blueprint was being compiled!")))
		{
			return;
		}

		UBlueprint* Blueprint = CompilerContext->Blueprint;
		if (!Blueprint->GeneratedClass)
		{
			return;
		}

		TArray<FMovieSceneDynamicBinding*> DynamicBindings;
		FMovieSceneDynamicBindingUtils::GatherDynamicBindings(MovieScene, DynamicBindings);
		for (FMovieSceneDynamicBinding* DynamicBinding : DynamicBindings)
		{
			// Set the pointer to the resolver function to invoke.
			if (DynamicBinding->CompiledFunctionName != NAME_None)
			{
				DynamicBinding->Function = Blueprint->GeneratedClass->FindFunctionByName(DynamicBinding->CompiledFunctionName);
			}
			else
			{
				DynamicBinding->Function = nullptr;
			}

			DynamicBinding->CompiledFunctionName = NAME_None;

			// Set the pointer to the resolve parameter field, if any.
			if (DynamicBinding->Function && !DynamicBinding->ResolveParamsPinName.IsNone())
			{
				FProperty* ResolveParamsProp = DynamicBinding->Function->FindPropertyByName(DynamicBinding->ResolveParamsPinName);
				DynamicBinding->ResolveParamsProperty = ResolveParamsProp;
			}
			else
			{
				DynamicBinding->ResolveParamsProperty = nullptr;
			}
		}

		if (!Blueprint->bIsRegeneratingOnLoad)
		{
			MovieScene->MarkAsChanged();
			MovieScene->MarkPackageDirty();
		}
	};

	CompilerContext->OnFunctionListCompiled().AddLambda(OnFunctionListGenerated);
}

