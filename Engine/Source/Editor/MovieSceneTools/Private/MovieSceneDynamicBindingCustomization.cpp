// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDynamicBindingCustomization.h"

#include "MovieScene.h"
#include "MovieSceneDynamicBinding.h"
#include "MovieSceneDynamicBindingUtils.h"
#include "MovieSceneSequence.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "EdGraphSchema_K2.h"
#include "IPropertyUtilities.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MovieSceneDynamicBindingCustomization"

TSharedRef<IPropertyTypeCustomization> FMovieSceneDynamicBindingCustomization::MakeInstance()
{
	TSharedRef<FMovieSceneDynamicBindingCustomization> Instance = MakeShared<FMovieSceneDynamicBindingCustomization>();
	return Instance;
}

TSharedRef<IPropertyTypeCustomization> FMovieSceneDynamicBindingCustomization::MakeInstance(UMovieScene* InMovieScene, FGuid InObjectBinding, int32 InBindingIndex)
{
	TSharedRef<FMovieSceneDynamicBindingCustomization> Instance = MakeShared<FMovieSceneDynamicBindingCustomization>();
	Instance->EditedMovieScene = InMovieScene;
	Instance->ObjectBinding = InObjectBinding;
	Instance->BindingIndex = InBindingIndex;
	return Instance;
}

void FMovieSceneDynamicBindingCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FMovieSceneDirectorBlueprintEndpointCustomization::CustomizeChildren(InPropertyHandle, ChildBuilder, CustomizationUtils);
	FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	BindingIndex = 0;
	if (PropertyPath.StartsWith(TEXT("Bindings[")))
	{
		int32 RightBounds;
		if (PropertyPath.FindChar(']', RightBounds))
		{
			FStringView BindingIndexStr = PropertyPath.Mid(9, RightBounds);
			// Grab the BindingIndex out of the PropertyPath
			BindingIndex = FCString::Atoi(BindingIndexStr.GetData());
		}
	}
}

void FMovieSceneDynamicBindingCustomization::GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const
{
	const FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData);
	for (const TPair<FName, FMovieSceneDynamicBindingPayloadVariable>& Pair : DynamicBinding->PayloadVariables)
	{
		OutPayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
	}
}

bool FMovieSceneDynamicBindingCustomization::SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewVariableValue)
{
	UMovieScene* MovieScene = Cast<UMovieScene>(EditObject);
	FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData);
	if (!MovieScene || !DynamicBinding)
	{
		return false;
	}

	MovieScene->Modify();

	if (!NewVariableValue.Value.IsEmpty())
	{
		FMovieSceneDynamicBindingPayloadVariable* PayloadVariable = DynamicBinding->PayloadVariables.Find(FieldName);
		if (!PayloadVariable)
		{
			PayloadVariable = &DynamicBinding->PayloadVariables.Add(FieldName);
		}

		PayloadVariable->Value = NewVariableValue.Value;
		PayloadVariable->ObjectValue = NewVariableValue.ObjectValue;
	}
	else
	{
		DynamicBinding->PayloadVariables.Remove(FieldName);
	}

	return true;
}

UK2Node* FMovieSceneDynamicBindingCustomization::FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const
{
	FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData);
	if (UK2Node* Node = Cast<UK2Node>(DynamicBinding->WeakEndpoint.Get()))
	{
		return Node;
	}
	return nullptr;
}

void FMovieSceneDynamicBindingCustomization::GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutWellKnownParameters) const
{
	FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData);
	OutWellKnownParameters.Add(DynamicBinding->ResolveParamsPinName);
}

void FMovieSceneDynamicBindingCustomization::GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const
{
	FWellKnownParameterCandidates ResolveParamsCandidates;
	ResolveParamsCandidates.Metadata.PickerLabel = LOCTEXT("ResolveParamsPin_Label", "Pass Resolve Params To");
	ResolveParamsCandidates.Metadata.PickerTooltip = LOCTEXT("ResolveParamsPin_Tooltip", "Specifies a pin to pass the resolve parameters through when the binding is resolved.");

	for (UEdGraphPin* Pin : Endpoint->Pins)
	{
		// Parameter pins are outputs on the function entry node.
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		// Pin of type FMovieSceneDynamicBindingResolveParams is eligible for passing the resolve params.
		if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
				Pin->PinType.PinSubCategoryObject == FMovieSceneDynamicBindingResolveParams::StaticStruct())
		{
			ResolveParamsCandidates.CandidatePinNames.Add(Pin->GetFName());
		}
	}
	
	OutCandidates.Add(ResolveParamsCandidates);
}

bool FMovieSceneDynamicBindingCustomization::SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName)
{
	FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData);
	switch (ParameterIndex)
	{
		case 0:
			DynamicBinding->ResolveParamsPinName = BoundPinName;
			return true;
	}
	return false;
}

FMovieSceneDirectorBlueprintEndpointDefinition FMovieSceneDynamicBindingCustomization::GenerateEndpointDefinition(UMovieSceneSequence* Sequence)
{
	FMovieSceneDirectorBlueprintEndpointDefinition Definition;
	Definition.EndpointType = EMovieSceneDirectorBlueprintEndpointType::Function;

	// We use a dummy utility class to get a function signature that takes no parameter and returns a UObject pointer.
	static const FName SampleResolveBindingFuncName("SampleResolveBinding");
	UClass* EndpointUtilClass = UMovieSceneDynamicBindingEndpointUtil::StaticClass();
	Definition.EndpointSignature = EndpointUtilClass->FindFunctionByName(SampleResolveBindingFuncName);
	check(Definition.EndpointSignature);

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBinding))
	{
		Definition.EndpointName = Possessable->GetName() + "_DynamicBinding";
		if (BindingIndex > 0)
		{
			Definition.EndpointName += "_" + FString::FromInt(BindingIndex);
		}
	}

	return Definition;
}

void FMovieSceneDynamicBindingCustomization::OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	MovieScene->Modify();

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		ensureMsgf(
			Cast<UMovieScene>(EditObjects[Index]) == MovieScene, 
			TEXT("Editing dynamic binding endpoint for a different sequence"));

		FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData[Index]);

		// Default call in editor to true
		if (UK2Node_FunctionEntry* CallFunction = Cast<UK2Node_FunctionEntry>(NewEndpoint))
		{
			CallFunction->MetaData.bCallInEditor = true;
		}

		FMovieSceneDynamicBindingUtils::SetEndpoint(MovieScene, DynamicBinding, NewEndpoint);
	}

	FMovieSceneDynamicBindingUtils::EnsureBlueprintExtensionCreated(Sequence, Blueprint);
}

void FMovieSceneDynamicBindingCustomization::OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	check(EditObjects.Num() == RawData.Num());

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		UMovieScene* MovieScene = Cast<UMovieScene>(EditObjects[Index]);
		FMovieSceneDynamicBinding* DynamicBinding = static_cast<FMovieSceneDynamicBinding*>(RawData[Index]);

		FMovieSceneDynamicBindingUtils::SetEndpoint(MovieScene, DynamicBinding, NewEndpoint);

		FMovieSceneDynamicBindingUtils::EnsureBlueprintExtensionCreated(Sequence, Blueprint);
	}
}

void FMovieSceneDynamicBindingCustomization::GetEditObjects(TArray<UObject*>& OutObjects) const
{
	OutObjects.Add(EditedMovieScene);
}

void FMovieSceneDynamicBindingCustomization::OnCollectQuickBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder)
{
	CollectResolverLibraryBindActions(Blueprint, MenuBuilder, false);
}

void FMovieSceneDynamicBindingCustomization::CollectResolverLibraryBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder, bool bIsRebinding)
{
	// We don't show the resolver library endpoints for rebinding, because we should only rebind
	// to other function graphs of the director blueprint.
	if (bIsRebinding)
	{
		return;
	}

	// We want the ability to create CallFunction nodes for any static method that we think can be used
	// as a bound object resolution function.
	FBlueprintActionFilter MenuFilter(FBlueprintActionFilter::BPFILTER_RejectGlobalFields | FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);
	MenuFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
	MenuFilter.Context.Blueprints.Add(Blueprint);
	
	// Add any class that has the "SequencerBindingResolverLibrary" meta as a target class.
	//
	// We don't consider *all* blueprint function libraries because there are many, many of them that expose
	// functions that are, technically speaking, compatible with bound object resolution (i.e. they return
	// a UObject pointer) but that are completely non-sensical in this context.
	const static FName SequencerBindingResolverLibraryMeta("SequencerBindingResolverLibrary");
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* CurrentClass = *ClassIt;
		if (CurrentClass->HasMetaData(SequencerBindingResolverLibraryMeta))
		{
			FBlueprintActionFilter::Add(MenuFilter.TargetClasses, CurrentClass);
		}
	}
	
	auto RejectAnyIncompatibleReturnValues = [](const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
	{
		const UFunction* Function = BlueprintAction.GetAssociatedFunction();
		const FStructProperty* FunctionReturnProperty = CastField<FStructProperty>(Function->GetReturnProperty());
		return 
			FunctionReturnProperty == nullptr || 
			FunctionReturnProperty->Struct != FMovieSceneDynamicBindingResolveResult::StaticStruct();
	};

	MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateLambda(RejectAnyIncompatibleReturnValues));

	MenuBuilder.AddMenuSection(MenuFilter, LOCTEXT("DynamicBindingCustomization", "Resolver Library"), 0);
}

#undef LOCTEXT_NAMESPACE

