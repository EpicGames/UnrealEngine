// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDynamicBindingUtils.h"

#include "K2Node.h"
#include "K2Node_FunctionEntry.h"
#include "MovieSceneSequence.h"
#include "MovieSceneDynamicBindingBlueprintExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDynamicBindingUtils)

void FMovieSceneDynamicBindingUtils::SetEndpoint(UMovieScene* MovieScene, FMovieSceneDynamicBinding* DynamicBinding, UK2Node* NewEndpoint)
{
	UK2Node* ExistingEndpoint = CastChecked<UK2Node>(DynamicBinding->WeakEndpoint.Get(), ECastCheckedType::NullAllowed);
	if (ExistingEndpoint)
	{
		ExistingEndpoint->OnUserDefinedPinRenamed().RemoveAll(MovieScene);
	}

	if (NewEndpoint)
	{
		checkf(
			NewEndpoint->IsA<UK2Node_FunctionEntry>(),
			TEXT("Only functions are supported as dynamic binding endpoints"));

		NewEndpoint->OnUserDefinedPinRenamed().AddUObject(MovieScene, &UMovieScene::OnDynamicBindingUserDefinedPinRenamed);
		DynamicBinding->WeakEndpoint = NewEndpoint;
	}
	else
	{
		DynamicBinding->WeakEndpoint = nullptr;
	}
}

void FMovieSceneDynamicBindingUtils::EnsureBlueprintExtensionCreated(UMovieSceneSequence* MovieSceneSequence, UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	check(MovieSceneSequence);

	for (const TObjectPtr<UBlueprintExtension>& Extension : Blueprint->GetExtensions())
	{
		UMovieSceneDynamicBindingBlueprintExtension* DynamidBindingExtension = Cast<UMovieSceneDynamicBindingBlueprintExtension>(Extension);
		if (DynamidBindingExtension)
		{
			DynamidBindingExtension->BindTo(MovieSceneSequence);
			return;
		}
	}

	UMovieSceneDynamicBindingBlueprintExtension* DynamidBindingExtension = NewObject<UMovieSceneDynamicBindingBlueprintExtension>(Blueprint);
	DynamidBindingExtension->BindTo(MovieSceneSequence);
	Blueprint->AddExtension(DynamidBindingExtension);
}

