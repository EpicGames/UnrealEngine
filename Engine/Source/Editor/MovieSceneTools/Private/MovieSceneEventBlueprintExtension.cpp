// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneEventBlueprintExtension.h"

#include "Channels/MovieSceneEvent.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "K2Node_FunctionEntry.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneEventUtils.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventBlueprintExtension)

class UEdGraphNode;

void UMovieSceneEventBlueprintExtension::PostLoad()
{
	EventSections.Remove(nullptr);
	Super::PostLoad();
}

void UMovieSceneEventBlueprintExtension::HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint)
{
	for (TWeakObjectPtr<UMovieSceneEventSectionBase> WeakEventSection : EventSections)
	{
		if (UMovieSceneEventSectionBase* EventSection = WeakEventSection.Get())
		{
			UBlueprint::ForceLoad(EventSection);
		}
	}
}

void UMovieSceneEventBlueprintExtension::HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext)
{
	for (TWeakObjectPtr<UMovieSceneEventSectionBase> WeakEventSection : EventSections)
	{
		UMovieSceneEventSectionBase* EventSection = WeakEventSection.Get();
		if (!EventSection)
		{
			continue;
		}

		ensureMsgf(!EventSection->HasAnyFlags(RF_NeedLoad), TEXT("Attempting to generate entry point functions before an event section has been loaded"));

		EventSection->AttemptUpgrade();

		for (FMovieSceneEvent& EntryPoint : EventSection->GetAllEntryPoints())
		{
			UEdGraphNode* Endpoint = FMovieSceneEventUtils::FindEndpoint(&EntryPoint, EventSection, CompilerContext->Blueprint);
			if (Endpoint)
			{
				UK2Node_FunctionEntry* FunctionEntry = FMovieSceneEventUtils::GenerateEntryPoint(&EntryPoint, CompilerContext, Endpoint);
				if (FunctionEntry)
				{
					EntryPoint.CompiledFunctionName = FunctionEntry->GetGraph()->GetFName();
				}
				else
				{
					EntryPoint.CompiledFunctionName = NAME_None;
				}
			}
		}


		auto OnFunctionListGenerated = [WeakEventSection = MakeWeakObjectPtr(EventSection)](FKismetCompilerContext* CompilerContext)
		{
			UMovieSceneEventSectionBase* PinnedSection = WeakEventSection.Get();
			if (ensureMsgf(PinnedSection, TEXT("Event section has been collected during blueprint compilation.")))
			{
				PinnedSection->OnPostCompile(CompilerContext->Blueprint);
			}
		};

		CompilerContext->OnFunctionListCompiled().AddLambda(OnFunctionListGenerated);
	}

	// Don't remove while transacting because it can remove when an undo is in progress
	if (!GIsTransacting)
	{
		///// Temporarily disabled for UE-132130
		/////FMovieSceneEventUtils::RemoveUnusedCustomEvents(EventSections, CompilerContext->Blueprint);
	}
}
