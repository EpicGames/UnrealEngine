// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintExtension.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneEventBlueprintExtension.generated.h"

class FKismetCompilerContext;
class UBlueprint;
class UMovieSceneEventSectionBase;
class UObject;

UCLASS()
class UMovieSceneEventBlueprintExtension : public UBlueprintExtension
{
public:

	GENERATED_BODY()

	void Add(TWeakObjectPtr<UMovieSceneEventSectionBase> EventSection)
	{
		EventSections.AddUnique(EventSection);
	}

private:

	virtual void PostLoad() override final;
	virtual void HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint) override final;
	virtual void HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext) override final;

	/** List of event sections that are bound to the blueprint */
	UPROPERTY()
	TArray<TWeakObjectPtr<UMovieSceneEventSectionBase>> EventSections;
};