// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneEventCustomization.h"

#include "MovieScene.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "MovieSceneEventUtils.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceEditor.h"

#include "Channels/MovieSceneEvent.h"
#include "PropertyHandle.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "Tracks/MovieSceneEventTrack.h"

#include "K2Node_CustomEvent.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"

#define LOCTEXT_NAMESPACE "MovieSceneEventCustomization"

TSharedRef<IPropertyTypeCustomization> FMovieSceneEventCustomization::MakeInstance()
{
	return MakeShared<FMovieSceneEventCustomization>();
}

TSharedRef<IPropertyTypeCustomization> FMovieSceneEventCustomization::MakeInstance(UMovieSceneSection* InSection)
{
	TSharedRef<FMovieSceneEventCustomization> Custo = MakeShared<FMovieSceneEventCustomization>();
	Custo->WeakExternalSection = InSection;
	return Custo;
}

void FMovieSceneEventCustomization::GetEditObjects(TArray<UObject*>& OutObjects) const
{
	UMovieSceneEventSectionBase* ExternalSection = Cast<UMovieSceneEventSectionBase>(WeakExternalSection.Get());
	if (ExternalSection)
	{
		OutObjects.Add(ExternalSection);
	}
	else
	{
		GetPropertyHandle()->GetOuterObjects(OutObjects);
	}
}

void FMovieSceneEventCustomization::GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const
{
	const FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData);
	for (const TPair<FName, FMovieSceneEventPayloadVariable>& Pair : EntryPoint->PayloadVariables)
	{
		OutPayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
	}
}

bool FMovieSceneEventCustomization::SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewVariableValue)
{
	UMovieSceneEventSectionBase* EventSection = Cast<UMovieSceneEventSectionBase>(EditObject);
	FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData);
	if (!EventSection || !EntryPoint)
	{
		return false;
	}

	EventSection->Modify();

	FMovieSceneEventPayloadVariable* PayloadVariable = EntryPoint->PayloadVariables.Find(FieldName);
	if (!PayloadVariable)
	{
		PayloadVariable = &EntryPoint->PayloadVariables.Add(FieldName);
	}

	PayloadVariable->Value = NewVariableValue.Value;
	PayloadVariable->ObjectValue = NewVariableValue.ObjectValue;
	return true;
}

UK2Node* FMovieSceneEventCustomization::FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const
{
	UMovieSceneEventSectionBase* EventSection = Cast<UMovieSceneEventSectionBase>(EditObject);
	FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData);
	if (EventSection && EntryPoint)
	{
		return FMovieSceneEventUtils::FindEndpoint(EntryPoint, EventSection, Blueprint);
	}
	return nullptr;
}

void FMovieSceneEventCustomization::GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutWellKnownParameters) const
{
	// We only have one well-known parameter: the (optional) bound object parameter.
	FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData);
	OutWellKnownParameters.Add(EntryPoint->BoundObjectPinName);
}

void FMovieSceneEventCustomization::GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const
{
	FWellKnownParameterCandidates BoundObjectCandidates;
	BoundObjectCandidates.Metadata.PickerLabel = LOCTEXT("BoundObjectPin_Label", "Pass Bound Object To");
	BoundObjectCandidates.Metadata.PickerTooltip = LOCTEXT("BoundObjectPin_Tooltip", "Specifies a pin to pass the bound object(s) through when the event is triggered. Interface and object pins are both supported.");

	for (UEdGraphPin* Pin : Endpoint->Pins)
	{
		if (Pin->Direction == EGPD_Output && 
				(Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object || 
				 Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface))
		{
			BoundObjectCandidates.CandidatePinNames.Add(Pin->GetFName());
		}
	}
	
	OutCandidates.Add(BoundObjectCandidates);
}

bool FMovieSceneEventCustomization::SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName)
{
	if (ensure(ParameterIndex == 0))
	{
		FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData);
		EntryPoint->BoundObjectPinName = BoundPinName;
		return true;
	}
	return false;
}

FMovieSceneDirectorBlueprintEndpointDefinition FMovieSceneEventCustomization::GenerateEndpointDefinition(UMovieSceneSequence* Sequence)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	TOptional<FGuid> CommonObjectBindingID;
	{
		TArray<UObject*> EditObjects;
		GetEditObjects(EditObjects);

		for (UObject* Outer : EditObjects)
		{
			FGuid ThisBindingID;
			MovieScene->FindTrackBinding(*Outer->GetTypedOuter<UMovieSceneTrack>(), ThisBindingID);

			if (CommonObjectBindingID.IsSet() && CommonObjectBindingID != ThisBindingID)
			{
				CommonObjectBindingID.Reset();
				break;
			}

			CommonObjectBindingID = ThisBindingID;
		}
	}

	return FMovieSceneEventUtils::GenerateEventDefinition(MovieScene, CommonObjectBindingID.Get(FGuid()));
}

void FMovieSceneEventCustomization::OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	check(RawData.Num() == EditObjects.Num());

	struct FSectionData
	{
		TArray<FMovieSceneEvent*> EntryPoints;
	};

	TSortedMap<UMovieSceneEventSectionBase*, FSectionData> PerSectionData;

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		UMovieSceneEventSectionBase* EventSection = Cast<UMovieSceneEventSectionBase>(EditObjects[Index]);
		FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData[Index]);

		if (EventSection)
		{
			FSectionData& SectionData = PerSectionData.FindOrAdd(EventSection);
			SectionData.EntryPoints.Add(EntryPoint);
		}
	}

	UEdGraphPin* BoundObjectPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEndpoint, EndpointDefinition.PossibleCallTargetClass);

	for (const TPair<UMovieSceneEventSectionBase*, FSectionData>& SectionPair : PerSectionData)
	{
		SectionPair.Key->Modify();
		FMovieSceneEventUtils::BindEventSectionToBlueprint(SectionPair.Key, Blueprint);

		for (FMovieSceneEvent* EntryPoint : SectionPair.Value.EntryPoints)
		{
			FMovieSceneEventUtils::SetEndpoint(EntryPoint, SectionPair.Key, NewEndpoint, BoundObjectPin);
		}
	}
}

void FMovieSceneEventCustomization::OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint)
{
	check(EditObjects.Num() == RawData.Num());

	for (int32 Index = 0; Index < RawData.Num(); ++Index)
	{
		UMovieSceneEventSectionBase* EventSection = Cast<UMovieSceneEventSectionBase>(EditObjects[Index]);
		if (EventSection)
		{
			EventSection->Modify();
			if (Blueprint)
			{
				FMovieSceneEventUtils::BindEventSectionToBlueprint(EventSection, Blueprint);
			}
		}

		FMovieSceneEvent* EntryPoint = static_cast<FMovieSceneEvent*>(RawData[Index]);
		if (EntryPoint)
		{
			UEdGraphPin* CallTargetPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEndpoint, EndpointDefinition.PossibleCallTargetClass);
			FMovieSceneEventUtils::SetEndpoint(EntryPoint, EventSection, NewEndpoint, CallTargetPin);
		}
	}
}

#undef LOCTEXT_NAMESPACE
