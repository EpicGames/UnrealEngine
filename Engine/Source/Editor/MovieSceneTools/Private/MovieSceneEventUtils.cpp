// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneEventUtils.h"

#include "Channels/MovieSceneEvent.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneEventBlueprintExtension.h"
#include "MovieSceneFwd.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Notifications/SNotificationList.h"

class UBlueprintExtension;

#define LOCTEXT_NAMESPACE "MovieSceneEventUtils"

FMovieSceneDirectorBlueprintEndpointDefinition FMovieSceneEventUtils::GenerateEventDefinition(UMovieSceneTrack* Track)
{
	check(Track);

	UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();

	FGuid ObjectBindingID;
	if (MovieScene->FindTrackBinding(*Track, ObjectBindingID))
	{
		return GenerateEventDefinition(MovieScene, ObjectBindingID);
	}

	FMovieSceneDirectorBlueprintEndpointDefinition Definition;
	Definition.EndpointType = EMovieSceneDirectorBlueprintEndpointType::Event;
	Definition.EndpointName  = TEXT("SequenceEvent");
	Definition.GraphName = TEXT("Sequencer Events");

	return Definition;
}

FMovieSceneDirectorBlueprintEndpointDefinition FMovieSceneEventUtils::GenerateEventDefinition(UMovieScene* MovieScene, const FGuid& ObjectBindingID)
{
	check(MovieScene);

	FMovieSceneDirectorBlueprintEndpointDefinition Definition;
	Definition.EndpointType = EMovieSceneDirectorBlueprintEndpointType::Event;
	Definition.GraphName = TEXT("Sequencer Events");

	if (ObjectBindingID.IsValid())
	{
		FString BoundObjectName = MovieScene->GetObjectDisplayName(ObjectBindingID).ToString();

		UClass* BoundObjectClass = nullptr;
		if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID))
		{
			BoundObjectClass = const_cast<UClass*>(Possessable->GetPossessedObjectClass());
		}
		else if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectBindingID))
		{
			BoundObjectClass = Spawnable->GetObjectTemplate()->GetClass();
		}

		FMovieSceneDirectorBlueprintEndpointParameter BoundObjectParam;
		{
			BoundObjectParam.PinDirection = EGPD_Output;
			BoundObjectParam.PinTypeCategory = UEdGraphSchema_K2::PC_Object;
			BoundObjectParam.PinName = BoundObjectName;
			BoundObjectParam.PinTypeClass = BoundObjectClass;
			Definition.ExtraPins.Add(BoundObjectParam);
		}

		Definition.EndpointName = BoundObjectName + TEXT("_Event");
		Definition.PossibleCallTargetClass = BoundObjectClass;
	}
	else
	{
		Definition.EndpointName  = TEXT("SequenceEvent");
	}

	return Definition;
}

UK2Node_CustomEvent* FMovieSceneEventUtils::BindNewUserFacingEvent(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UBlueprint* Blueprint)
{
	check(EntryPoint && EventSection && Blueprint);

	UMovieSceneEventTrack* Track = EventSection->GetTypedOuter<UMovieSceneEventTrack>();

	// Modify necessary objects
	EventSection->Modify();
	Blueprint->Modify();

	// Ensure the section is bound to the blueprint function generation event
	FMovieSceneEventUtils::BindEventSectionToBlueprint(EventSection, Blueprint);

	// Create the new user-facing event node
	FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition = FMovieSceneEventUtils::GenerateEventDefinition(Track);
	UK2Node_CustomEvent* NewEventNode = FMovieSceneDirectorBlueprintUtils::CreateEventEndpoint(Blueprint, EndpointDefinition);

	if (NewEventNode)
	{
		// Bind the node to the event entry point
		UEdGraphPin* BoundObjectPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEventNode, EndpointDefinition.PossibleCallTargetClass);
		FMovieSceneEventUtils::SetEndpoint(EntryPoint, EventSection, NewEventNode, BoundObjectPin);
	}

	return NewEventNode;
}

UK2Node* FMovieSceneEventUtils::FindEndpoint(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UBlueprint* OwnerBlueprint)
{
	check(OwnerBlueprint);
	check(EntryPoint);

	if (EntryPoint->WeakEndpoint.IsStale())
	{
		return nullptr;
	}
	if (UK2Node* Node = Cast<UK2Node>(EntryPoint->WeakEndpoint.Get()))
	{
		return Node;
	}

	if (!EntryPoint->GraphGuid_DEPRECATED.IsValid())
	{
		return nullptr;
	}

	if (EntryPoint->NodeGuid_DEPRECATED.IsValid())
	{
		for (UEdGraph* Graph : OwnerBlueprint->UbergraphPages)
		{
			if (Graph->GraphGuid == EntryPoint->GraphGuid_DEPRECATED)
			{
				for (UEdGraphNode* Node : Graph->Nodes)
				{
					if (Node->NodeGuid == EntryPoint->NodeGuid_DEPRECATED)
					{
						UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Node);
						if (ensureMsgf(CustomEvent, TEXT("Encountered an event entry point node that is bound to something other than a custom event")))
						{
							CustomEvent->OnUserDefinedPinRenamed().AddUObject(EventSection, &UMovieSceneEventSectionBase::OnUserDefinedPinRenamed);
							EntryPoint->WeakEndpoint = CustomEvent;
							return CustomEvent;
						}
					}
				}
			}
		}
	}
	// If the node guid is invalid, this must be a function graph on the BP
	else for (UEdGraph* Graph : OwnerBlueprint->FunctionGraphs)
	{
		if (Graph->GraphGuid == EntryPoint->GraphGuid_DEPRECATED)
		{
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				if (UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
				{
					FunctionEntry->OnUserDefinedPinRenamed().AddUObject(EventSection, &UMovieSceneEventSectionBase::OnUserDefinedPinRenamed);
					EntryPoint->WeakEndpoint = FunctionEntry;
					return FunctionEntry;
				}
			}
		}
	}

	return nullptr;
}

void FMovieSceneEventUtils::SetEndpoint(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UK2Node* InNewEndpoint, UEdGraphPin* BoundObjectPin)
{
	check(EntryPoint);

	UK2Node* ExistingEndpoint = CastChecked<UK2Node>(EntryPoint->WeakEndpoint.Get(), ECastCheckedType::NullAllowed);
	if (ExistingEndpoint)
	{
		ExistingEndpoint->OnUserDefinedPinRenamed().RemoveAll(EventSection);
	}

	if (InNewEndpoint)
	{
		const bool bIsFunction    = InNewEndpoint->IsA<UK2Node_FunctionEntry>();
		const bool bIsCustomEvent = InNewEndpoint->IsA<UK2Node_CustomEvent>();

		checkf(bIsFunction || bIsCustomEvent, TEXT("Only functions and custom events are supported as event endpoints"));

		if (BoundObjectPin)
		{
			EntryPoint->BoundObjectPinName = BoundObjectPin->GetFName();
		}
		else
		{
			EntryPoint->BoundObjectPinName = NAME_None;
		}

		InNewEndpoint->OnUserDefinedPinRenamed().AddUObject(EventSection, &UMovieSceneEventSectionBase::OnUserDefinedPinRenamed);
		EntryPoint->WeakEndpoint = InNewEndpoint;
	}
	else
	{
		EntryPoint->WeakEndpoint = nullptr;
		EntryPoint->BoundObjectPinName = NAME_None;
	}
}

UK2Node_FunctionEntry* FMovieSceneEventUtils::GenerateEntryPoint(FMovieSceneEvent* EntrypointDefinition, FKismetCompilerContext* Compiler, UEdGraphNode* Endpoint)
{
	FMovieSceneDirectorBlueprintEndpointCall EndpointCall;
	for (const TPair<FName, FMovieSceneEventPayloadVariable>& Pair : EntrypointDefinition->PayloadVariables)
	{
		EndpointCall.PayloadVariables.Add(Pair.Key, FMovieSceneDirectorBlueprintVariableValue{ Pair.Value.ObjectValue, Pair.Value.Value });
	}
	if (!EntrypointDefinition->BoundObjectPinName.IsNone())
	{
		EndpointCall.ExposedPinNames.Add(EntrypointDefinition->BoundObjectPinName);
	}
	EndpointCall.Endpoint = Endpoint;
	FMovieSceneDirectorBlueprintEntrypointResult Result = FMovieSceneDirectorBlueprintUtils::GenerateEntryPoint(EndpointCall, Compiler);
	Result.CleanUpStalePayloadVariables(EntrypointDefinition->PayloadVariables);
	return Result.Entrypoint;
}

void FMovieSceneEventUtils::BindEventSectionToBlueprint(UMovieSceneEventSectionBase* EventSection, UBlueprint* DirectorBP)
{
	check(EventSection && DirectorBP);

	for (const TObjectPtr<UBlueprintExtension>& Extension : DirectorBP->GetExtensions())
	{
		UMovieSceneEventBlueprintExtension* EventExtension = Cast<UMovieSceneEventBlueprintExtension>(Extension);
		if (EventExtension)
		{
			EventExtension->Add(EventSection);
			return;
		}
	}

	UMovieSceneEventBlueprintExtension* EventExtension = NewObject<UMovieSceneEventBlueprintExtension>(DirectorBP);
	EventExtension->Add(EventSection);
	DirectorBP->AddExtension(EventExtension);
}

void FMovieSceneEventUtils::RemoveEndpointsForEventSection(UMovieSceneEventSectionBase* EventSection, UBlueprint* DirectorBP)
{
	check(EventSection && DirectorBP);

	static const TCHAR* const EventGraphName = TEXT("Sequencer Events");
	UEdGraph* SequenceEventGraph = FindObject<UEdGraph>(DirectorBP, EventGraphName);

	if (SequenceEventGraph)
	{
		for (FMovieSceneEvent& EntryPoint : EventSection->GetAllEntryPoints())
		{
			UEdGraphNode* Endpoint = FMovieSceneEventUtils::FindEndpoint(&EntryPoint, EventSection, DirectorBP);
			if (Endpoint)
			{
				UE_LOGF(LogMovieScene, Display, "Removing event: %ls from: %ls", *GetNameSafe(Endpoint), *GetNameSafe(DirectorBP));
				SequenceEventGraph->RemoveNode(Endpoint);
			}
		}
	}
}

void FMovieSceneEventUtils::ClearInvalidEndpointsForEventSection(UMovieSceneEventSectionBase* EventSection)
{
	check(EventSection);

	UMovieSceneSequence* Sequence = EventSection->GetTypedOuter<UMovieSceneSequence>();
	if (Sequence)
	{
		TArray<TTuple<FMovieSceneEvent*, UObject*, UMovieSceneSequence*>> InvalidEntryPoints;

		for (FMovieSceneEvent& EntryPoint : EventSection->GetAllEntryPoints())
		{
			TWeakObjectPtr<UObject> EndPoint = EntryPoint.WeakEndpoint;
			if (!EndPoint.IsValid())
			{
				continue;
			}

			UBlueprint* FunctionBlueprint = EndPoint->GetTypedOuter<UBlueprint>();
			if (!FunctionBlueprint)
			{
				continue;
			}

			UMovieSceneSequence* FunctionBlueprintSequence = Cast<UMovieSceneSequence>(FunctionBlueprint->GetOuter());
			if (FunctionBlueprintSequence != nullptr && FunctionBlueprintSequence != Sequence)
			{
				InvalidEntryPoints.Emplace(&EntryPoint, EndPoint.Get(), FunctionBlueprintSequence);
			}
		}

		if (InvalidEntryPoints.Num())
		{
			FScopedTransaction ClearInvalidEndpoints(LOCTEXT("ClearInvalidEndpoint", "Clear Invalid Endpoints"));

			for (TTuple<FMovieSceneEvent*, UObject*, UMovieSceneSequence*> InvalidEntryPoint : InvalidEntryPoints)
			{
				FMovieSceneEvent* EntryPoint = InvalidEntryPoint.Get<0>();
				UObject* EndPoint = InvalidEntryPoint.Get<1>();
				UMovieSceneSequence* FunctionBlueprintSequence = InvalidEntryPoint.Get<2>();

				FNotificationInfo Info(FText::Format(LOCTEXT("ClearInvalidEndpointNotify", "Clearing event {0} from {1} because it refers to a different sequence {2}"), FText::FromString(GetNameSafe(EndPoint)), FText::FromString(GetNameSafe(Sequence)), FText::FromString(GetNameSafe(FunctionBlueprintSequence))));
				Info.ExpireDuration = 3.f;
				FSlateNotificationManager::Get().AddNotification(Info);

				UE_LOGF(LogMovieScene, Display, "Clearing event %ls from %ls because it refers to a different sequence %ls", *GetNameSafe(EndPoint), *GetNameSafe(Sequence), *GetNameSafe(FunctionBlueprintSequence));

				// Clear this entry point
				*EntryPoint = FMovieSceneEvent();
			}
		}
	}
}

void FMovieSceneEventUtils::RemoveUnusedCustomEvents(const TArray<TWeakObjectPtr<UMovieSceneEventSectionBase>>& EventSections, UBlueprint* DirectorBP)
{
	check(DirectorBP);

	static const TCHAR* const EventGraphName = TEXT("Sequencer Events");
	UEdGraph* SequenceEventGraph = FindObject<UEdGraph>(DirectorBP, EventGraphName);
	if (SequenceEventGraph)
	{
		TArray<UK2Node_CustomEvent*> ExistingNodes;
		SequenceEventGraph->GetNodesOfClass(ExistingNodes);

		TSet<UEdGraphNode*> Endpoints;
		for (TWeakObjectPtr<UMovieSceneEventSectionBase> WeakEventSection : EventSections)
		{
			UMovieSceneEventSectionBase* EventSection = WeakEventSection.Get();
			if (!EventSection)
			{
				continue;
			}

			for (FMovieSceneEvent& EntryPoint : EventSection->GetAllEntryPoints())
			{
				if (UEdGraphNode* Endpoint = FMovieSceneEventUtils::FindEndpoint(&EntryPoint, EventSection, DirectorBP))
				{
					Endpoints.Add(Endpoint);
				}
			}
		}
				
		FScopedTransaction RemoveUnusedCustomEvents(LOCTEXT("RemoveUnusedCustomEvents", "Remove Unused Custom Events"));
	
		for (UK2Node_CustomEvent* ExistingNode : ExistingNodes)
		{
			const bool bHasEntryPoint = Endpoints.Contains(ExistingNode);
			if (!bHasEntryPoint)
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("RemoveUnusedCustomEventNotify", "Remove unused custom event {0} from {1}"), FText::FromString(GetNameSafe(ExistingNode)), FText::FromString(GetNameSafe(DirectorBP))));
				Info.ExpireDuration = 3.f;
				FSlateNotificationManager::Get().AddNotification(Info);
				
				UE_LOGF(LogMovieScene, Display, "Remove unused custom event %ls from %ls", *GetNameSafe(ExistingNode), *GetNameSafe(DirectorBP));

				DirectorBP->Modify();

				SequenceEventGraph->RemoveNode(ExistingNode);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
