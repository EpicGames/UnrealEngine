// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneObjectBindingIDCustomization.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "ISequencer.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "MovieSceneBindingOwnerInterface.h"
#include "PropertyEditorDelegates.h"
#include "PropertyHandle.h"
#include "SDropTarget.h"
#include "ScopedTransaction.h"
#include "SequencerObjectBindingDragDropOp.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h"
#include "EditorClassUtils.h"


struct FGeometry;

#define LOCTEXT_NAMESPACE "MovieSceneObjectBindingIDCustomization"

void FMovieSceneObjectBindingIDCustomization::BindTo(TSharedRef<ISequencer> OuterSequencer)
{
	OuterSequencer->OnInitializeDetailsPanel().AddStatic(
		[](TSharedRef<IDetailsView> DetailsView, TSharedRef<ISequencer> InSequencer)
		{
			TWeakPtr<ISequencer> WeakSequencer = InSequencer;

			FOnGetPropertyTypeCustomizationInstance BindingIDCustomizationFactory = FOnGetPropertyTypeCustomizationInstance::CreateLambda(
				[WeakSequencer]
				{
					if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
					{
						return MakeShared<FMovieSceneObjectBindingIDCustomization>(Sequencer->GetFocusedTemplateID(), Sequencer);
					}

					return MakeShared<FMovieSceneObjectBindingIDCustomization>();
				}
			);

			// Register an object binding ID customization that can use the current sequencer interface
			DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneObjectBindingID", BindingIDCustomizationFactory);
		}
	);
}

void FMovieSceneObjectBindingIDCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	using namespace UE::Sequencer;

	StructProperty = PropertyHandle;
	StructProperty->SetOnPropertyResetToDefault(
		FSimpleDelegate::CreateSP(this, &FMovieSceneObjectBindingIDCustomization::OnResetToDefault));

	const FString& MetaClassName = PropertyHandle->GetMetaData("MetaClass");
	const FString& MustImplementName = PropertyHandle->GetMetaData("MustImplement");

	AllowedClassFilters = PropertyCustomizationHelpers::GetClassesFromMetadataString(PropertyHandle->GetMetaData("AllowedClasses"));
	DisallowedClassFilters = PropertyCustomizationHelpers::GetClassesFromMetadataString(PropertyHandle->GetMetaData("DisallowedClasses"));

	ClassPropertyMetaClass = !MetaClassName.IsEmpty()
		? FEditorClassUtils::GetClassFromString(MetaClassName)
		: UObject::StaticClass();
	InterfaceThatMustBeImplemented = FEditorClassUtils::GetClassFromString(MustImplementName);

	Initialize();

	auto IsAcceptable = [](TSharedPtr<FDragDropOperation> Operation)
	{
		using namespace UE::Sequencer;
		if (!Operation->IsOfType<FSequencerObjectBindingDragDropOp>())
		{
			return false;
		}

		FSequencerObjectBindingDragDropOp* DragDropOp = static_cast<FSequencerObjectBindingDragDropOp*>(Operation.Get());
		return DragDropOp->GetDraggedRebindableBindings().Num() == 1;
	};

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SDropTarget)
			.OnDropped(this, &FMovieSceneObjectBindingIDCustomization::OnDrop)
			.OnAllowDrop_Static(IsAcceptable)
			.OnIsRecognized_Static(IsAcceptable)
			[
				SNew(SComboButton)
				.IsEnabled_Lambda([this]() { return StructProperty->IsEditable(); })
				.ToolTipText(this, &FMovieSceneObjectBindingIDCustomization::GetToolTipText)
				.OnGetMenuContent(this, &FMovieSceneObjectBindingIDCustomization::GetPickerMenu)
				.ContentPadding(FMargin(4.0, 2.0))
				.ButtonContent()
				[
					GetCurrentItemWidget(
						SNew(STextBlock)
						.Font(CustomizationUtils.GetRegularFont())
					)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
		[
			GetWarningWidget()
		]
	];
}

FReply FMovieSceneObjectBindingIDCustomization::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencerObjectBindingDragDropOp> SequencerOp = InDragDropEvent.GetOperationAs<FSequencerObjectBindingDragDropOp>();
	if (SequencerOp)
	{
		TArray<UE::MovieScene::FFixedObjectBindingID> Bindings = SequencerOp->GetDraggedRebindableBindings();
		if (Bindings.Num() == 1)
		{
			SetBindingId(Bindings[0]);
		}
	}

	return FReply::Handled();
}

UMovieSceneSequence* FMovieSceneObjectBindingIDCustomization::GetSequence() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() != 1)
	{
		return nullptr;
	}

	for ( UObject* NextOuter = OuterObjects[0]; NextOuter; NextOuter = NextOuter->GetOuter() )
	{
		if (IMovieSceneBindingOwnerInterface* Result = Cast<IMovieSceneBindingOwnerInterface>(NextOuter))
		{
			return Result->RetrieveOwnedSequence();
		}
	}
	return nullptr;
}

bool FMovieSceneObjectBindingIDCustomization::HasMultipleValues() const
{
	TArray<void*> Ptrs;
	StructProperty->AccessRawData(Ptrs);

	return Ptrs.Num() > 1;
}

FMovieSceneObjectBindingID FMovieSceneObjectBindingIDCustomization::GetCurrentValue() const
{
	TArray<void*> Ptrs;
	StructProperty->AccessRawData(Ptrs);

	if (Ptrs.Num() == 0 || Ptrs[0] == nullptr)
	{
		return FMovieSceneObjectBindingID();
	}

	FMovieSceneObjectBindingID Value = *static_cast<FMovieSceneObjectBindingID*>(Ptrs[0]);

	// If more than one value and not all equal, return empty
	for (int32 Index = 1; Index < Ptrs.Num(); ++Index)
	{
		if (Ptrs[Index] != nullptr && *static_cast<FMovieSceneObjectBindingID*>(Ptrs[Index]) != Value)
		{
			return FMovieSceneObjectBindingID();
		}
	}

	return Value;
}

void FMovieSceneObjectBindingIDCustomization::SetCurrentValue(const FMovieSceneObjectBindingID& InObjectBinding)
{
	FScopedTransaction Transaction(LOCTEXT("SetBinding", "Set Binding"));

	StructProperty->NotifyPreChange();

	TArray<UObject*> Objects;
	StructProperty->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		Object->Modify();
	}

	TArray<void*> Ptrs;
	StructProperty->AccessRawData(Ptrs);

	for (int32 Index = 0; Index < Ptrs.Num(); ++Index)
	{
		*static_cast<FMovieSceneObjectBindingID*>(Ptrs[Index]) = InObjectBinding;
	}
	
	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();
}

void FMovieSceneObjectBindingIDCustomization::OnResetToDefault()
{
	StructProperty->RequestRebuildChildren();
}

#undef LOCTEXT_NAMESPACE
