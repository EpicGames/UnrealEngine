// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneObjectBindingIDPicker.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequence.h"
#include "SequenceBindingTree.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorClassUtils.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"

struct FMovieSceneSequenceHierarchy;

#define LOCTEXT_NAMESPACE "MovieSceneObjectBindingIDPicker"

bool FMovieSceneObjectBindingIDPicker::IsEmpty() const
{
	return !DataTree.IsValid() || DataTree->IsEmpty();
}

void FMovieSceneObjectBindingIDPicker::Initialize()
{
	if (!DataTree.IsValid())
	{
		DataTree = MakeShared<FSequenceBindingTree>();
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : GetSequence();
	UMovieSceneSequence* ActiveSequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : GetSequence();
	FMovieSceneSequenceID ActiveSequenceID = Sequencer.IsValid() ? Sequencer->GetFocusedTemplateID() : MovieSceneSequenceID::Root;

	DataTree->ConditionalRebuild(Sequence, ActiveSequence, ActiveSequenceID);

	UpdateCachedData();
}

void FMovieSceneObjectBindingIDPicker::OnGetMenuContent(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceBindingNode> Node)
{
	check(Node.IsValid());

	bool bHadAnyEntries = false;

	if (Node->BindingID.Guid.IsValid() && IsNodeAllowed(Node))
	{
		bHadAnyEntries = true;
		MenuBuilder.AddMenuEntry(
			Node->DisplayString,
			FText(),
			Node->Icon,
			FUIAction(
				FExecuteAction::CreateRaw(this, &FMovieSceneObjectBindingIDPicker::SetBindingId, Node->BindingID)
			)
		);
	}

	for (const TSharedPtr<FSequenceBindingNode> Child : Node->Children)
	{
		check(Child.IsValid())

		if (!Child->BindingID.Guid.IsValid())
		{
			if (Child->Children.Num())
			{
				bHadAnyEntries = true;
				MenuBuilder.AddSubMenu(
					Child->DisplayString,
					FText(),
					FNewMenuDelegate::CreateRaw(this, &FMovieSceneObjectBindingIDPicker::OnGetMenuContent, Child),
					false,
					Child->Icon,
					false
					);
			}
		}
		else if (IsNodeAllowed(Child))
		{
			bHadAnyEntries = true;
			MenuBuilder.AddMenuEntry(
				Child->DisplayString,
				FText(),
				Child->Icon,
				FUIAction(
					FExecuteAction::CreateRaw(this, &FMovieSceneObjectBindingIDPicker::SetBindingId, Child->BindingID)
				)
			);
		}
	}

	if (!bHadAnyEntries)
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("NoEntries", "No Object Bindings"), FText(), FSlateIcon(), FUIAction());
	}
}

TSharedRef<SWidget> FMovieSceneObjectBindingIDPicker::GetPickerMenu()
{
	// The menu are generated through reflection and sometime the API exposes some recursivity (think about a Widget returning it parent which is also a Widget). Just by reflection
	// it is not possible to determine when the root object is reached. It needs a kind of simulation which is not implemented. Also, even if the recursivity was correctly handled, the possible
	// permutations tend to grow exponentially. Until a clever solution is found, the simple approach is to disable recursively searching those menus. User can still search the current one though.
	// See UE-131257
	const bool bInRecursivelySearchable = false;

	// Close self only to enable use inside context menus
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true, &FCoreStyle::Get(), true, NAME_None, bInRecursivelySearchable);

	Initialize();
	GetPickerMenu(MenuBuilder);

	// Hold onto the menu widget so we can destroy it manually
	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	DismissWidget = MenuWidget;
	return MenuWidget;
}

void FMovieSceneObjectBindingIDPicker::GetPickerMenu(FMenuBuilder& MenuBuilder)
{
	OnGetMenuContent(MenuBuilder, DataTree->GetRootNode());
}

TSharedRef<SWidget> FMovieSceneObjectBindingIDPicker::GetCurrentItemWidget(TSharedRef<STextBlock> TextContent)
{
	TextContent->SetText(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FMovieSceneObjectBindingIDPicker::GetCurrentText)));
	
	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image_Raw(this, &FMovieSceneObjectBindingIDPicker::GetCurrentIconBrush)
			]

			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			[
				SNew(SImage)
				.Visibility_Raw(this, &FMovieSceneObjectBindingIDPicker::GetSpawnableIconOverlayVisibility)
				.Image(FAppStyle::GetBrush("Sequencer.SpawnableIconOverlay"))
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(4.f,0,0,0)
		.VAlign(VAlign_Center)
		[
			TextContent
		];
}

TSharedRef<SWidget> FMovieSceneObjectBindingIDPicker::GetWarningWidget()
{
	return SNew(SButton)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ContentPadding(FMargin(0))
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("FixedBindingWarningText", "This binding is fixed to the current Root Sequence hierarchy, so will break if evaluated in a different hierarchy.\nClick here to fix this problem."))
		.Visibility_Raw(this, &FMovieSceneObjectBindingIDPicker::GetFixedWarningVisibility)
		.OnClicked_Raw(this, &FMovieSceneObjectBindingIDPicker::AttemptBindingFixup)
		[
			SNew(SImage)
			.Image(FStarshipCoreStyle::GetCoreStyle().GetBrush("Icons.Warning"))
		];
}

EVisibility FMovieSceneObjectBindingIDPicker::GetFixedWarningVisibility() const
{
	FMovieSceneObjectBindingID CurrentValue = GetCurrentValue();

	const bool bShowError = CurrentValue.IsFixedBinding() && LocalSequenceID != MovieSceneSequenceID::Invalid;
	return bShowError ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FMovieSceneObjectBindingIDPicker::AttemptBindingFixup()
{
	SetCurrentValueFromFixed(GetCurrentValueAsFixed());
	return FReply::Handled();
}

bool FMovieSceneObjectBindingIDPicker::IsNodeAllowed(const TSharedPtr<FSequenceBindingNode>& Node) const
{
	return IsClassAllowed(Node);
}

bool FMovieSceneObjectBindingIDPicker::IsClassAllowed(const TSharedPtr<FSequenceBindingNode>& Node) const
{
	const UClass* InClass = nullptr;
	const UE::MovieScene::FFixedObjectBindingID& BindingID = Node->BindingID;

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence* BindSequence = Sequencer.IsValid() ? Sequencer->GetEvaluationState()->FindSequence(BindingID.SequenceID) : nullptr;
	UMovieScene* BindMovieScene = BindSequence ? BindSequence->GetMovieScene() : nullptr;

	InClass = MovieSceneHelpers::GetBoundObjectClass(BindSequence, BindingID.Guid);

	if (InClass == nullptr || InClass == UObject::StaticClass())
	{
		return true;
	}

	// Abstract types and deprecations are excluded
	bool bMatchesFlags = !InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated);

	if (bMatchesFlags && InClass->IsChildOf(ClassPropertyMetaClass)
		&& (!InterfaceThatMustBeImplemented || InClass->ImplementsInterface(InterfaceThatMustBeImplemented)))
	{
		auto PredicateFn = [InClass](const UClass* Class)
		{
			return InClass->IsChildOf(Class);
		};

		if (DisallowedClassFilters.FindByPredicate(PredicateFn) == nullptr &&
			(AllowedClassFilters.Num() == 0 || AllowedClassFilters.FindByPredicate(PredicateFn) != nullptr))
		{
			return true;
		}
	}

	return false;

}

void FMovieSceneObjectBindingIDPicker::SetBindingId(UE::MovieScene::FFixedObjectBindingID InBindingId)
{
	SetCurrentValueFromFixed(InBindingId);
	UpdateCachedData();

	TSharedPtr<SWidget> MenuWidget = DismissWidget.Pin();
	if (MenuWidget.IsValid())
	{
		FSlateApplication::Get().DismissMenuByWidget(MenuWidget.ToSharedRef());
	}
}

void FMovieSceneObjectBindingIDPicker::UpdateCachedData()
{
	using namespace UE::MovieScene;

	FFixedObjectBindingID CurrentValue = GetCurrentValueAsFixed();

	TSharedPtr<FSequenceBindingNode> Object = CurrentValue.Guid.IsValid() ? DataTree->FindNode(CurrentValue) : nullptr;

	if (!Object.IsValid())
	{
		CurrentIcon = FSlateIcon();
		bIsCurrentItemSpawnable = false;
		
		if (HasMultipleValues())
		{
			CurrentText = LOCTEXT("MultipleValues", "Multiple Values");
			ToolTipText = LOCTEXT("MultipleValues_ToolTip", "The specified binding has multiple values");
			return;
		}
		else
		{
			CurrentText = LOCTEXT("UnresolvedBinding", "Unresolved Binding");
			ToolTipText = LOCTEXT("UnresolvedBinding_ToolTip", "The specified binding could not be located in the sequence");
		}
	}
	else
	{
		CurrentText = Object->DisplayString;
		CurrentIcon = Object->Icon;
		bIsCurrentItemSpawnable = Object->bIsSpawnable;

		ToolTipText = FText();
		while (Object.IsValid() && Object->BindingID.SequenceID != MovieSceneSequenceID::Invalid)
		{
			ToolTipText = ToolTipText.IsEmpty() ? Object->DisplayString : FText::Format(LOCTEXT("ToolTipFormat", "{0} -> {1}"), Object->DisplayString, ToolTipText);
			Object = DataTree->FindNode(Object->ParentID);
		}
	}
}

FText FMovieSceneObjectBindingIDPicker::GetToolTipText() const
{
	return ToolTipText;
}

FText FMovieSceneObjectBindingIDPicker::GetCurrentText() const
{
	return CurrentText;
}

FSlateIcon FMovieSceneObjectBindingIDPicker::GetCurrentIcon() const
{
	return CurrentIcon;
}

const FSlateBrush* FMovieSceneObjectBindingIDPicker::GetCurrentIconBrush() const
{
	return CurrentIcon.GetOptionalIcon();
}

EVisibility FMovieSceneObjectBindingIDPicker::GetSpawnableIconOverlayVisibility() const
{
	return bIsCurrentItemSpawnable ? EVisibility::Visible : EVisibility::Collapsed;
}

UE::MovieScene::FFixedObjectBindingID FMovieSceneObjectBindingIDPicker::GetCurrentValueAsFixed() const
{
	FMovieSceneObjectBindingID ID = GetCurrentValue();

	// If the ID is in local space, remap it to the root space as according to the LocalSequenceID we were created with
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		ID = ID.ResolveToFixed(LocalSequenceID, *Sequencer);
	}

	return ID.ReinterpretAsFixed();
}

void FMovieSceneObjectBindingIDPicker::SetCurrentValueFromFixed(UE::MovieScene::FFixedObjectBindingID InValue)
{
	TSharedPtr<ISequencer>              Sequencer = WeakSequencer.Pin();
	const FMovieSceneSequenceHierarchy* Hierarchy = Sequencer.IsValid() ? Sequencer->GetEvaluationTemplate().GetHierarchy() : nullptr;

	// If there is no sequencer, use a relative binding since there should be no contexts where this is different from a fixed binding
	//    but relative bindings will support cases where this needs to be resolved within the context of a different subsequence (ie, if Sequence A
	//    has a GetSequenceBinding node for itself inside its director BP, and that sequence is added to another sequence).
	if (!Sequencer.IsValid())
	{
		SetCurrentValue(UE::MovieScene::FRelativeObjectBindingID(InValue.Guid, InValue.SequenceID));
	}
	// If we don't know the local sequence ID, or we have no hierarchy, or we're resetting the binding; set a relative ID
	else if (LocalSequenceID == MovieSceneSequenceID::Invalid || !InValue.Guid.IsValid() || Hierarchy == nullptr)
	{
		SetCurrentValue(UE::MovieScene::FRelativeObjectBindingID(InValue.Guid));
	}
	else
	{
		// Attempt to remap the desired binding to the current local sequence by either making it local to this sequence
		// or specifying a parent index so that this binding is still able to resolve correctly if the root sequence is added
		// as a subsequence elsewhere
		// This ensures that you can work on sub sequences on their own, or within a root sequence and the binding will resolve correctly.
		SetCurrentValue(InValue.ConvertToRelative(LocalSequenceID, Hierarchy));
	}
}

#undef LOCTEXT_NAMESPACE
