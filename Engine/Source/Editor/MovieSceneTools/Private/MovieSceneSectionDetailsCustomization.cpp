// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSectionDetailsCustomization.h"
#include "IDetailPropertyRow.h"
#include "Misc/FrameNumber.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "PropertyCustomizationHelpers.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFrameMigration.h"
#include "FrameNumberDetailsCustomization.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Math/RangeBound.h"
#include "Widgets/Input/SButton.h"
#include "EditorFontGlyphs.h"
#include "MovieSceneSection.h"
#include "DetailCategoryBuilder.h"
#include "MovieScene.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "MovieSceneTools"

void FMovieSceneSectionDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Determine if we should show the section toggle buttons
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	bool bSectionCanHaveOpenLowerBound = true;
	bool bSectionCanHaveOpenUpperBound = true;

	for (TWeakObjectPtr<UObject> Object : Objects)
	{
		if (Object.IsValid() && Object->IsA(UMovieSceneSection::StaticClass()))
		{
			UMovieSceneSection* MovieSceneSection = (UMovieSceneSection*)Object.Get();
			if (!MovieSceneSection->GetSupportsInfiniteRange())
			{
				bSectionCanHaveOpenLowerBound = false;
				bSectionCanHaveOpenUpperBound = false;
				break;
			}
			else 
			{
				if (!MovieSceneSection->CanHaveOpenLowerBound())
				{
					bSectionCanHaveOpenLowerBound = false;
				}
				if (!MovieSceneSection->CanHaveOpenUpperBound())
				{
					bSectionCanHaveOpenUpperBound = false;
				}
			}
		}
	}

	MovieSceneSectionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneSection, SectionRange));
	MovieSceneSectionPropertyHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& SectionCategory = DetailBuilder.EditCategory("Section");
	SectionCategory.AddCustomRow(LOCTEXT("StartTimeLabel", "Start Section Time"))
	.NameContent()
	[
	 	SNew(STextBlock)
	 	.Text(LOCTEXT("SectionRangeStart", "Section Range Start"))
	 	.ToolTipText(LOCTEXT("SectionRangeTooltip", "You can specify the bounds of the section for non-infinite bounds."))
		.Font(DetailBuilder.GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Fill)
		[
			SNew(SEditableTextBox)
			.Text(this, &FMovieSceneSectionDetailsCustomization::OnGetRangeStartText)
			.ToolTipText(this, &FMovieSceneSectionDetailsCustomization::OnGetRangeStartToolTipText)
			.OnTextCommitted(this, &FMovieSceneSectionDetailsCustomization::OnRangeStartTextCommitted)
			.IsEnabled(this, &FMovieSceneSectionDetailsCustomization::IsRangeStartTextboxEnabled)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.ClearKeyboardFocusOnCommit(false)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1)
		[
			SNew(SButton)
			.Visibility_Lambda([bSectionCanHaveOpenLowerBound]() -> EVisibility {
				return bSectionCanHaveOpenLowerBound ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.OnClicked(this, &FMovieSceneSectionDetailsCustomization::ToggleRangeStartBounded)
			.ContentPadding(0)
			.ToolTipText(LOCTEXT("LockedRangeBounds", "Some sections support infinite ranges and fixed ranges. Toggling this will change the bound type."))
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &FMovieSceneSectionDetailsCustomization::GetRangeStartButtonIcon)
			]
		]
	];

	SectionCategory.AddCustomRow(LOCTEXT("EndTimeLabel", "End Section Time"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("SectionRangeEnd", "Section Range End"))
		.ToolTipText(LOCTEXT("SectionRangeTooltip", "You can specify the bounds of the section for non-infinite bounds."))
		.Font(DetailBuilder.GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Fill)
		[
			SNew(SEditableTextBox)
			.Text(this, &FMovieSceneSectionDetailsCustomization::OnGetRangeEndText)
			.ToolTipText(this, &FMovieSceneSectionDetailsCustomization::OnGetRangeEndToolTipText)
			.OnTextCommitted(this, &FMovieSceneSectionDetailsCustomization::OnRangeEndTextCommitted)
			.IsEnabled(this, &FMovieSceneSectionDetailsCustomization::IsRangeEndTextboxEnabled)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.ClearKeyboardFocusOnCommit(false)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1)
		[
			SNew(SButton)
			.Visibility_Lambda([bSectionCanHaveOpenUpperBound]() -> EVisibility {
				return bSectionCanHaveOpenUpperBound ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.OnClicked(this, &FMovieSceneSectionDetailsCustomization::ToggleRangeEndBounded)
			.ContentPadding(0)
			.ToolTipText(LOCTEXT("LockedRangeBounds", "Some sections support infinite ranges and fixed ranges. Toggling this will change the bound type."))
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &FMovieSceneSectionDetailsCustomization::GetRangeEndButtonIcon)
			]
		]
	];
}

/** Get the range start value, or return false if multiple values differ */
FMovieSceneSectionDetailsCustomization::ERangeBoundValueType FMovieSceneSectionDetailsCustomization::GetRangeStartValue(FFrameNumber& OutValue) const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* CurrentMovieSceneRange = (FMovieSceneFrameRange*)RawData[i];
		if (CurrentMovieSceneRange)
		{
			TRange<FFrameNumber> CurrentFrameRange = CurrentMovieSceneRange->Value;

			// Unbounded ranges have no value.
			if (CurrentFrameRange.GetLowerBound().IsOpen())
			{
				return ERangeBoundValueType::Infinite;
			}

			if (i > 0)
			{
				if (CurrentFrameRange.GetLowerBoundValue().Value != OutValue)
				{
					// No need to check the rest of the selected items once we've determined one of them is different.
					return ERangeBoundValueType::MultipleValues;
				}
			}
			else
			{
				// If this is the first one we're looking at we just assign this as our value.
				OutValue = CurrentFrameRange.GetLowerBoundValue().Value;
			}
		}
	}

	return ERangeBoundValueType::Finite;
}

/** Convert the range start into an FText for display */
FText FMovieSceneSectionDetailsCustomization::OnGetRangeStartText() const
{
	FFrameNumber FrameValue;
	ERangeBoundValueType ValueType = GetRangeStartValue(FrameValue);
	switch (ValueType)
	{
		case ERangeBoundValueType::Infinite:
			return FText::GetEmpty();
		case ERangeBoundValueType::MultipleValues:
			return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
		case ERangeBoundValueType::Finite:
		default:
			return FText::FromString(NumericTypeInterface->ToString((double)FrameValue.Value));
	}
}

FText FMovieSceneSectionDetailsCustomization::OnGetRangeStartToolTipText() const
{
	FFrameNumber FrameValue;
	ERangeBoundValueType ValueType = GetRangeStartValue(FrameValue);
	switch (ValueType)
	{
		case ERangeBoundValueType::Infinite:
			return LOCTEXT("InfiniteBound", "Infinite");
		case ERangeBoundValueType::MultipleValues:
			return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
		case ERangeBoundValueType::Finite:
		default:
			return FText::Format(LOCTEXT("FrameTicks", "{0} ticks"), FrameValue.Value);
	}
}

/** Convert the text into a new range start */
void FMovieSceneSectionDetailsCustomization::OnRangeStartTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	// Find the new value for the start range.
	TOptional<double> NewStart = NumericTypeInterface->FromString(InText.ToString(), 0.0);

	// Early out if we couldn't parse it, no need to reset them all to zero.
	if (!NewStart.IsSet())
	{
		return;
	}

	GEditor->BeginTransaction(FText::Format(LOCTEXT("EditProperty", "Edit {0}"), MovieSceneSectionPropertyHandle->GetPropertyDisplayName()));

	MovieSceneSectionPropertyHandle->NotifyPreChange();

	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange && !MovieSceneFrameRange->Value.GetLowerBound().IsOpen())
		{
			MovieSceneFrameRange->Value.SetLowerBoundValue(FFrameTime::FromDecimal(NewStart.GetValue()).RoundToFrame());
		}
	}

	MovieSceneSectionPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	MovieSceneSectionPropertyHandle->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();
}

/** Should the textbox be editable? False if we have an infinite range.  */
bool FMovieSceneSectionDetailsCustomization::IsRangeStartTextboxEnabled() const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange)
		{
			if (MovieSceneFrameRange->GetLowerBound().IsOpen())
			{
				return false;
			}
		}
	}

	return true;
}

/** Determines if the range is Open, Closed, or Undetermined which can happen in the case of multi-select.  */
ECheckBoxState FMovieSceneSectionDetailsCustomization::GetRangeStartBoundedState() const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	if (RawData.Num() > 0)
	{
		ECheckBoxState OutState = ((FMovieSceneFrameRange*)RawData[0])->Value.GetLowerBound().IsOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		for (int32 i = 1; i < RawData.Num(); i++)
		{
			FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
			if (MovieSceneFrameRange)
			{
				// If we're multi-selecting and their values don't match then we're undetermined.
				ECheckBoxState NewState = MovieSceneFrameRange->GetLowerBound().IsOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				if (OutState != NewState)
				{
					return ECheckBoxState::Undetermined;
				}
			}
		}

		return OutState;
	}

	return ECheckBoxState::Undetermined;
}

/** Get the FText representing the appropriate Unicode icon for the toggle button. */
FText FMovieSceneSectionDetailsCustomization::GetRangeStartButtonIcon() const
{
	ECheckBoxState State = GetRangeStartBoundedState();
	switch (State)
	{
	case ECheckBoxState::Checked:
		return  FEditorFontGlyphs::Lock;
	case ECheckBoxState::Unchecked:
		return  FEditorFontGlyphs::Unlock;
	case ECheckBoxState::Undetermined:
	default:
		return  FEditorFontGlyphs::Bars;
	}
}

/** Called when the button is pressed to toggle the current state. */
FReply FMovieSceneSectionDetailsCustomization::ToggleRangeStartBounded()
{
	FScopedTransaction Transaction(LOCTEXT("ToggleRangeStartBounded", "Toggle Range Start Bounded"));
	TArray<UObject*> Objects;
	MovieSceneSectionPropertyHandle->GetOuterObjects(Objects);

	for (auto Outer : Objects)
	{
		for (UObject* Obj : Objects)
		{
			Obj->Modify();
		}
	}

	if (IsRangeStartTextboxEnabled())
	{
		SetRangeStartBounded(false);
	}
	else
	{
		SetRangeStartBounded(true);
	}

	return FReply::Handled();
}

/** Sets the range to have a fixed bound or convert to an open bound. */
void FMovieSceneSectionDetailsCustomization::SetRangeStartBounded(bool InbIsBounded)
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange)
		{
			TRange<FFrameNumber> CurrentRange = MovieSceneFrameRange->Value;

			if (InbIsBounded)
			{
				// We'll try to use our parent's working range to determine our new value. This helps us avoid always making the new bounds on frame 0/1, which might be off-screen for many use cases.
				int32 NewFrameNumber = 0;
				if (ParentMovieScene.IsValid() && !ParentMovieScene->GetPlaybackRange().GetLowerBound().IsOpen())
				{
					NewFrameNumber = ParentMovieScene->GetPlaybackRange().GetLowerBoundValue().Value;
				}

				MovieSceneFrameRange->Value.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(FFrameNumber(NewFrameNumber)));
			}
			else
			{
				// We're replacing a closed bound with an open one, we unfortunately wipe out the old value they had.
				MovieSceneFrameRange->Value.SetLowerBound(TRangeBound<FFrameNumber>());
			}
		}
	}
}

/** Get the range start value, or return false if multiple values differ */
FMovieSceneSectionDetailsCustomization::ERangeBoundValueType FMovieSceneSectionDetailsCustomization::GetRangeEndValue(FFrameNumber& OutValue) const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* CurrentMovieSceneRange = (FMovieSceneFrameRange*)RawData[i];
		if (CurrentMovieSceneRange)
		{
			TRange<FFrameNumber> CurrentFrameRange = CurrentMovieSceneRange->Value;

			// Unbounded ranges have no value.
			if (CurrentFrameRange.GetUpperBound().IsOpen())
			{
				return ERangeBoundValueType::Infinite;
			}

			if (i > 0)
			{
				if (CurrentFrameRange.GetUpperBoundValue().Value != OutValue)
				{
					// No need to check the rest of the selected items once we've determined one of them is different.
					return ERangeBoundValueType::MultipleValues;
				}
			}
			else
			{
				// If this is the first one we're looking at we just assign this as our value.
				OutValue = CurrentFrameRange.GetUpperBoundValue().Value;
			}
		}
	}

	return ERangeBoundValueType::Finite;
}

/** Convert the range end into an FText for display */
FText FMovieSceneSectionDetailsCustomization::OnGetRangeEndText() const
{
	FFrameNumber FrameValue;
	ERangeBoundValueType ValueType = GetRangeEndValue(FrameValue);
	switch (ValueType)
	{
		case ERangeBoundValueType::Infinite:
			return FText::GetEmpty();
		case ERangeBoundValueType::MultipleValues:
			return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
		case ERangeBoundValueType::Finite:
		default:
			return FText::FromString(NumericTypeInterface->ToString((double)FrameValue.Value));
	}
}

FText FMovieSceneSectionDetailsCustomization::OnGetRangeEndToolTipText() const
{
	FFrameNumber FrameValue;
	ERangeBoundValueType ValueType = GetRangeEndValue(FrameValue);
	switch (ValueType)
	{
		case ERangeBoundValueType::Infinite:
			return LOCTEXT("InfiniteBound", "Infinite");
		case ERangeBoundValueType::MultipleValues:
			return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
		case ERangeBoundValueType::Finite:
		default:
			return FText::Format(LOCTEXT("FrameTicks", "{0} ticks"), FrameValue.Value);
	}
}

/** Convert the text into a new range end */
void FMovieSceneSectionDetailsCustomization::OnRangeEndTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	// Find the new value for the end range.
	TOptional<double> NewEnd = NumericTypeInterface->FromString(InText.ToString(), 0.0);

	// Early out if we couldn't parse it, no need to reset them all to zero.
	if (!NewEnd.IsSet())
	{
		return;
	}

	GEditor->BeginTransaction(FText::Format(LOCTEXT("EditProperty", "Edit {0}"), MovieSceneSectionPropertyHandle->GetPropertyDisplayName()));

	MovieSceneSectionPropertyHandle->NotifyPreChange();

	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange && !MovieSceneFrameRange->Value.GetUpperBound().IsOpen())
		{
			MovieSceneFrameRange->Value.SetUpperBoundValue(FFrameTime::FromDecimal(NewEnd.GetValue()).RoundToFrame());
		}
	}

	MovieSceneSectionPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	MovieSceneSectionPropertyHandle->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();
}

/** Should the textbox be editable? False if we have an infinite range.  */
bool FMovieSceneSectionDetailsCustomization::IsRangeEndTextboxEnabled() const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange)
		{
			if (MovieSceneFrameRange->GetUpperBound().IsOpen())
			{
				return false;
			}
		}
	}

	return true;
}

/** Determines if the range is Open, Closed, or Undetermined which can happen in the case of multi-select.  */
ECheckBoxState FMovieSceneSectionDetailsCustomization::GetRangeEndBoundedState() const
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	if (RawData.Num() > 0)
	{
		ECheckBoxState OutState = ((FMovieSceneFrameRange*)RawData[0])->Value.GetUpperBound().IsOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		for (int32 i = 1; i < RawData.Num(); i++)
		{
			FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
			if (MovieSceneFrameRange)
			{
				// If we're multi-selecting and their values don't match then we're undetermined.
				ECheckBoxState NewState = MovieSceneFrameRange->GetUpperBound().IsOpen() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				if (OutState != NewState)
				{
					return ECheckBoxState::Undetermined;
				}
			}
		}

		return OutState;
	}

	return ECheckBoxState::Undetermined;
}

/** Get the FText representing the appropriate Unicode icon for the toggle button. */
FText FMovieSceneSectionDetailsCustomization::GetRangeEndButtonIcon() const
{
	ECheckBoxState State = GetRangeEndBoundedState();
	switch (State)
	{
	case ECheckBoxState::Checked:
		return  FEditorFontGlyphs::Lock;
	case ECheckBoxState::Unchecked:
		return  FEditorFontGlyphs::Unlock;
	case ECheckBoxState::Undetermined:
	default:
		return  FEditorFontGlyphs::Bars;
	}
}

/** Called when the button is pressed to toggle the current state. */
FReply FMovieSceneSectionDetailsCustomization::ToggleRangeEndBounded()
{
	FScopedTransaction Transaction(LOCTEXT("ToggleRangeEndBounded", "Toggle Range End Bounded"));
	TArray<UObject*> Objects;
	MovieSceneSectionPropertyHandle->GetOuterObjects(Objects);

	for (auto Outer : Objects)
	{
		for (UObject* Obj : Objects)
		{
			Obj->Modify();
		}
	}

	if (IsRangeEndTextboxEnabled())
	{
		SetRangeEndBounded(false);
	}
	else
	{
		SetRangeEndBounded(true);
	}

	return FReply::Handled();
}

/** Sets the range to have a fixed bound or convert to an open bound. */
void FMovieSceneSectionDetailsCustomization::SetRangeEndBounded(bool InbIsBounded)
{
	TArray<void*> RawData;
	MovieSceneSectionPropertyHandle->AccessRawData(RawData);

	for (int32 i = 0; i < RawData.Num(); i++)
	{
		FMovieSceneFrameRange* MovieSceneFrameRange = (FMovieSceneFrameRange*)RawData[i];
		if (MovieSceneFrameRange)
		{
			TRange<FFrameNumber> CurrentRange = MovieSceneFrameRange->Value;

			if (InbIsBounded)
			{
				// We'll try to use our parent's working range to determine our new value. This helps us avoid always making the new bounds on frame 0/1, which might be off-screen for many use cases.
				int32 NewFrameNumber = 1;
				if (ParentMovieScene.IsValid() && !ParentMovieScene->GetPlaybackRange().GetUpperBound().IsOpen())
				{
					NewFrameNumber = ParentMovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
				}
				MovieSceneFrameRange->Value.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(FFrameNumber(NewFrameNumber)));
			}
			else
			{
				// We're replacing a closed bound with an open one, we unfortunately wipe out the old value they had.
				MovieSceneFrameRange->Value.SetUpperBound(TRangeBound<FFrameNumber>());
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE
