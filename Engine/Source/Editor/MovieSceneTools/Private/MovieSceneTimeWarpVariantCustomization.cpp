// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTimeWarpVariantCustomization.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "MovieSceneSequence.h"
#include "SequencerUtilities.h"

#include "ScopedTransaction.h"

#include "UObject/Class.h"
#include "Templates/SubclassOf.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "MovieSceneTimeWarpVariantCustomization"

namespace UE::MovieScene
{

void FMovieSceneTimeWarpVariantCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;

	auto DeduceType = [this](const void* RawData, const int32, const int32) -> bool
	{
		const FMovieSceneTimeWarpVariant* Variant = static_cast<const FMovieSceneTimeWarpVariant*>(RawData);

		if (Variant->GetType() == EMovieSceneTimeWarpType::Custom)
		{
			this->bIsFixed = false;

			UMovieSceneTimeWarpGetter* Getter = Variant->AsCustom();
			if (!Getter)
			{
				this->Class.Reset();
				return false;
			}
			else if (Class && Getter->GetClass() != Class.GetValue())
			{
				this->Class.Reset();
				return false;
			}
			else
			{
				this->Class = Getter->GetClass();
			}
		}

		return true;
	};

	StructPropertyHandle->EnumerateConstRawData(DeduceType);

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	.MinDesiredWidth(TOptional<float>())
	.MaxDesiredWidth(TOptional<float>())
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.Padding(FMargin(0.0f, 0.0f, 6.0f, 0.0f))
		[
			SNew(SComboButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.OnGetMenuContent(this, &FMovieSceneTimeWarpVariantCustomization::BuildTypePickerMenu)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FMovieSceneTimeWarpVariantCustomization::GetTypeComboLabel)
			]
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.Padding(FMargin(6.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(SSpinBox<double>)
			.Visibility(this, &FMovieSceneTimeWarpVariantCustomization::GetFixedVisibility)
			.Style(FAppStyle::Get(), "Sequencer.HyperlinkSpinBox")
			.Font(FAppStyle::GetFontStyle("Sequencer.FixedFont"))
			.OnValueCommitted(this, &FMovieSceneTimeWarpVariantCustomization::OnCommitFixedPlayRate)
			.OnValueChanged(this, &FMovieSceneTimeWarpVariantCustomization::SetFixedPlayRate)
			.MinValue(TOptional<double>())
			.MaxValue(TOptional<double>())
			.OnEndSliderMovement(this, &FMovieSceneTimeWarpVariantCustomization::SetFixedPlayRate)
			.Value(this, &FMovieSceneTimeWarpVariantCustomization::GetFixedPlayRate)
		]
	];
}

void FMovieSceneTimeWarpVariantCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

EVisibility FMovieSceneTimeWarpVariantCustomization::GetFixedVisibility() const
{
	return IsFixed() ? EVisibility::Visible : EVisibility::Collapsed;
}

void FMovieSceneTimeWarpVariantCustomization::OnCommitFixedPlayRate(double InValue, ETextCommit::Type Type)
{
	SetFixedPlayRate(InValue);
}

void FMovieSceneTimeWarpVariantCustomization::SetFixedPlayRate(double InValue)
{
	FScopedTransaction Transaction(LOCTEXT("ChangeValue_Transation", "Change Time Warp"));

	PropertyHandle->NotifyPreChange();

	bool bNeedsRefresh = false;
	auto SetFixedPlayRate = [InValue, &bNeedsRefresh](void* RawData, const int32, const int32) -> bool
	{
		FMovieSceneTimeWarpVariant* Variant = static_cast<FMovieSceneTimeWarpVariant*>(RawData);
		// Refresh the children if any of the edited values are not already fixed
		bNeedsRefresh |= Variant->GetType() != EMovieSceneTimeWarpType::FixedPlayRate;
		Variant->Set(InValue);
		return true;
	};
	PropertyHandle->EnumerateRawData(SetFixedPlayRate);

	bIsFixed = true;
	Class.Reset();

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
	if (bNeedsRefresh)
	{
		PropertyHandle->RequestRebuildChildren();
	}
}

double FMovieSceneTimeWarpVariantCustomization::GetFixedPlayRate() const
{
	double Value = 0.0;

	auto GatherFixedPlayRate = [&Value](void* RawData, const int32, const int32) -> bool
	{
		FMovieSceneTimeWarpVariant* Variant = static_cast<FMovieSceneTimeWarpVariant*>(RawData);
		if (Variant->GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
		{
			Value = Variant->AsFixedPlayRate();
		}
		return true;
	};

	PropertyHandle->EnumerateRawData(GatherFixedPlayRate);
	return Value;
}

FText FMovieSceneTimeWarpVariantCustomization::GetTypeComboLabel() const
{
	if (bIsFixed)
	{
		return LOCTEXT("FixedPlayRateLabel", "Fixed Play Rate");
	}
	else if (Class.IsSet())
	{
		return Class.GetValue()->GetDisplayNameText();
	}
	else
	{
		return LOCTEXT("MixedTypesLabel", "<< Mixed Types >>");
	}
}

void FMovieSceneTimeWarpVariantCustomization::ChangeClassType(UClass* InClass)
{
	FScopedTransaction Transaction(LOCTEXT("ChangeType_Transation", "Change Time Warp Type"));

	if (!InClass || !InClass->IsChildOf(UMovieSceneTimeWarpGetter::StaticClass()))
	{
		return;
	}

	PropertyHandle->NotifyPreChange();

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	bool bNeedsRefresh = false;
	auto SetFixedPlayRate = [InClass, &Objects, &bNeedsRefresh](void* RawData, const int32 Index, const int32 Num) -> bool
	{
		FMovieSceneTimeWarpVariant* Variant = static_cast<FMovieSceneTimeWarpVariant*>(RawData);

		if (Num != Objects.Num())
		{
			return false;
		}

		// Refresh the children if any of the edited values are already fixed, or have a different class
		UMovieSceneTimeWarpGetter* Existing = Variant->GetType() == EMovieSceneTimeWarpType::Custom ? Variant->AsCustom() : nullptr;
		if (!Existing || Existing->GetClass() != InClass)
		{
			UObject* Object = Objects[Index];
			check(Object);
			Object->Modify();

			bNeedsRefresh = true;

			UMovieSceneSection* Section = Cast<UMovieSceneSection>(Object); 
			if (!Section)
			{
				Section = Object->GetTypedOuter<UMovieSceneSection>();
			}
			ensure(Section);

			UObject* GetterOuter = Object->GetTypedOuter<UMovieSceneSequence>();
			UMovieSceneTimeWarpGetter* New = NewObject<UMovieSceneTimeWarpGetter>(GetterOuter ? GetterOuter : Object, InClass, NAME_None, RF_Transactional);
			New->InitializeDefaults(Section);
			Variant->Set(New);
		}
		return true;
	};
	PropertyHandle->EnumerateRawData(SetFixedPlayRate);

	Class = InClass;
	bIsFixed = false;

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
	if (bNeedsRefresh)
	{
		PropertyHandle->RequestRebuildChildren();
	}
}

bool FMovieSceneTimeWarpVariantCustomization::IsFixed() const
{
	return bIsFixed;
}

void FMovieSceneTimeWarpVariantCustomization::SetFixed()
{
	SetFixedPlayRate(1.0);
}

TSharedRef<SWidget> FMovieSceneTimeWarpVariantCustomization::BuildTypePickerMenu()
{
	bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarpTypesHeader", "Choose a Time Warp:"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("FixedPlayRate_Label", "Fixed Play Rate"),
			LOCTEXT("FixedPlayRate_Tip",   "Change this time warp to have a fixed (constant) play rate."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FMovieSceneTimeWarpVariantCustomization::SetFixed)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddSeparator();

		FSequencerUtilities::PopulateTimeWarpSubMenu(MenuBuilder, [this](TSubclassOf<UMovieSceneTimeWarpGetter> NewClass){
			this->ChangeClassType(NewClass.Get());
		});
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

} // namespace UE::MovieScene

#undef LOCTEXT_NAMESPACE