// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Types/SlateEnums.h"

struct EVisibility;

class FText;
class UClass;
class SWidget;

namespace UE::MovieScene
{

class FMovieSceneTimeWarpVariantCustomization : public IPropertyTypeCustomization
{
public:

	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	bool IsFixed() const;
	void SetFixed();

	void SetFixedPlayRate(double InValue);
	void OnCommitFixedPlayRate(double InValue, ETextCommit::Type Type = ETextCommit::Default);
	double GetFixedPlayRate() const;

	FText GetTypeComboLabel() const;

	EVisibility GetFixedVisibility() const;

	TSharedRef<SWidget> BuildTypePickerMenu();

	void ChangeClassType(UClass* InClass);

	TSharedPtr<IPropertyHandle> PropertyHandle;
	TOptional<UClass*> Class;
	bool bIsFixed = true;
};

} // namespace UE::MovieScene