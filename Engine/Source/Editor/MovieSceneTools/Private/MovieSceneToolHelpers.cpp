// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneToolHelpers.h"
#include "ActorForWorldTransforms.h"
#include "Animation/AnimationSettings.h"
#include "MovieSceneToolsModule.h"
#include "MovieScene.h"
#include "Layout/Margin.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/ArrayView.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboBox.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EditorDirectories.h"
#include "Sections/MovieSceneDoubleSection.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "LevelSequence.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DesktopPlatformModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "MovieSceneTranslatorEDL.h"
#include "MessageLogModule.h"
#include "IMessageLogListing.h"
#include "FbxImporter.h"
#include "MovieSceneToolsProjectSettings.h"
#include "MovieSceneToolsUserSettings.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Math/UnitConversion.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "LevelEditorViewport.h"
#include "AssetToolsModule.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"
#include "TrackEditors/TransformTrackEditor.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieScene3DConstraintTrack.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Engine/LevelStreaming.h"
#include "FbxExporter.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "AnimationRecorder.h"
#include "Components/SkeletalMeshComponent.h"
#include "ILiveLinkClient.h"
#include "LiveLinkPresetTypes.h"
#include "LiveLinkSourceSettings.h"
#include "Features/IModularFeatures.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Sections/MovieSceneSpawnSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Exporters/AnimSeqExportOption.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "FrameNumberDetailsCustomization.h"
#include "PropertyEditorDelegates.h"
#include "INodeAndChannelMappings.h"
#include "BakingAnimationKeySettings.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "ConstraintsManager.h"
#include "IMovieScenePlaybackClient.h"
#include "SequencerChannelInterface.h"
#include "Constraints/MovieSceneConstraintChannelHelper.inl"
#include "Exporters/FbxExportOption.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "GameFramework/Character.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "Transform/AnimationEvaluation.h"

/* FSkelMeshRecorder
 ***********/

void FSkelMeshRecorderState::Init(USkeletalMeshComponent* InComponent)
{
	SkelComp = InComponent;

	if (InComponent)
	{
		CachedSkelCompForcedLodModel = InComponent->GetForcedLOD();
		InComponent->SetForcedLOD(1);

		// turn off URO and make sure we always update even if out of view
		bCachedEnableUpdateRateOptimizations = InComponent->bEnableUpdateRateOptimizations;
		CachedVisibilityBasedAnimTickOption = InComponent->VisibilityBasedAnimTickOption;

		InComponent->bEnableUpdateRateOptimizations = false;
		InComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
}

void FSkelMeshRecorderState::FinishRecording()
{
	if (SkelComp.IsValid())
	{
		// restore force lod setting
		SkelComp->SetForcedLOD(CachedSkelCompForcedLodModel);

		// restore update flags
		SkelComp->bEnableUpdateRateOptimizations = bCachedEnableUpdateRateOptimizations;
		SkelComp->VisibilityBasedAnimTickOption = CachedVisibilityBasedAnimTickOption;
	}
}


/* MovieSceneToolHelpers
 *****************************************************************************/

void MovieSceneToolHelpers::TrimSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time, bool bTrimLeft, bool bDeleteKeys)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section)
		{
			Section->TrimSection(Time, bTrimLeft, bDeleteKeys);
		}
	}
}

bool MovieSceneToolHelpers::CanTrimSectionLeft(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section)
		{
			TRange<FFrameNumber> Range = Section->GetRange();
			TArray<TRange<FFrameNumber> > Splits = Range.Split(Time.Time.FrameNumber);

			if (Splits.Num() > 0)
			{
				if (Splits[0] == Range) // can't split
				{
					continue;
				}
				else
				{
					TRange<FFrameNumber> Head = Splits[0];

					if (!Head.IsEmpty())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool MovieSceneToolHelpers::CanTrimSectionRight(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section)
		{
			TRange<FFrameNumber> Range = Section->GetRange();
			TArray<TRange<FFrameNumber> > Splits = Range.Split(Time.Time.FrameNumber);

			if (Splits.Num() > 0)
			{
				if (Splits[0] == Range) // can't split
				{
					continue;
				}
				else
				{
					TRange<FFrameNumber> Tail = Splits.Last();

					if (!Tail.IsEmpty())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void MovieSceneToolHelpers::TrimOrExtendSection(UMovieSceneTrack* Track, TOptional<int32> SpecifiedRowIndex, FQualifiedFrameTime Time, bool bTrimOrExtendLeft, bool bDeleteKeys)
{
	Track->Modify();

	int32 StartRowIndex = SpecifiedRowIndex.IsSet() ? SpecifiedRowIndex.GetValue() : 0;
	int32 EndRowIndex = SpecifiedRowIndex.IsSet() ? SpecifiedRowIndex.GetValue() : Track->GetMaxRowIndex();

	for (int32 RowIndex = StartRowIndex; RowIndex <= EndRowIndex; ++RowIndex)
	{
		// First, trim all intersecting sections
		bool bAnyIntersects = false;
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section->GetRowIndex() == RowIndex && Section->HasStartFrame() && Section->HasEndFrame() && Section->GetRange().Contains(Time.Time.GetFrame()))
			{
				Section->TrimSection(Time, bTrimOrExtendLeft, bDeleteKeys);
				bAnyIntersects = true;
			}
		}

		// If there aren't any intersects, extend the closest start/end
		if (!bAnyIntersects)
		{
			UMovieSceneSection* ClosestSection = nullptr;
			TOptional<FFrameNumber> MinDiff;

			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				if (Section->GetRowIndex() == RowIndex)
				{
					if (bTrimOrExtendLeft)
					{
						if (Section->HasStartFrame())
						{
							FFrameNumber StartFrame = Section->GetInclusiveStartFrame();
							if (StartFrame > Time.Time.GetFrame())
							{
								FFrameNumber Diff = StartFrame - Time.Time.GetFrame();
								if (!MinDiff.IsSet() || Diff < MinDiff.GetValue())
								{
									ClosestSection = Section;
									MinDiff = Diff;
								}
							}
						}
					}
					else
					{
						if (Section->HasEndFrame())
						{
							FFrameNumber EndFrame = Section->GetExclusiveEndFrame();
							if (EndFrame < Time.Time.GetFrame())
							{
								FFrameNumber Diff = Time.Time.GetFrame() - EndFrame;
								if (!MinDiff.IsSet() || Diff < MinDiff.GetValue())
								{
									ClosestSection = Section;
									MinDiff = Diff;
								}
							}
						}
					}
				}
			}

			if (ClosestSection)
			{
				ClosestSection->Modify();
				if (bTrimOrExtendLeft)
				{
					ClosestSection->SetStartFrame(Time.Time.GetFrame());
				}
				else
				{
					ClosestSection->SetEndFrame(Time.Time.GetFrame());
				}
			}
		}
	}
}


void MovieSceneToolHelpers::SplitSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time, bool bDeleteKeys)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section)
		{
			Section->SplitSection(Time, bDeleteKeys);
		}
	}
}

bool MovieSceneToolHelpers::CanSplitSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section)
		{
			TRange<FFrameNumber> Range = Section->GetRange();
			TArray<TRange<FFrameNumber> > Splits = Range.Split(Time.Time.FrameNumber);

			if (Splits.Num() > 0)
			{
				if (Splits[0] == Range) // can't split
				{
					continue;
				}
				else if (Splits.Num() == 2)
				{
					TRange<FFrameNumber> Head = Splits[0];
					TRange<FFrameNumber> Tail = Splits[1];

					if (!Head.IsEmpty() && !Tail.IsEmpty())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

FTransform MovieSceneToolHelpers::GetTransformOriginForFocusedSequence(TSharedPtr<ISequencer> InSequencer)
{
	FTransform TransformOrigin;

	const IMovieScenePlaybackClient*  Client       = InSequencer->GetPlaybackClient();
	const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
	const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

	const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
	if (bHasInterface)
	{
		// Retrieve the current origin
		TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);
	}
	
	const TArray<FMovieSceneSequenceID>& Hierarchy = InSequencer->GetSubSequenceHierarchy();

	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = InSequencer->GetEvaluationTemplate();
	const UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if(!EntityLinker)
	{
		return TransformOrigin;
	}

	const UMovieSceneTransformOriginSystem* TransformOriginSystem = EntityLinker->FindSystem<UMovieSceneTransformOriginSystem>();

	if(!TransformOriginSystem)
	{
		return TransformOrigin;
	}

	const TMap<FMovieSceneSequenceID, UE::MovieScene::FInstanceHandle> SequenceIDToInstanceHandle = TransformOriginSystem->GetSequenceIDToInstanceHandle();

	// Transform Origins will be pre-multiplied at this step, so only retrieve the entry for the currently focused sub-sequence.
	if(Hierarchy.Num())
	{
		const FMovieSceneSequenceID CurrentSequence = Hierarchy.Last();
		if(SequenceIDToInstanceHandle.Contains(CurrentSequence))
		{
			const UE::MovieScene::FInstanceHandle CurrentHandle = SequenceIDToInstanceHandle[CurrentSequence];
			// Overrides the origin only if entry is present
			TransformOriginSystem->GetTransformOrigin(CurrentHandle, TransformOrigin);
		}
	}

	return TransformOrigin;
}

bool MovieSceneToolHelpers::ParseShotName(const FString& InShotName, FString& ShotPrefix, uint32& ShotNumber, uint32& TakeNumber, uint32& ShotNumberDigits, uint32& TakeNumberDigits)
{
	// Parse a shot name
	// 
	// sht010:
	//  ShotPrefix = sht
	//  ShotNumber = 10
	//  TakeNumber = 1 (default)
	// 
	// sp020_002
	//  ShotPrefix = sp
	//  ShotNumber = 20
	//  TakeNumber = 2
	//

	// If the shot prefix is known and is already at the beginning of the shot name, remove it and parse the rest
	//
	// sq050_sh0010_01
	//  ShotPrefix = sq050_sh, ie. parse(0010_01)
	//  ShotNumber = 10
	//  TakeNumber = 1
	FString ShotName = InShotName;
	ShotName.RemoveFromStart(ShotPrefix);

	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	uint32 FirstShotNumberIndex = INDEX_NONE;
	uint32 LastShotNumberIndex = INDEX_NONE;
	bool bInShotNumber = false;

	uint32 FirstTakeNumberIndex = INDEX_NONE;
	uint32 LastTakeNumberIndex = INDEX_NONE;
	bool bInTakeNumber = false;

	bool bFoundTakeSeparator = false;
	TOptional<uint32> ParsedTakeNumber;
	TakeNumber = ProjectSettings->FirstTakeNumber;

	for (int32 CharIndex = 0; CharIndex < ShotName.Len(); ++CharIndex)
	{
		if (FChar::IsDigit(ShotName[CharIndex]))
		{
			// Find shot number indices
			if (FirstShotNumberIndex == INDEX_NONE)
			{
				bInShotNumber = true;
				FirstShotNumberIndex = CharIndex;
			}
			if (bInShotNumber)
			{
				LastShotNumberIndex = CharIndex;
			}

			if (FirstShotNumberIndex != INDEX_NONE && LastShotNumberIndex != INDEX_NONE)
			{
				if (bFoundTakeSeparator)
				{
					// Find take number indices
					if (FirstTakeNumberIndex == INDEX_NONE)
					{
						bInTakeNumber = true;
						FirstTakeNumberIndex = CharIndex;
					}
					if (bInTakeNumber)
					{
						LastTakeNumberIndex = CharIndex;
					}
				}
			}
		}

		if (FirstShotNumberIndex != INDEX_NONE && LastShotNumberIndex != INDEX_NONE)
		{
			if (ShotName[CharIndex] == ProjectSettings->TakeSeparator[0])
			{
				bFoundTakeSeparator = true;
				ShotNumberDigits = LastShotNumberIndex - FirstShotNumberIndex + 1;
			}
		}
	}

	if (FirstShotNumberIndex != INDEX_NONE)
	{
		if (FirstShotNumberIndex != 0)
		{
			ShotPrefix = ShotName.Left(FirstShotNumberIndex);
		}

		ShotNumber = FCString::Atoi(*ShotName.Mid(FirstShotNumberIndex, LastShotNumberIndex-FirstShotNumberIndex+1));
	}

	if (FirstTakeNumberIndex != INDEX_NONE)
	{
		FString TakeStr = ShotName.Mid(FirstTakeNumberIndex, LastTakeNumberIndex-FirstTakeNumberIndex+1);
		if (TakeStr.IsNumeric())
		{
			ParsedTakeNumber = FCString::Atoi(*TakeStr);
			TakeNumberDigits = TakeStr.Len();
		}
	}

	// If take number wasn't found, start over with the original shot name, search backwards to find the first take separator and assume [shot prefix]_[take number]
	//
	if (!ParsedTakeNumber.IsSet())
	{
		ShotName = InShotName;
		int32 LastSlashPos = ShotName.Find(ProjectSettings->TakeSeparator, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (LastSlashPos != INDEX_NONE)
		{
			if (LastSlashPos != 0)
			{
				ShotPrefix = ShotName.Left(LastSlashPos);
			}
			
			FString TakeStr = ShotName.RightChop(LastSlashPos + ProjectSettings->TakeSeparator.Len());
			if (TakeStr.IsNumeric())
			{
				ShotNumber = INDEX_NONE; // Nullify the shot number since we only have a shot prefix
				TakeNumber = FCString::Atoi(*TakeStr);
				TakeNumberDigits = ShotName.Len() - (LastSlashPos + ProjectSettings->TakeSeparator.Len());
				return true;
			}
		}
	}

	if (ParsedTakeNumber.IsSet())
	{
		TakeNumber = ParsedTakeNumber.GetValue();
	}

	if (FirstShotNumberIndex == INDEX_NONE)
	{
		ShotPrefix = InShotName;
		ShotNumber = INDEX_NONE; // Nullify the shot number since we only have a shot prefix
		TakeNumber = 0;
		TakeNumberDigits = ProjectSettings->TakeNumDigits;
		return true;
	}

	return FirstShotNumberIndex != INDEX_NONE;
}

FString MovieSceneToolHelpers::ComposeShotName(const FString& ShotPrefix, uint32 ShotNumber, uint32 TakeNumber, uint32 ShotNumberDigits, uint32 TakeNumberDigits)
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	FString ShotName = ShotPrefix;

	if (ShotNumber != INDEX_NONE)
	{
		ShotName += FString::Printf(TEXT("%0*d"), ShotNumberDigits, ShotNumber);
	}

	if (TakeNumber != INDEX_NONE)
	{
		FString TakeFormat = TEXT("%0") + FString::Printf(TEXT("%d"), TakeNumberDigits) + TEXT("d");
		
		ShotName += ProjectSettings->TakeSeparator;
		ShotName += FString::Printf(TEXT("%0*d"), TakeNumberDigits, TakeNumber);
	}
	return ShotName;
}

bool IsPackageNameUnique(const TArray<FAssetData>& ObjectList, FString& NewPackageName)
{
	for (auto AssetObject : ObjectList)
	{
		if (AssetObject.PackageName.ToString() == NewPackageName)
		{
			return false;
		}
	}
	return true;
}

FString MovieSceneToolHelpers::GenerateNewShotPath(UMovieScene* SequenceMovieScene, FString& NewShotName)
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	return GenerateNewSubsequencePath(SequenceMovieScene, ProjectSettings->ShotDirectory, NewShotName);
}

FString MovieSceneToolHelpers::GenerateNewSubsequencePath(UMovieScene * SequenceMovieScene, const FString& SubsequenceDirectory, FString &NewShotName)
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> ObjectList;
	AssetRegistryModule.Get().GetAssetsByClass(ULevelSequence::StaticClass()->GetClassPathName(), ObjectList);

	UObject* SequenceAsset = SequenceMovieScene->GetOuter();
	UPackage* SequencePackage = SequenceAsset->GetOutermost();
	FString SequencePackageName = SequencePackage->GetName(); // ie. /Game/cine/max/root
	int32 LastSlashPos = SequencePackageName.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	FString SequencePath = SequencePackageName.Left(LastSlashPos);

	FString NewShotPrefix;
	uint32 NewShotNumber = INDEX_NONE;
	uint32 NewTakeNumber = INDEX_NONE;
	uint32 ShotNumberDigits = ProjectSettings->ShotNumDigits;
	uint32 TakeNumberDigits = ProjectSettings->TakeNumDigits;
	ParseShotName(NewShotName, NewShotPrefix, NewShotNumber, NewTakeNumber, ShotNumberDigits, TakeNumberDigits);

	FString NewDirectory = ComposeShotName(NewShotPrefix, NewShotNumber, INDEX_NONE, ShotNumberDigits, TakeNumberDigits);
	FString NewPath = SequencePath;

	FString Directory = SubsequenceDirectory;
	if (!Directory.IsEmpty())
	{
		NewPath /= Directory;
	}
	NewPath /= NewDirectory; // put this in the shot directory, ie. /Game/cine/max/shots/shot0010

	// Make sure this shot path is unique
	FString NewPackageName = NewPath;
	NewPackageName /= NewShotName; // ie. /Game/cine/max/shots/shot0010/shot0010_001
	if (!IsPackageNameUnique(ObjectList, NewPackageName))
	{
		while (1)
		{
			NewShotNumber += ProjectSettings->ShotIncrement;
			NewShotName = ComposeShotName(NewShotPrefix, NewShotNumber, NewTakeNumber, ShotNumberDigits, TakeNumberDigits);
			NewDirectory = ComposeShotName(NewShotPrefix, NewShotNumber, INDEX_NONE, ShotNumberDigits, TakeNumberDigits);
			NewPath = SequencePath;
			if (!Directory.IsEmpty())
			{
				NewPath /= Directory;
			}
			NewPath /= NewDirectory;

			NewPackageName = NewPath;
			NewPackageName /= NewShotName;
			if (IsPackageNameUnique(ObjectList, NewPackageName))
			{
				break;
			}
		}
	}

	return NewPath;
}

FString MovieSceneToolHelpers::GenerateNewShotName(const TArray<UMovieSceneSection*>& AllSections, FFrameNumber Time)
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	return GenerateNewSubsequenceName(AllSections, ProjectSettings->ShotPrefix, Time);
}

FString MovieSceneToolHelpers::GenerateNewSubsequenceName(const TArray<UMovieSceneSection*>&AllSections, const FString& SubsequencePrefix, FFrameNumber Time)
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	UMovieSceneSubSection* CurrentSection = Cast<UMovieSceneSubSection>(MovieSceneHelpers::FindSectionAtTime(AllSections, Time));
	UMovieSceneSubSection* NextSection = Cast<UMovieSceneSubSection>(MovieSceneHelpers::FindNextSection(AllSections, Time));

	if (!CurrentSection)
	{
		CurrentSection = Cast<UMovieSceneSubSection>(MovieSceneHelpers::FindPreviousSection(AllSections, Time));
	}

	if (!NextSection)
	{
		NextSection = CurrentSection;
	}

	FString NextSectionName = NextSection && NextSection->GetSequence() ? NextSection->GetSequence()->GetName() : FString();
	FString CurrentSectionName = CurrentSection && CurrentSection->GetSequence() ? CurrentSection->GetSequence()->GetName() : FString();

	// This is the first or last shot
	if ((CurrentSection == nullptr && NextSection) || (CurrentSection != nullptr && CurrentSection == NextSection))
	{
		FString NextShotPrefix = SubsequencePrefix;
		uint32 NextShotNumber = ProjectSettings->FirstShotNumber;
		uint32 NextTakeNumber = ProjectSettings->FirstTakeNumber;
		uint32 ShotNumberDigits = ProjectSettings->ShotNumDigits;
		uint32 TakeNumberDigits = ProjectSettings->TakeNumDigits;

		if (ParseShotName(NextSectionName, NextShotPrefix, NextShotNumber, NextTakeNumber, ShotNumberDigits, TakeNumberDigits))
		{
			// Valid shot number
			if (NextShotNumber != INDEX_NONE)
			{
				uint32 NewShotNumber = NextShotNumber + ProjectSettings->ShotIncrement;
				return ComposeShotName(NextShotPrefix, NewShotNumber, ProjectSettings->FirstTakeNumber, ShotNumberDigits, TakeNumberDigits);
			}
			// No shot number, but valid take number
			else if (NextTakeNumber != INDEX_NONE)
			{
				uint32 NewTakeNumber = NextTakeNumber + ProjectSettings->ShotIncrement;
				return ComposeShotName(NextShotPrefix, INDEX_NONE, NewTakeNumber, ShotNumberDigits, TakeNumberDigits);
			}
		}
	}
	// This is in between two shots
	else if (CurrentSection && NextSection)
	{
		FString CurrentShotPrefix = SubsequencePrefix;
		uint32 CurrentShotNumber = ProjectSettings->FirstShotNumber;
		uint32 CurrentTakeNumber = ProjectSettings->FirstTakeNumber;
		uint32 CurrentShotNumberDigits = ProjectSettings->ShotNumDigits;
		uint32 CurrentTakeNumberDigits = ProjectSettings->TakeNumDigits;

		FString NextShotPrefix = SubsequencePrefix;
		uint32 NextShotNumber = ProjectSettings->FirstShotNumber;
		uint32 NextTakeNumber = ProjectSettings->FirstTakeNumber;
		uint32 NextShotNumberDigits = ProjectSettings->ShotNumDigits;
		uint32 NextTakeNumberDigits = ProjectSettings->TakeNumDigits;

		if (ParseShotName(CurrentSectionName, CurrentShotPrefix, CurrentShotNumber, CurrentTakeNumber, CurrentShotNumberDigits, CurrentTakeNumberDigits) &&
			ParseShotName(NextSectionName, NextShotPrefix, NextShotNumber, NextTakeNumber, NextShotNumberDigits, NextTakeNumberDigits))
		{
			// Valid shot numbers
			if (NextShotNumber != INDEX_NONE && CurrentShotNumber != INDEX_NONE)
			{
				uint32 NewShotNumber = CurrentShotNumber + ProjectSettings->ShotIncrement;
				if (NextShotNumber - CurrentShotNumber > 1)
				{
					NewShotNumber = CurrentShotNumber + ( (NextShotNumber - CurrentShotNumber) / 2); // what if we can't find one? or conflicts with another?
				}
				return ComposeShotName(CurrentShotPrefix, NewShotNumber, ProjectSettings->FirstTakeNumber, CurrentShotNumberDigits, CurrentTakeNumberDigits); // use before or next shot?
			}
			// No shot numbers, but valid take numbers
			else if (NextTakeNumber != INDEX_NONE && CurrentTakeNumber != INDEX_NONE)
			{
				uint32 NewTakeNumber = CurrentTakeNumber + ProjectSettings->ShotIncrement;
				if (NextTakeNumber - CurrentTakeNumber > 1)
				{
					NewTakeNumber = CurrentTakeNumber + ( (NextTakeNumber - CurrentTakeNumber) / 2); // what if we can't find one? or conflicts with another?
				}
				return ComposeShotName(CurrentShotPrefix, INDEX_NONE, NewTakeNumber, CurrentShotNumberDigits, CurrentTakeNumberDigits); // use before or next shot?
			}
		}
	}

	// Default case
	return ComposeShotName(SubsequencePrefix, ProjectSettings->FirstShotNumber, ProjectSettings->FirstTakeNumber, ProjectSettings->ShotNumDigits, ProjectSettings->TakeNumDigits);
}

UMovieSceneSequence* MovieSceneToolHelpers::CreateSequence(FString& NewSequenceName, FString& NewSequencePath, UMovieSceneSubSection* SectionToDuplicate, TSubclassOf<UMovieSceneSequence> SequenceClass)
{
	// Create a new level sequence asset with the appropriate name
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	if (!SequenceClass)
	{
		SequenceClass = ULevelSequence::StaticClass();
	}
	UObject* NewAsset = nullptr;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* CurrentClass = *It;
		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == SequenceClass)
			{
				if (SectionToDuplicate != nullptr)
				{
					NewAsset = AssetTools.DuplicateAssetWithDialog(NewSequenceName, NewSequencePath, SectionToDuplicate->GetSequence());
				}
				else
				{
					NewAsset = AssetTools.CreateAssetWithDialog(NewSequenceName, NewSequencePath, SequenceClass, Factory);
				}
				break;
			}
		}
	}

	if (NewAsset == nullptr)
	{
		return nullptr;
	}

	return Cast<UMovieSceneSequence>(NewAsset);
}

void MovieSceneToolHelpers::GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber)
{
	const UMovieSceneSubSection* SubSection = Cast<const UMovieSceneSubSection>(Section);
	
	if (SubSection->GetSequence() == nullptr)
	{
		return;
	}

	if (FMovieSceneToolsModule::Get().GatherTakes(Section, AssetData, OutCurrentTakeNumber))
	{
		return;
	}

	FAssetData ShotData(SubSection->GetSequence()->GetOuter());

	FString ShotPackagePath = ShotData.PackagePath.ToString();

	FString ShotPrefix;
	uint32 ShotNumber = INDEX_NONE;
	OutCurrentTakeNumber = INDEX_NONE;
	uint32 ShotNumberDigits = 0;
	uint32 TakeNumberDigits = 0;

	FString SequenceName = SubSection->GetSequence()->GetName();
	ParseShotName(SequenceName, ShotPrefix, ShotNumber, OutCurrentTakeNumber, ShotNumberDigits, TakeNumberDigits);

	// Gather up all level sequence assets
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> ObjectList;
	AssetRegistryModule.Get().GetAssetsByClass(ULevelSequence::StaticClass()->GetClassPathName(), ObjectList);

	for (auto AssetObject : ObjectList)
	{
		FString AssetPackagePath = AssetObject.PackagePath.ToString();

		if (AssetPackagePath == ShotPackagePath)
		{
			FString AssetShotPrefix;
			uint32 AssetShotNumber = INDEX_NONE;
			uint32 AssetTakeNumber = INDEX_NONE;

			ParseShotName(AssetObject.AssetName.ToString(), AssetShotPrefix, AssetShotNumber, AssetTakeNumber, ShotNumberDigits, TakeNumberDigits);
			
			if (AssetShotPrefix == ShotPrefix && AssetShotNumber == ShotNumber)
			{
				AssetData.Add(AssetObject);
			}
		}
	}
}

bool MovieSceneToolHelpers::GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber)
{
	if (FMovieSceneToolsModule::Get().GetTakeNumber(Section, AssetData, OutTakeNumber))
	{
		return true;
	}

	const UMovieSceneSubSection* SubSection = Cast<const UMovieSceneSubSection>(Section);

	FAssetData ShotData(SubSection->GetSequence()->GetOuter());

	FString ShotPackagePath = ShotData.PackagePath.ToString();
	int32 ShotLastSlashPos = INDEX_NONE;
	ShotPackagePath.FindLastChar(TCHAR('/'), ShotLastSlashPos);
	ShotPackagePath.LeftInline(ShotLastSlashPos, EAllowShrinking::No);

	FString ShotPrefix;
	uint32 ShotNumber = INDEX_NONE;
	uint32 TakeNumberDummy = INDEX_NONE;
	uint32 ShotNumberDigits = 0;
	uint32 TakeNumberDigits = 0;

	FString SequenceName = SubSection->GetSequence()->GetName();
	ParseShotName(SequenceName, ShotPrefix, ShotNumber, TakeNumberDummy, ShotNumberDigits, TakeNumberDigits);
	
	// Gather up all level sequence assets
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> ObjectList;
	AssetRegistryModule.Get().GetAssetsByClass(ULevelSequence::StaticClass()->GetClassPathName(), ObjectList);

	for (auto AssetObject : ObjectList)
	{
		if (AssetObject == AssetData)
		{
			FString AssetPackagePath = AssetObject.PackagePath.ToString();
			int32 AssetLastSlashPos = INDEX_NONE;
			AssetPackagePath.FindLastChar(TCHAR('/'), AssetLastSlashPos);
			AssetPackagePath.LeftInline(AssetLastSlashPos, EAllowShrinking::No);

			if (AssetPackagePath == ShotPackagePath)
			{
				FString AssetShotPrefix;
				uint32 AssetShotNumber = INDEX_NONE;
				uint32 AssetTakeNumber = INDEX_NONE;

				ParseShotName(AssetObject.AssetName.ToString(), AssetShotPrefix, AssetShotNumber, AssetTakeNumber, ShotNumberDigits, TakeNumberDigits);

				if (AssetShotPrefix == ShotPrefix && AssetShotNumber == ShotNumber)
				{
					OutTakeNumber = AssetTakeNumber;
					return true;
				}
			}
		}
	}

	return false;
}

bool MovieSceneToolHelpers::SetTakeNumber(const UMovieSceneSection* Section, uint32 InTakeNumber)
{
	return FMovieSceneToolsModule::Get().SetTakeNumber(Section, InTakeNumber);
}

int32 MovieSceneToolHelpers::FindAvailableRowIndex(UMovieSceneTrack* InTrack, UMovieSceneSection* InSection, const TArray<UMovieSceneSection*>& SectionsToDisregard)
{
	for (int32 RowIndex = 0; RowIndex <= InTrack->GetMaxRowIndex(); ++RowIndex)
	{
		bool bFoundIntersect = false;
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			if (SectionsToDisregard.Contains(Section))
			{
				continue;
			}
	
			if (!Section->HasStartFrame() || !Section->HasEndFrame() || !InSection->HasStartFrame() || !InSection->HasEndFrame())
			{
				bFoundIntersect = true;
				break;
			}

			if (Section != InSection && Section->GetRowIndex() == RowIndex && Section->GetRange().Overlaps(InSection->GetRange()))
			{
				bFoundIntersect = true;
				break;
			}
		}
		if (!bFoundIntersect)
		{
			return RowIndex;
		}
	}

	return InTrack->GetMaxRowIndex() + 1;
}


bool MovieSceneToolHelpers::OverlapsSection(UMovieSceneTrack* InTrack, UMovieSceneSection* InSection, const TArray<UMovieSceneSection*>& SectionsToDisregard)
{
	for (UMovieSceneSection* Section : InTrack->GetAllSections())
	{
		if (SectionsToDisregard.Contains(Section))
		{
			continue;
		}
	
		if (!Section->HasStartFrame() || !Section->HasEndFrame() || !InSection->HasStartFrame() || !InSection->HasEndFrame())
		{
			return true;
		}

		if (Section != InSection && Section->GetRange().Overlaps(InSection->GetRange()))
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SWidget> MovieSceneToolHelpers::MakeEnumComboBox(const UEnum* InEnum, TAttribute<int32> InCurrentValue, SEnumComboBox::FOnEnumSelectionChanged InOnSelectionChanged)
{
	return SNew(SEnumComboBox, InEnum)
		.CurrentValue(InCurrentValue)
		.ContentPadding(FMargin(2, 0))
		.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
		.OnEnumSelectionChanged(InOnSelectionChanged);
}

bool MovieSceneToolHelpers::ShowImportEDLDialog(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InOpenDirectory)
{
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("CMX 3600 EDL (*.edl)|*.edl|");

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("MovieSceneToolHelpers", "ImportEDL", "Import EDL from...").ToString(), 
			InOpenDirectory,
			TEXT(""), 
			*ExtensionStr,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}
	if (!bOpen)
	{
		return false;
	}

	if (!OpenFilenames.Num())
	{
		return false;
	}

	const FScopedTransaction Transaction( NSLOCTEXT( "MovieSceneTools", "ImportEDLTransaction", "Import EDL" ) );

	return MovieSceneTranslatorEDL::ImportEDL(InMovieScene, InFrameRate, OpenFilenames[0]);
}

bool MovieSceneToolHelpers::ShowExportEDLDialog(const UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InSaveDirectory, int32 InHandleFrames, FString InMovieExtension)
{
	TArray<FString> SaveFilenames;
	FString SequenceName = InMovieScene->GetOuter()->GetName();

	// Pop open a dialog to request the location of the edl
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bSave = false;
	if (DesktopPlatform)
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("CMX 3600 EDL (*.edl)|*.edl|");
		ExtensionStr += TEXT("RV (*.rv)|*.rv|");

		bSave = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("MovieSceneTools", "ExportEDL", "Export EDL to...").ToString(), 
			InSaveDirectory,
			SequenceName + TEXT(".edl"), 
			*ExtensionStr,
			EFileDialogFlags::None,
			SaveFilenames
			);
	}
	if (!bSave)
	{
		return false;
	}

	if (!SaveFilenames.Num())
	{
		return false;
	}

	if (MovieSceneTranslatorEDL::ExportEDL(InMovieScene, InFrameRate, SaveFilenames[0], InHandleFrames, InMovieExtension))
	{
		const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SaveFilenames[0]);
		const FString SaveDirectory = FPaths::GetPath(AbsoluteFilename);

		FNotificationInfo NotificationInfo(NSLOCTEXT("MovieSceneTools", "EDLExportFinished", "EDL Export finished"));
		NotificationInfo.ExpireDuration = 5.f;
		NotificationInfo.Hyperlink = FSimpleDelegate::CreateStatic( [](FString InDirectory) { FPlatformProcess::ExploreFolder( *InDirectory ); }, SaveDirectory);
		NotificationInfo.HyperlinkText = NSLOCTEXT("MovieSceneTools", "OpenEDLExportFolder", "Open EDL Export Folder...");
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		
		return true;
	}

	return false;
}

bool MovieSceneToolHelpers::MovieSceneTranslatorImport(FMovieSceneImporter* InImporter, UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InOpenDirectory)
{
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		FString FileTypeDescription = InImporter->GetFileTypeDescription().ToString();
		FString DialogTitle = InImporter->GetDialogTitle().ToString();

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			DialogTitle,
			InOpenDirectory,
			TEXT(""),
			FileTypeDescription,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}

	if (!bOpen || !OpenFilenames.Num())
	{
		return false;
	}

	FScopedTransaction Transaction(InImporter->GetTransactionDescription());

	TSharedRef<FMovieSceneTranslatorContext> ImportContext(new FMovieSceneTranslatorContext);
	ImportContext->Init();

	bool bSuccess = InImporter->Import(InMovieScene, InFrameRate, OpenFilenames[0], ImportContext);

	// Display any messages in context
	MovieSceneTranslatorLogMessages(InImporter, ImportContext, true);

	// Roll back transaction when import fails.
	if (!bSuccess)
	{
		Transaction.Cancel();
	}

	return bSuccess;
}

bool MovieSceneToolHelpers::MovieSceneTranslatorExport(FMovieSceneExporter* InExporter, const UMovieScene* InMovieScene, const FMovieSceneCaptureSettings& Settings)
{
	if (InExporter == nullptr || InMovieScene == nullptr)
	{
		return false;
	}

	FString SaveDirectory = FPaths::ConvertRelativePathToFull(Settings.OutputDirectory.Path);
	int32 HandleFrames = Settings.HandleFrames;
	// @todo: generate filename based on filename format, currently outputs {shot}.avi
	FString FilenameFormat = Settings.OutputFormat;
	FFrameRate FrameRate = Settings.GetFrameRate();
	uint32 ResX = Settings.Resolution.ResX;
	uint32 ResY = Settings.Resolution.ResY;
	FString MovieExtension = Settings.MovieExtension;

	TArray<FString> SaveFilenames;
	FString SequenceName = InMovieScene->GetOuter()->GetName();

	// Pop open a dialog to request the location of the edl
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bSave = false;
	if (DesktopPlatform)
	{
		FString FileTypeDescription = InExporter->GetFileTypeDescription().ToString();
		FString DialogTitle = InExporter->GetDialogTitle().ToString();
		FString FileExtension = InExporter->GetDefaultFileExtension().ToString();

		bSave = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			DialogTitle,
			SaveDirectory,
			SequenceName + TEXT(".") + FileExtension,
			FileTypeDescription,
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bSave || !SaveFilenames.Num())
	{
		return false;
	}

	TSharedRef<FMovieSceneTranslatorContext> ExportContext(new FMovieSceneTranslatorContext);
	ExportContext->Init();

	bool bSuccess = InExporter->Export(InMovieScene, FilenameFormat, FrameRate, ResX, ResY, HandleFrames, SaveFilenames[0], ExportContext, MovieExtension);
	
	// Display any messages in context
	MovieSceneTranslatorLogMessages(InExporter, ExportContext, true);

	if (bSuccess)
	{
		const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(SaveFilenames[0]);
		const FString ActualSaveDirectory = FPaths::GetPath(AbsoluteFilename);

		FNotificationInfo NotificationInfo(InExporter->GetNotificationExportFinished());
		NotificationInfo.ExpireDuration = 5.f;
		NotificationInfo.Hyperlink = FSimpleDelegate::CreateStatic([](FString InDirectory) { FPlatformProcess::ExploreFolder(*InDirectory); }, ActualSaveDirectory);
		NotificationInfo.HyperlinkText = InExporter->GetNotificationHyperlinkText();
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	return bSuccess;
}

void MovieSceneToolHelpers::MovieSceneTranslatorLogMessages(FMovieSceneTranslator *InTranslator, TSharedRef<FMovieSceneTranslatorContext> InContext, bool bDisplayMessages)
{
	if (InTranslator == nullptr || InContext->GetMessages().Num() == 0)
	{
		return;
	}
	
	// Clear any old messages after an import or export
	const FName LogTitle = InTranslator->GetMessageLogWindowTitle();
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedRef<IMessageLogListing> LogListing = MessageLogModule.GetLogListing(LogTitle);
	LogListing->SetLabel(InTranslator->GetMessageLogLabel());
	LogListing->ClearMessages();

	for (TSharedRef<FTokenizedMessage> Message : InContext->GetMessages())
	{
		LogListing->AddMessage(Message);
	}

	if (bDisplayMessages)
	{
		MessageLogModule.OpenMessageLog(LogTitle);
	}
}

void MovieSceneToolHelpers::MovieSceneTranslatorLogOutput(FMovieSceneTranslator *InTranslator, TSharedRef<FMovieSceneTranslatorContext> InContext)
{
	if (InTranslator == nullptr || InContext->GetMessages().Num() == 0)
	{
		return;
	}

	for (TSharedRef<FTokenizedMessage> Message : InContext->GetMessages())
	{
		if (Message->GetSeverity() == EMessageSeverity::Error)
		{
			UE_LOGF(LogMovieScene, Error, "%ls", *Message->ToText().ToString());
		}
		else if (Message->GetSeverity() == EMessageSeverity::Warning)
		{
			UE_LOGF(LogMovieScene, Warning, "%ls", *Message->ToText().ToString());

		}
	}
}

static FGuid GetHandleToObject(UObject* InObject, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, bool bCreateIfMissing = true)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	// Attempt to resolve the object through the movie scene instance first, 
	FGuid PropertyOwnerGuid = FGuid();
	if (InObject != nullptr && !MovieScene->IsReadOnly())
	{
		FGuid ObjectGuid = Player->FindObjectId(*InObject, TemplateID);
		if (ObjectGuid.IsValid())
		{
			// Check here for spawnable otherwise spawnables get recreated as possessables, which doesn't make sense
			FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectGuid);
			if (Spawnable)
			{
				PropertyOwnerGuid = ObjectGuid;
			}
			else
			{
				FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectGuid);
				if (Possessable)
				{
					PropertyOwnerGuid = ObjectGuid;
				}
			}
		}
	}

	if (PropertyOwnerGuid.IsValid())
	{
		return PropertyOwnerGuid;
	}

	if (bCreateIfMissing)
	{
		// Otherwise, create a possessable for this object. Note this will handle creating the parent possessables if this is a component.
		PropertyOwnerGuid = Player->CreateBinding(InSequence, InObject);
	}
	
	return PropertyOwnerGuid;
}

template<typename TrackType, typename ChannelType>
bool ImportFBXPropertyToPropertyTrack(UMovieScene* MovieScene, const FGuid PropertyOwnerGuid, const FMovieSceneToolsFbxSettings& FbxSetting, FRichCurve& Source)
{
	TrackType* PropertyTrack = MovieScene->FindTrack<TrackType>(PropertyOwnerGuid, *FbxSetting.PropertyPath.PropertyName);
	if (!PropertyTrack)
	{
		MovieScene->Modify();
		PropertyTrack = MovieScene->AddTrack<TrackType>(PropertyOwnerGuid);
		PropertyTrack->SetPropertyNameAndPath(*FbxSetting.PropertyPath.PropertyName, *FbxSetting.PropertyPath.PropertyName);
	}

	if (!PropertyTrack)
	{
		return false;
	}

	PropertyTrack->Modify();
	PropertyTrack->RemoveAllAnimationData();

	FFrameRate FrameRate = PropertyTrack->template GetTypedOuter<UMovieScene>()->GetTickResolution();

	bool bSectionAdded = false;
	UMovieSceneSection* Section = Cast<UMovieSceneSection>(PropertyTrack->FindOrAddSection(0, bSectionAdded));
	if (!Section)
	{
		return false;
	}

	Section->Modify();

	if (bSectionAdded)
	{
		Section->SetRange(TRange<FFrameNumber>::All());
	}

	ChannelType* Channel = Section->GetChannelProxy().GetChannel<ChannelType>(0);
	TMovieSceneChannelData<typename ChannelType::ChannelValueType> ChannelData = Channel->GetData();

	ChannelData.Reset();
	double DecimalRate = FrameRate.AsDecimal();

	for (auto SourceIt = Source.GetKeyHandleIterator(); SourceIt; ++SourceIt)
	{
		FRichCurveKey &Key = Source.GetKey(*SourceIt);
		float ArriveTangent = Key.ArriveTangent;
		FKeyHandle PrevKeyHandle = Source.GetPreviousKey(*SourceIt);
		if (Source.IsKeyHandleValid(PrevKeyHandle))
		{
			FRichCurveKey &PrevKey = Source.GetKey(PrevKeyHandle);
			ArriveTangent = ArriveTangent / (Key.Time - PrevKey.Time);

		}
		float LeaveTangent = Key.LeaveTangent;
		FKeyHandle NextKeyHandle = Source.GetNextKey(*SourceIt);
		if (Source.IsKeyHandleValid(NextKeyHandle))
		{
			FRichCurveKey &NextKey = Source.GetKey(NextKeyHandle);
			LeaveTangent = LeaveTangent / (NextKey.Time - Key.Time);
		}

		FFrameNumber KeyTime = (Key.Time * FrameRate).RoundToFrame();
		MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, Key.Value, ArriveTangent, LeaveTangent,
				Key.InterpMode, Key.TangentMode, FrameRate, Key.TangentWeightMode,
				Key.ArriveTangentWeight, Key.LeaveTangentWeight);
	}

	Channel->AutoSetTangents();

	const UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetDefault<UMovieSceneUserImportFBXSettings>();

	if (ImportFBXSettings->bReduceKeys)
	{
		FKeyDataOptimizationParams Params;
		Params.Tolerance = ImportFBXSettings->ReduceKeysTolerance;
		Params.DisplayRate = FrameRate;
		Params.bAutoSetInterpolation = true; //we use this to perform the AutoSetTangents after the keys are reduced.
		Channel->Optimize(Params);
	}

	return true;
}


// Static function to handle applying uniform scale data to keys inside of camera imports that are distance based.
void ApplyUniformScaleToCameraKeys(const FString& AnimatedPropertyName, FRichCurve& Source)
{
	const UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetDefault<UMovieSceneUserImportFBXSettings>();

	// This is the only property right now.
	if (AnimatedPropertyName == "FocusDistance")
	{
		for (int i=0; i<Source.Keys.Num(); i++)
		{
			Source.Keys[i].Value = Source.Keys[i].Value * ImportFBXSettings->ImportUniformScale;
		}
	}
};

bool ImportFBXProperty(FString NodeName, FString AnimatedPropertyName, FGuid ObjectBinding, UnFbx::FFbxCurvesAPI& CurveAPI, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	const int32 ChannelIndex = 0;
	const int32 CompositeIndex = 0;
	FRichCurve Source;
	const bool bNegative = false;
	CurveAPI.GetCurveDataForSequencer(NodeName, AnimatedPropertyName, ChannelIndex, CompositeIndex, Source, bNegative);

	// Modify distance based camera curve info to account for Uniform Scale.
	ApplyUniformScaleToCameraKeys(AnimatedPropertyName, Source);

	// First, see if any of the custom importers can import this named property
	if (FMovieSceneToolsModule::Get().ImportAnimatedProperty(AnimatedPropertyName, Source, ObjectBinding, MovieScene))
	{
		return true;
	}

	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	TArrayView<TWeakObjectPtr<>> BoundObjects = Player->FindBoundObjects(ObjectBinding, TemplateID);

	for (auto FbxSetting : ProjectSettings->FbxSettings)
	{
		if (FCString::Strcmp(*FbxSetting.FbxPropertyName.ToUpper(), *AnimatedPropertyName.ToUpper()) != 0)
		{
			continue;
		}

		for (TWeakObjectPtr<>& WeakObject : BoundObjects)
		{
			UObject* FoundObject = WeakObject.Get();

			if (!FoundObject)
			{
				continue;
			}
			
			UObject* PropertyOwner = FoundObject;
			if (!FbxSetting.PropertyPath.ComponentName.IsEmpty())
			{
				PropertyOwner = FindObjectFast<UObject>(FoundObject, *FbxSetting.PropertyPath.ComponentName);
			}

			if (!PropertyOwner)
			{
				continue;
			}
		
			FGuid PropertyOwnerGuid = GetHandleToObject(PropertyOwner, InSequence, Player, TemplateID);
			if (!PropertyOwnerGuid.IsValid())
			{
				continue;
			}

			if (!PropertyOwnerGuid.IsValid())
			{
				continue;
			}

			bool bImported = false;
			switch (FbxSetting.PropertyType)
			{
				case EMovieSceneToolsPropertyTrackType::FloatTrack:
					bImported = ImportFBXPropertyToPropertyTrack<UMovieSceneFloatTrack, FMovieSceneFloatChannel>(MovieScene, PropertyOwnerGuid, FbxSetting, Source);
					break;
				case EMovieSceneToolsPropertyTrackType::DoubleTrack:
					bImported = ImportFBXPropertyToPropertyTrack<UMovieSceneDoubleTrack, FMovieSceneDoubleChannel>(MovieScene, PropertyOwnerGuid, FbxSetting, Source);
					break;
			}
			if (bImported)
			{
				return true;
			}
		}
	}
	return false;
}


void MovieSceneToolHelpers::LockCameraActorToViewport(const TSharedPtr<ISequencer>& Sequencer, ACameraActor* CameraActor)
{
	Sequencer->SetPerspectiveViewportCameraCutEnabled(false);

	// Lock the viewport to this camera
	if (GCurrentLevelEditingViewportClient && CameraActor && CameraActor->GetLevel())
	{
		GCurrentLevelEditingViewportClient->SetCinematicActorLock(nullptr);
		GCurrentLevelEditingViewportClient->SetActorLock(CameraActor);
		GCurrentLevelEditingViewportClient->bLockedCameraView = true;
		GCurrentLevelEditingViewportClient->UpdateViewForLockedActor();
		GCurrentLevelEditingViewportClient->Invalidate();
	}
}

void MovieSceneToolHelpers::CreateCameraCutSectionForCamera(UMovieScene* OwnerMovieScene, FGuid CameraGuid, FFrameNumber FrameNumber)
{
	// If there's a cinematic shot track, no need to set this camera to a shot
	UMovieSceneTrack* CinematicShotTrack = OwnerMovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
	if (CinematicShotTrack)
	{
		return;
	}

	UMovieSceneTrack* CameraCutTrack = OwnerMovieScene->GetCameraCutTrack();

	// If there's a camera cut track with at least one section, no need to change the section
	if (CameraCutTrack && CameraCutTrack->GetAllSections().Num() > 0)
	{
		return;
	}

	if (!CameraCutTrack)
	{
		CameraCutTrack = OwnerMovieScene->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
	}

	if (CameraCutTrack)
	{
		UMovieSceneSection* Section = MovieSceneHelpers::FindSectionAtTime(CameraCutTrack->GetAllSections(), FrameNumber);
		UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);

		if (CameraCutSection)
		{
			CameraCutSection->Modify();
			CameraCutSection->SetCameraGuid(CameraGuid);
		}
		else
		{
			CameraCutTrack->Modify();

			UMovieSceneCameraCutSection* NewSection = Cast<UMovieSceneCameraCutSection>(CameraCutTrack->CreateNewSection());
			NewSection->SetRange(OwnerMovieScene->GetPlaybackRange());
			NewSection->SetCameraGuid(CameraGuid);
			CameraCutTrack->AddSection(*NewSection);
		}
	}
}

template<typename ChannelType>
void ImportTransformChannelToBezierChannel(const FRichCurve& Source, ChannelType* Dest, FFrameRate DestFrameRate, bool bNegateTangents, bool bClearChannel,FFrameNumber StartFrame = 0, bool bNegateValue = false)
{
	// If there are no keys, don't clear the existing channel
	if (!Source.GetNumKeys())
	{
		return;
	}

	TMovieSceneChannelData<typename ChannelType::ChannelValueType> ChannelData = Dest->GetData();

	if (bClearChannel)
	{
		ChannelData.Reset();
	}
	for (auto SourceIt = Source.GetKeyHandleIterator(); SourceIt; ++SourceIt)
	{
		const FRichCurveKey Key = Source.GetKey(*SourceIt);
		float ArriveTangent = Key.ArriveTangent;
		float LeaveTangent = Key.LeaveTangent;
		if (bNegateTangents)
		{
			ArriveTangent = -ArriveTangent;
			LeaveTangent = -LeaveTangent;
		}

		FFrameNumber KeyTime = (Key.Time * DestFrameRate).RoundToFrame();
		float Value = !bNegateValue ? Key.Value : -Key.Value;
		MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime + StartFrame, Value, ArriveTangent, LeaveTangent,
			Key.InterpMode, Key.TangentMode, DestFrameRate, Key.TangentWeightMode,
			Key.ArriveTangentWeight, Key.LeaveTangentWeight);
	}

	Dest->AutoSetTangents();

	const UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetDefault<UMovieSceneUserImportFBXSettings>();
	if (ImportFBXSettings->bReduceKeys)
	{
		FKeyDataOptimizationParams Params;
		Params.Tolerance = ImportFBXSettings->ReduceKeysTolerance;
		Params.DisplayRate = DestFrameRate;
		Dest->Optimize(Params);
	}
}

void ImportTransformChannelToDouble(const FRichCurve& Source, FMovieSceneDoubleChannel* Dest, FFrameRate DestFrameRate, bool bNegateTangents, bool bClearChannel, FFrameNumber StartFrame = 0, bool bNegateValue = false)
{
	ImportTransformChannelToBezierChannel(Source, Dest, DestFrameRate, bNegateTangents, bClearChannel, StartFrame, bNegateValue);
}

void ImportTransformChannelToFloat(const FRichCurve& Source, FMovieSceneFloatChannel* Dest, FFrameRate DestFrameRate, bool bNegateTangents, bool bClearChannel, FFrameNumber StartFrame = 0, bool bNegateValue = false)
{
	ImportTransformChannelToBezierChannel(Source, Dest, DestFrameRate, bNegateTangents, bClearChannel, StartFrame, bNegateValue);
}

void ImportTransformChannelToBool(const FRichCurve& Source, FMovieSceneBoolChannel* Dest, FFrameRate DestFrameRate, bool bClearChannel, FFrameNumber StartFrame)
{
	// If there are no keys, don't clear the existing channel
	if (!Source.GetNumKeys())
	{
		return;
	}

	TMovieSceneChannelData<bool> ChannelData = Dest->GetData();

	if (bClearChannel)
	{
		ChannelData.Reset();
	}
	for (auto SourceIt = Source.GetKeyHandleIterator(); SourceIt; ++SourceIt)
	{
		const FRichCurveKey Key = Source.GetKey(*SourceIt);
		bool bValue = Key.Value != 0.0f ? true : false;

		FFrameNumber KeyTime = (Key.Time * DestFrameRate).RoundToFrame();

		KeyTime += StartFrame;
		if (ChannelData.FindKey(KeyTime) == INDEX_NONE)
		{
			ChannelData.AddKey(KeyTime, bValue);
		} //todo need to do a set here?
	}
}

void ImportTransformChannelToEnum(const FRichCurve& Source, FMovieSceneByteChannel* Dest, FFrameRate DestFrameRate, bool bClearChannel, FFrameNumber StartFrame)
{
	// If there are no keys, don't clear the existing channel
	if (!Source.GetNumKeys())
	{
		return;
	}

	TMovieSceneChannelData<uint8> ChannelData = Dest->GetData();

	if (bClearChannel)
	{
		ChannelData.Reset();
	}
	for (auto SourceIt = Source.GetKeyHandleIterator(); SourceIt; ++SourceIt)
	{
		const FRichCurveKey Key = Source.GetKey(*SourceIt);
		uint8 Value = (uint8)Key.Value;

		FFrameNumber KeyTime = (Key.Time * DestFrameRate).RoundToFrame();

		KeyTime += StartFrame;
		if (ChannelData.FindKey(KeyTime) == INDEX_NONE)
		{
			ChannelData.AddKey(KeyTime, Value);
		} //todo need to do a set here?
	}
}


void ImportTransformChannelToInteger(const FRichCurve& Source, FMovieSceneIntegerChannel* Dest, FFrameRate DestFrameRate, bool bClearChannel, FFrameNumber StartFrame)
{
	// If there are no keys, don't clear the existing channel
	if (!Source.GetNumKeys())
	{
		return;
	}

	TMovieSceneChannelData<int32> ChannelData = Dest->GetData();

	if (bClearChannel)
	{
		ChannelData.Reset();
	}
	for (auto SourceIt = Source.GetKeyHandleIterator(); SourceIt; ++SourceIt)
	{
		const FRichCurveKey Key = Source.GetKey(*SourceIt);
		int32 Value = (int32)Key.Value;

		FFrameNumber KeyTime = (Key.Time * DestFrameRate).RoundToFrame();

		KeyTime += StartFrame;
		if (ChannelData.FindKey(KeyTime) == INDEX_NONE)
		{
			ChannelData.AddKey(KeyTime, Value);
		} //todo need to do a set here?
	}
}

void SetChannelValue(FMovieSceneDoubleChannel* DoubleChannel, FMovieSceneFloatChannel* FloatChannel, FMovieSceneBoolChannel *BoolChannel, FMovieSceneByteChannel *EnumChannel, FMovieSceneIntegerChannel* IntegerChannel,
	const FFrameRate& FrameRate, const FFrameNumber& StartFrame,
	FControlRigChannelEnum ChannelEnum, UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
	const FTransform& DefaultTransform,
	const FRichCurve& TranslationX, const FRichCurve& TranslationY, const FRichCurve& TranslationZ,
	const FRichCurve& EulerRotationX, const FRichCurve& EulerRotationY, const FRichCurve& EulerRotationZ,
	const FRichCurve& ScaleX, const FRichCurve& ScaleY, const FRichCurve& ScaleZ)
{
	const FRichCurve* TransformCurves[9] = {&TranslationX, &TranslationY, &TranslationZ, &EulerRotationX, &EulerRotationY, &EulerRotationZ, &ScaleX, &ScaleY, &ScaleZ};
	
	FVector Location = DefaultTransform.GetLocation(), Rotation = DefaultTransform.GetRotation().Euler(), Scale3D = DefaultTransform.GetScale3D();
	const double DefaultTransformValues[9] = {Location[0], Location[1], Location[2], Rotation[0], Rotation[1], Rotation[2], Scale3D[0], Scale3D[1], Scale3D[2]};

	const FRichCurve *RichCurve = &TranslationX;
	float DefaultValue = Location.X;
	bool bNegate = false;

	uint8 ChannelIndex = 0; // Default Channel is TX
	
	if (const FControlToTransformMappings* ChannelMapping = ImportFBXControlRigSettings->ControlChannelMappings.FindByPredicate(
		[ChannelEnum](const FControlToTransformMappings& Mapping)	{ return ChannelEnum == Mapping.ControlChannel; }))
	{
		bNegate = ChannelMapping->bNegate;
		
		ChannelIndex = (uint8)ChannelMapping->FBXChannel;
	}
	else
	{
		const uint8 ChannelEnumIndex = (uint8)ChannelEnum;

		// 2DX, Bool and other default to TX
		if (ChannelEnum == FControlRigChannelEnum::Vector2DY)
		{
			ChannelIndex = 1;
		}
		else if (ChannelEnumIndex >= (uint8)FControlRigChannelEnum::PositionX && ChannelEnumIndex <= (uint8)FControlRigChannelEnum::ScaleZ)
		{
			ChannelIndex = ChannelEnumIndex - (uint8)FControlRigChannelEnum::PositionX;
		}
	}

	if (ChannelIndex < 9)
	{
		DefaultValue = DefaultTransformValues[ChannelIndex];
		RichCurve = TransformCurves[ChannelIndex];
	}
	
	if (ChannelEnum == FControlRigChannelEnum::Bool && BoolChannel)
	{
		const bool Default = DefaultValue != 0.0 ? !bNegate : bNegate;

		BoolChannel->SetDefault(Default);
		ImportTransformChannelToBool(*RichCurve, BoolChannel, FrameRate, false, StartFrame);
	}
	else if (ChannelEnum == FControlRigChannelEnum::Enum && EnumChannel)
	{
		const uint8 Default = (uint8)(bNegate ? -DefaultValue : DefaultValue);
		EnumChannel->SetDefault(Default);
		ImportTransformChannelToEnum(*RichCurve, EnumChannel, FrameRate, false, StartFrame);
	}
	else if (ChannelEnum == FControlRigChannelEnum::Integer && IntegerChannel)
	{
		const int32 Default = (int32)(bNegate ? -DefaultValue : DefaultValue);
		IntegerChannel->SetDefault(Default);
		ImportTransformChannelToInteger(*RichCurve, IntegerChannel, FrameRate, false, StartFrame);
	}
	else if (FloatChannel || DoubleChannel)
	{
		if (*RichCurve == TranslationY)
		{
			bNegate = !bNegate;
		}
		
		const float Default = bNegate ? -DefaultValue : DefaultValue;
		if (FloatChannel)
		{
			FloatChannel->SetDefault(Default);
			ImportTransformChannelToFloat(*RichCurve, FloatChannel, FrameRate, bNegate, false, StartFrame, bNegate);
		}
		else if (DoubleChannel)
		{
			DoubleChannel->SetDefault(Default);
			ImportTransformChannelToDouble(*RichCurve, DoubleChannel, FrameRate, bNegate, false, StartFrame, bNegate);
		}
	}
}

static bool ImportFBXNonTransformCurvesToChannels(const FString& NodeName, const UMovieSceneUserImportFBXSettings* ImportFBXSettings,UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
                                                  const FFrameNumber StartFrame, const FFrameRate FrameRate, FRigControlFBXNodeAndChannels& NodeAndChannels, UnFbx::FFbxCurvesAPI& CurveAPI)
{
	FbxNode* Node = CurveAPI.Scene->GetRootNode()->FindChild(TCHAR_TO_UTF8(*NodeName));
	if (!Node)
	{
		return false;
	}
	FRichCurve EmptyDefaultCurve;

	FbxProperty Property = Node->FindProperty(TCHAR_TO_UTF8(*NodeAndChannels.ControlName.ToString()));
	if (!Property.IsValid())
	{
		return false;
	}	
	EmptyDefaultCurve.SetDefaultValue(Property.Get<float>());

	TMap<FName, FRichCurve> Curves;
	CurveAPI.GetConvertedNonTransformCurveData(NodeName, true, ImportFBXSettings->ImportUniformScale, Curves);

	FRichCurve* Curve = Curves.Find(NodeAndChannels.ControlName);
	if (!Curve)
	{
		Curve = &EmptyDefaultCurve;
	}
	
	const float CurveDefault = Curve->GetDefaultValue();

	auto ShouldNegateChannel = [ImportFBXControlRigSettings](FControlRigChannelEnum ChannelType)
	{
		const FControlToTransformMappings* ControlMapping = ImportFBXControlRigSettings->ControlChannelMappings.FindByPredicate(
		[ChannelType](const FControlToTransformMappings& Mapping)
			{
				return Mapping.ControlChannel == ChannelType;
			}
		);
		return ControlMapping && ControlMapping->bNegate;
	};
	
	if (!NodeAndChannels.BoolChannels.IsEmpty())
	{
		const bool bNegate = ShouldNegateChannel(FControlRigChannelEnum::Bool);
		const bool ChannelDefault = bNegate ? CurveDefault == 0.0 : CurveDefault != 0.0;
		
		NodeAndChannels.BoolChannels[0]->SetDefault(ChannelDefault);
		ImportTransformChannelToBool(*Curve, NodeAndChannels.BoolChannels[0], FrameRate, false, StartFrame);
	}
	else if (!NodeAndChannels.EnumChannels.IsEmpty())
	{
		const bool bNegate = ShouldNegateChannel(FControlRigChannelEnum::Enum);
		const uint8 ChannelDefault = bNegate ? -(uint8)CurveDefault : (uint8)CurveDefault;
		
		NodeAndChannels.EnumChannels[0]->SetDefault(ChannelDefault);
		ImportTransformChannelToEnum(*Curve, NodeAndChannels.EnumChannels[0], FrameRate, false, StartFrame);
	}
	else if (!NodeAndChannels.IntegerChannels.IsEmpty())
	{
		const bool bNegate = ShouldNegateChannel(FControlRigChannelEnum::Integer);
		const int32 ChannelDefault = bNegate ? -(int32)CurveDefault : (int32)CurveDefault;
		NodeAndChannels.IntegerChannels[0]->SetDefault(ChannelDefault);
		
		ImportTransformChannelToInteger(*Curve, NodeAndChannels.IntegerChannels[0], FrameRate, false, StartFrame);
	}
	else if (!NodeAndChannels.FloatChannels.IsEmpty())
	{
		const bool bNegate = ShouldNegateChannel(FControlRigChannelEnum::Integer);
		const float ChannelDefault = bNegate ? -CurveDefault : CurveDefault;
		NodeAndChannels.FloatChannels[0]->SetDefault(ChannelDefault);
		
		ImportTransformChannelToFloat(*Curve, NodeAndChannels.FloatChannels[0], FrameRate, false, false, StartFrame, bNegate);
	}
	else
	{
		return false;	
	}
	return true;
}

//if one channel goes to Y
//if two channel go to X Y
//if three channel to to x y z
// if 9 due full
static bool ImportFBXTransformToChannels(FString NodeName, const UMovieSceneUserImportFBXSettings* ImportFBXSettings,UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
                                         const FFrameNumber StartFrame, const FFrameRate FrameRate, FRigControlFBXNodeAndChannels& NodeAndChannels, UnFbx::FFbxCurvesAPI& CurveAPI)
{
	TArray<FMovieSceneDoubleChannel*>& DoubleChannels = NodeAndChannels.DoubleChannels;
	TArray<FMovieSceneFloatChannel*>& FloatChannels = NodeAndChannels.FloatChannels;
	TArray<FMovieSceneBoolChannel*>& BoolChannels = NodeAndChannels.BoolChannels;
	TArray<FMovieSceneByteChannel*>& EnumChannels = NodeAndChannels.EnumChannels;
	TArray<FMovieSceneIntegerChannel*>& IntegerChannels = NodeAndChannels.IntegerChannels;
	
	// Look for transforms explicitly
	FRichCurve Transform[9];
	FTransform DefaultTransform;
	const bool bUseSequencerCurve = true;
	CurveAPI.GetConvertedTransformCurveData(NodeName, Transform[0], Transform[1], Transform[2], Transform[3], Transform[4], Transform[5], Transform[6], Transform[7], Transform[8], DefaultTransform, bUseSequencerCurve, ImportFBXSettings->ImportUniformScale);
	
	FVector Location = DefaultTransform.GetLocation(), Rotation = DefaultTransform.GetRotation().Euler(), Scale3D = DefaultTransform.GetScale3D();
	const double DefaultTransformValues[] = {Location[0], Location[1], Location[2], Rotation[0], Rotation[1], Rotation[2], Scale3D[0], Scale3D[1], Scale3D[2]};
	
		
	//For non-transforms we need to re-negate the Y since it happens automatically(todo double check.).
	//But then if we negate we need to re-re-negate... so leave it alone.

	if (BoolChannels.Num() == 1)
	{
		FControlRigChannelEnum Channel = FControlRigChannelEnum::Bool;
		SetChannelValue(nullptr, nullptr, BoolChannels[0], nullptr,nullptr, FrameRate, StartFrame,
			Channel, ImportFBXControlRigSettings, DefaultTransform,
			Transform[0], Transform[1], Transform[2], Transform[3],
			Transform[4], Transform[5], Transform[6], Transform[7], Transform[8]);
	}

	if (EnumChannels.Num() == 1)
	{
		FControlRigChannelEnum Channel = FControlRigChannelEnum::Enum;
		SetChannelValue(nullptr, nullptr, nullptr, EnumChannels[0], nullptr, FrameRate, StartFrame,
			Channel, ImportFBXControlRigSettings, DefaultTransform,
			Transform[0], Transform[1], Transform[2], Transform[3],
			Transform[4], Transform[5], Transform[6], Transform[7], Transform[8]);
	}

	if (IntegerChannels.Num() == 1)
	{
		FControlRigChannelEnum Channel = FControlRigChannelEnum::Integer;
		SetChannelValue(nullptr, nullptr, nullptr, nullptr, IntegerChannels[0], FrameRate, StartFrame,
			Channel,  ImportFBXControlRigSettings,DefaultTransform,
			Transform[0], Transform[1], Transform[2], Transform[3], 
			Transform[4], Transform[5], Transform[6], Transform[7], Transform[8]);
	}

	int ChannelEnumIndex = INDEX_NONE;

	if (FloatChannels.Num() == 1)
	{
		ChannelEnumIndex = (int)FControlRigChannelEnum::Float;
	}
	else if (FloatChannels.Num() == 2)
	{
		ChannelEnumIndex = (int)FControlRigChannelEnum::Vector2DX;
	}
	else if (FloatChannels.Num() == 3)
	{
		if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Position)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::PositionX;
		}
		else if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Rotator)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::RotatorX;
		}
		else if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Scale)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::ScaleX;
		}
	}
	if (ChannelEnumIndex != INDEX_NONE)
	{
		for (int i = 0; i < FloatChannels.Num(); i++)
		{
			const FControlRigChannelEnum ChannelEnum = (FControlRigChannelEnum)(ChannelEnumIndex + i);
			SetChannelValue(nullptr, FloatChannels[i], nullptr, nullptr, nullptr,
				FrameRate, StartFrame, ChannelEnum, ImportFBXControlRigSettings, DefaultTransform,
				Transform[0], Transform[1], Transform[2], Transform[3],
				Transform[4], Transform[5], Transform[6], Transform[7], Transform[8]);	
		}
	}
	else if (FloatChannels.Num() == 9 || FloatChannels.Num() == 6)
	{
		for (int i = 0; i < FloatChannels.Num(); i++)
		{
			FloatChannels[i]->SetDefault(DefaultTransformValues[i]);
			ChannelEnumIndex = (int)FControlRigChannelEnum::PositionX;
			bool bNegate = false;
			const FControlRigChannelEnum ChannelEnum = (FControlRigChannelEnum)(ChannelEnumIndex + i);
			if (const FControlToTransformMappings* ChannelMapping = ImportFBXControlRigSettings->ControlChannelMappings.FindByPredicate(
				[ChannelEnum](const FControlToTransformMappings& Mapping) { return ChannelEnum == Mapping.ControlChannel; }))
			{
				bNegate = ChannelMapping->bNegate;
			}
			ImportTransformChannelToFloat(Transform[i], FloatChannels[i], FrameRate, bNegate, false, StartFrame, bNegate);
		}
	}

	ChannelEnumIndex = INDEX_NONE;

	if (DoubleChannels.Num() == 1)
	{
		ChannelEnumIndex = (int)FControlRigChannelEnum::Float;
	}
	else if (DoubleChannels.Num() == 2)
	{
		ChannelEnumIndex = (int)FControlRigChannelEnum::Vector2DX;
	}
	else if (DoubleChannels.Num() == 3)
	{
		if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Position)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::PositionX;
		}
		else if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Rotator)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::RotatorX;
		}
		else if (NodeAndChannels.ControlType == FFBXControlRigTypeProxyEnum::Scale)
		{
			ChannelEnumIndex = (int)FControlRigChannelEnum::ScaleX;
		}
	}
	if (ChannelEnumIndex != INDEX_NONE)
	{
		for (int i = 0; i < DoubleChannels.Num(); i++)
		{
			const FControlRigChannelEnum ChannelEnum = (FControlRigChannelEnum)(ChannelEnumIndex + i);
			SetChannelValue(DoubleChannels[i], nullptr, nullptr, nullptr, nullptr,
				FrameRate, StartFrame, ChannelEnum, ImportFBXControlRigSettings, DefaultTransform,
				Transform[0], Transform[1], Transform[2], Transform[3],
				Transform[4], Transform[5], Transform[6], Transform[7], Transform[8]);	
		}
	}
	else if (DoubleChannels.Num() == 9 || DoubleChannels.Num() == 6)
	{
		for (int i = 0; i < DoubleChannels.Num(); i++)
		{
			DoubleChannels[i]->SetDefault(DefaultTransformValues[i]);
			ImportTransformChannelToDouble(Transform[i], DoubleChannels[i], FrameRate, false, false, StartFrame);
		}
	}
	return true;
}
static FString GetNewString(const FString& InString, UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings)
{
	FString NewString = InString;
	int Index = INDEX_NONE;
	
	if (ImportFBXControlRigSettings->bStripNamespace && InString.FindLastChar(':', Index))
	{
		NewString.RightChopInline(Index + 1);
	}
	for (const FControlFindReplaceString& FindReplace : ImportFBXControlRigSettings->FindAndReplaceStrings)
	{
		NewString.ReplaceInline(*FindReplace.Find, *FindReplace.Replace); //ignores tupe
	}
	return NewString;
}

static void PrepForInsertReplaceAnimation(bool bInsert, const FRigControlFBXNodeAndChannels& NodeAndChannel,
	FFrameNumber  FrameToInsertOrReplace, FFrameNumber  StartFrame, FFrameNumber  EndFrame)
{

	TArray<FMovieSceneChannel*> Channels;
	for (FMovieSceneDoubleChannel* DChannel : NodeAndChannel.DoubleChannels)
	{
		Channels.Add(DChannel);
	}
	for (FMovieSceneFloatChannel* FChannel : NodeAndChannel.FloatChannels)
	{
		Channels.Add(FChannel);
	}
	for (FMovieSceneBoolChannel* BChannel : NodeAndChannel.BoolChannels)
	{
		Channels.Add(BChannel);
	}
	for (FMovieSceneByteChannel* EChannel : NodeAndChannel.EnumChannels)
	{
		Channels.Add(EChannel);
	}
	for (FMovieSceneIntegerChannel* IChannel : NodeAndChannel.IntegerChannels)
	{
		Channels.Add(IChannel);
	}

	FFrameNumber Diff = EndFrame - StartFrame;
	FrameToInsertOrReplace += StartFrame;
	if (bInsert)
	{
		for (FMovieSceneChannel* Channel : Channels)
		{
			TArray<FFrameNumber> KeyTimes;
			TArray<FKeyHandle> Handles;
			Channel->GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			for (int32 Index = 0; Index < KeyTimes.Num(); Index++)
			{
				FFrameNumber FrameNumber = KeyTimes[Index];
				if (FrameNumber >= FrameToInsertOrReplace)
				{
					FrameNumber += Diff;
					KeyTimes[Index] += Diff;
				}

			}
			Channel->SetKeyTimes(Handles, KeyTimes);
		}
	}
	else //we replace the animation by first deleting keys in the interval
	{
		for (FMovieSceneChannel* Channel : Channels)
		{
			TArray<FFrameNumber> KeyTimes;
			TArray<FKeyHandle> Handles;
			Channel->GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			TArray<FKeyHandle> HandlesToDelete;
			for (int32 Index = 0; Index < KeyTimes.Num(); Index++)
			{
				FFrameNumber FrameNumber = KeyTimes[Index];
				if (FrameNumber >= FrameToInsertOrReplace && FrameNumber <= (FrameToInsertOrReplace + EndFrame))
				{
					HandlesToDelete.Add(Handles[Index]);
				}
			}
			Channel->DeleteKeys(HandlesToDelete);		
		}
	}
}

class SControlRigImportFBXSettings : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SControlRigImportFBXSettings) {}
	SLATE_ARGUMENT(FString, ImportFilename)
	SLATE_END_ARGS()
	~SControlRigImportFBXSettings()
	{
		if (NodeAndChannels != nullptr)
		{
			delete NodeAndChannels;
		}
	}
	void Construct(const FArguments& InArgs,  const TSharedRef<ISequencer>& InSequencer)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Import FBX Settings";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		Sequencer = InSequencer;

		DetailView->RegisterInstancedCustomPropertyTypeLayout("FrameNumber", FOnGetPropertyTypeCustomizationInstance::CreateSP(InSequencer, &ISequencer::MakeFrameNumberDetailsCustomization));


		ChildSlot
			[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SComboButton)
				.HasDownArrow(true)
				.OnGetMenuContent(this, &SControlRigImportFBXSettings::HandlePresetMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("MovieSceneTools", "ControlMappingPresets", "Control Mapping Presets"))
					.ToolTipText(NSLOCTEXT("MovieSceneTools", "SetControlMappingFromAPreset", "Set Control Mappings From A Preset"))
				]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
			.Text(NSLOCTEXT("MovieSceneTools", "ImportFBXButtonText", "Import"))
			.OnClicked(this, &SControlRigImportFBXSettings::OnImportFBXClicked)
			]

			];

		ImportFilename = InArgs._ImportFilename;
		NodeAndChannels = nullptr;
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		DetailView->SetObject(ImportFBXSettings);
	}
	TSharedRef<SWidget> HandlePresetMenuContent()
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("MovieSceneTools", "DefaultControlMappings", "Default Control Mappings"),
			NSLOCTEXT("MovieSceneTools", "DefaultControlMappings_Tooltip", "Use Default Control Mappings Preset"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SControlRigImportFBXSettings::SetPresets,false)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("MovieSceneTools", "MetaHumanControlMappings", "MetaHuman Control Mappings"),
			NSLOCTEXT("MovieSceneTools", "MetaHumanControlMappings_Tooltip", "Use MetaHuman Control Mappings Preset"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SControlRigImportFBXSettings::SetPresets,true)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		return MenuBuilder.MakeWidget();
	}

	void SetNodeNames(const TArray<FString>& NodeNames)
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		if (ImportFBXSettings)
		{
			ImportFBXSettings->ImportedNodeNames = NodeNames;
		}
	}
	void SetFrameRate(const FString& InFrameRate)
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		if (ImportFBXSettings)
		{
			ImportFBXSettings->ImportedFrameRate = InFrameRate;
		}
	}
	void SetStartTime(FFrameNumber StartTime)
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		if (ImportFBXSettings)
		{
			ImportFBXSettings->ImportedStartTime = StartTime;
			ImportFBXSettings->StartTimeRange = StartTime;
		}
	}
	void SetEndTime(FFrameNumber EndTime)
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		if (ImportFBXSettings)
		{
			ImportFBXSettings->ImportedEndTime = EndTime;
			ImportFBXSettings->EndTimeRange = EndTime;
		}
	}
	void SetFileName(const FString& FileName)
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		if (ImportFBXSettings)
		{
			ImportFBXSettings->ImportedFileName = FileName;
		}
	}
	void SetNodeAndChannels(TArray<FRigControlFBXNodeAndChannels>* InNodeAndChannels)
	{
		NodeAndChannels = InNodeAndChannels;
	}

private:

	FReply OnImportFBXClicked()
	{
		if (!Sequencer.IsValid() || !NodeAndChannels)
		{
			return  FReply::Unhandled();
		}

		UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		
		TArray<FName> SelectedControlNames;
		for (const FRigControlFBXNodeAndChannels& NodeAndChannel : *NodeAndChannels)
		{
			if (NodeAndChannel.MovieSceneTrack)
			{
				INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(NodeAndChannel.MovieSceneTrack);
				if (ChannelMapping)
				{
					TArray<FName> LocalControls;
					ChannelMapping->GetSelectedNodes(LocalControls);
					SelectedControlNames.Append(LocalControls);
				}
			}
		}

		const bool bValid = MovieSceneToolHelpers::ImportFBXIntoControlRigChannels(
			Sequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene(), ImportFilename, ImportFBXControlRigSettings, 
		    NodeAndChannels,SelectedControlNames, Sequencer.Pin()->GetFocusedTickResolution());
		
		const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		if (bValid)
		{
			Sequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}
		return bValid ? FReply::Handled() : FReply::Unhandled();
	}

	void SetPresets(const bool bMetaHuman) const
	{
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings = GetMutableDefault<UMovieSceneUserImportFBXControlRigSettings>();
		ImportFBXControlRigSettings->LoadControlMappingsFromPreset(bMetaHuman);
	}

	TSharedPtr<IDetailsView> DetailView;
	FString ImportFilename;
	TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels = nullptr;
	TWeakPtr<ISequencer> Sequencer;
};

class SControlRigExportFBXSettings : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SControlRigExportFBXSettings) {}
	SLATE_ARGUMENT(FString, ExportFilename)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,  const TSharedRef<ISequencer>& InSequencer, TWeakObjectPtr<UMovieSceneTrack> InTrack)
	{
		Sequencer = InSequencer;
		Track = InTrack;
		
		UMovieSceneUserExportFBXControlRigSettings* ExportFBXSettings = GetMutableDefault<UMovieSceneUserExportFBXControlRigSettings>();
		ExportFBXSettings->ExportFileName = InArgs._ExportFilename;

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Export FBX Settings";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);
		DetailView->SetObject(ExportFBXSettings);

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SComboButton)
				.HasDownArrow(true)
				.OnGetMenuContent(this, &SControlRigExportFBXSettings::HandlePresetMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("MovieSceneTools", "ControlMappingPresets", "Control Mapping Presets"))
					.ToolTipText(NSLOCTEXT("MovieSceneTools", "SetControlMappingFromAPreset", "Set Control Mappings From A Preset"))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
				.Text(NSLOCTEXT("MovieSceneTools", "ExportFBXButtonText", "Export"))
				.OnClicked(this, &SControlRigExportFBXSettings::OnExportFBXClicked)
			]
		];
	}
	TSharedRef<SWidget> HandlePresetMenuContent()
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("MovieSceneTools", "EmptyControlMappings", "Empty Control Mappings"),
			NSLOCTEXT("MovieSceneTools", "EmptyControlMappings_Tooltip",
				"Do not use Control Mappings. Bool, Enum, Int and Float channels will be exported as attributes of the corresponding data type"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SControlRigExportFBXSettings::ClearPresets)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
		
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("MovieSceneTools", "DefaultControlMappings", "Default Control Mappings"),
			NSLOCTEXT("MovieSceneTools", "DefaultControlMappings_Tooltip", "Use Default Control Mappings Preset"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SControlRigExportFBXSettings::SetPresets, false)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("MovieSceneTools", "MetaHumanControlMappings", "MetaHuman Control Mappings"),
			NSLOCTEXT("MovieSceneTools", "MetaHumanControlMappings_Tooltip", "Use MetaHuman Control Mappings Preset"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SControlRigExportFBXSettings::SetPresets, true)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		return MenuBuilder.MakeWidget();
	}

private:

	FReply OnExportFBXClicked()
	{
		if (!Sequencer.IsValid() || !Track.IsValid())
		{
			return  FReply::Unhandled();
		}

		UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings = GetMutableDefault<UMovieSceneUserExportFBXControlRigSettings>();
		
		TArray<FName> SelectedControlNames;
		if (Track.IsValid())
		{
			if (INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(Track.Get()))
			{
				ChannelMapping->GetSelectedNodes(SelectedControlNames);
			}
		}

		FMovieSceneSequenceTransform RootToLocalTransform;
		if (Sequencer.IsValid())
		{
			RootToLocalTransform = Sequencer.Pin()->GetFocusedMovieSceneSequenceTransformUnwarped();
		}

		const bool bValid = MovieSceneToolHelpers::ExportFBXFromControlRigChannels(
			Track.Get()->GetSectionToKey(), ExportFBXControlRigSettings, SelectedControlNames, RootToLocalTransform);

		if (bValid)
		{
			FNotificationInfo Info(NSLOCTEXT("MovieSceneTools", "ExportControlRigFBXSucceeded", "FBX Export Succeeded."));
			Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString InFilename) { FPlatformProcess::ExploreFolder(*InFilename); }, ExportFBXControlRigSettings->ExportFileName);
			Info.HyperlinkText = FText::FromString(ExportFBXControlRigSettings->ExportFileName);
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Success);
		}
		else
		{
			FNotificationInfo Info(NSLOCTEXT("MovieSceneTools", "ExportControlRigFBXFailed", "FBX Export Failed."));
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
		}
		
		const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return bValid ? FReply::Handled() : FReply::Unhandled();
	}

	void SetPresets(const bool bMetaHuman) const
	{
		UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings = GetMutableDefault<UMovieSceneUserExportFBXControlRigSettings>();
		ExportFBXControlRigSettings->LoadControlMappingsFromPreset(bMetaHuman);
	}

	void ClearPresets() const
	{
		UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings = GetMutableDefault<UMovieSceneUserExportFBXControlRigSettings>();
		ExportFBXControlRigSettings->ControlChannelMappings.Empty();
	}
	
	TSharedPtr<IDetailsView> DetailView;
	TWeakPtr<ISequencer> Sequencer;
	TWeakObjectPtr<UMovieSceneTrack> Track;
};

bool MovieSceneToolHelpers::ImportFBXIntoControlRigChannels(UMovieScene* MovieScene,const FString& ImportFilename, UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
	TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels, const TArray<FName>& SelectedControlNames, FFrameRate FrameRate)
{
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

	bool bValid = true;

	UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();
	bool bOldbConvertScene = ImportOptions->bConvertScene;
	bool bOldbConvertSceneUnit = ImportOptions->bConvertSceneUnit;
	bool bOldbForceFrontXAxis = ImportOptions->bForceFrontXAxis;
	float OldUniformScale = ImportOptions->ImportUniformScale;
	EFBXAnimationLengthImportType OldAnimLengthType = ImportOptions->AnimationLengthImportType;


	ImportOptions->bConvertScene = true;
	ImportOptions->bConvertSceneUnit = ImportFBXControlRigSettings->bConvertSceneUnit;
	ImportOptions->bForceFrontXAxis = ImportFBXControlRigSettings->bForceFrontXAxis;
	ImportOptions->ImportUniformScale = ImportFBXControlRigSettings->ImportUniformScale;
	ImportOptions->AnimationLengthImportType = FBXALIT_ExportedTime;

	const FString FileExtension = FPaths::GetExtension(ImportFilename);
	if (!FbxImporter->ImportFromFile(*ImportFilename, FileExtension, true))
	{
		// Log the error message and fail the import.
		FbxImporter->ReleaseScene();
		bValid = false;
	}
	else
	{
		const FScopedTransaction Transaction(NSLOCTEXT("MovieSceneTools", "ImportFBXControlRigTransaction", "Import FBX Onto Control Rig"));

		UMovieSceneUserImportFBXSettings* CurrentImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXSettings>();
		TArray<uint8> OriginalSettings;
		FObjectWriter ObjWriter(CurrentImportFBXSettings, OriginalSettings);

		CurrentImportFBXSettings->bMatchByNameOnly = false;
		CurrentImportFBXSettings->bConvertSceneUnit = ImportFBXControlRigSettings->bConvertSceneUnit;
		CurrentImportFBXSettings->bForceFrontXAxis = ImportFBXControlRigSettings->bForceFrontXAxis;
		CurrentImportFBXSettings->ImportUniformScale = ImportFBXControlRigSettings->ImportUniformScale;
		CurrentImportFBXSettings->bCreateCameras = false;
		CurrentImportFBXSettings->bReduceKeys = false;
		CurrentImportFBXSettings->ReduceKeysTolerance = 0.01f;

		UnFbx::FFbxCurvesAPI CurveAPI;
		FbxImporter->PopulateAnimatedCurveData(CurveAPI);
		TArray<FString> AllNodeNames;
		CurveAPI.GetAllNodeNameArray(AllNodeNames);

		//if matching selected remove out the non-selected
		if (ImportFBXControlRigSettings->bImportOntoSelectedControls)
		{
			NodeAndChannels->RemoveAll([SelectedControlNames](const FRigControlFBXNodeAndChannels& A)
				{ return !SelectedControlNames.Contains(A.ControlName); }
			);
		}

		FFrameNumber  FrameToInsertOrReplace = ImportFBXControlRigSettings->TimeToInsertOrReplaceAnimation;

		FFrameNumber  StartFrame = ImportFBXControlRigSettings->StartTimeRange;
		FFrameNumber  EndFrame = ImportFBXControlRigSettings->EndTimeRange;

		// We'll be looking for timecode properties on all of the nodes in the FBX being
		// considered for import, and the first one found will be used to populate the
		// timecode source of all modified movie scene sections.
		FName TCHourAttrName(TEXT("TCHour"));
		FName TCMinuteAttrName(TEXT("TCMinute"));
		FName TCSecondAttrName(TEXT("TCSecond"));
		FName TCFrameAttrName(TEXT("TCFrame"));

		if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
		{
			TCHourAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.HourAttributeName;
			TCMinuteAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.MinuteAttributeName;
			TCSecondAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SecondAttributeName;
			TCFrameAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.FrameAttributeName;
		}

		const TArray<FString> TimecodePropertyNames = {
			TCHourAttrName.ToString(),
			TCMinuteAttrName.ToString(),
			TCSecondAttrName.ToString(),
			TCFrameAttrName.ToString()
		};

		FTimecode Timecode;
		bool bFoundTimecode = false;

		TSet<UMovieSceneSection*> AllModifiedSections;

		for (const FString& NodeName : AllNodeNames)
		{
			const FString NewNodeName = GetNewString(*NodeName, ImportFBXControlRigSettings);

			TSet<UMovieSceneSection*> ModifiedSections;
			for (FRigControlFBXNodeAndChannels& NodeAndChannel : *NodeAndChannels)
			{
				if (NodeAndChannel.NodeName.Equals(NewNodeName, ESearchCase::IgnoreCase))
				{
					if (NodeAndChannel.MovieSceneTrack)
					{
						UMovieSceneSection* SectionToModify = NodeAndChannel.MovieSceneTrack->GetSectionToKey();
						if (!SectionToModify && NodeAndChannel.MovieSceneTrack->GetAllSections().Num() > 0)
						{
							SectionToModify = NodeAndChannel.MovieSceneTrack->GetAllSections()[0];
						}

						if (SectionToModify && !ModifiedSections.Contains(SectionToModify))
						{
							SectionToModify->SetFlags(RF_Transactional);
							SectionToModify->Modify();
							ModifiedSections.Add(SectionToModify);
							AllModifiedSections.Add(SectionToModify);
						}
					}

					PrepForInsertReplaceAnimation(ImportFBXControlRigSettings->bInsertAnimation, NodeAndChannel,
						FrameToInsertOrReplace,
						StartFrame, EndFrame);

					FFBXControlRigTypeProxyEnum ControlType = NodeAndChannel.ControlType;
					bool bTryImportNonTransformCurves = ControlType == FFBXControlRigTypeProxyEnum::Bool || ControlType == FFBXControlRigTypeProxyEnum::Float || ControlType == FFBXControlRigTypeProxyEnum::Integer;

					// If the node channels could not be imported from custom FBX attribute curves, try importing it them from the FBX transform channels, using the corresponding mapping 
					if (!bTryImportNonTransformCurves || !ImportFBXNonTransformCurvesToChannels(
						NodeName, CurrentImportFBXSettings, ImportFBXControlRigSettings, FrameToInsertOrReplace,
						FrameRate, NodeAndChannel, CurveAPI))
					{
						ImportFBXTransformToChannels(NodeName, CurrentImportFBXSettings, ImportFBXControlRigSettings, FrameToInsertOrReplace, FrameRate, NodeAndChannel, CurveAPI);
					}
				}
			}

			// We consider *all* nodes in the FBX when looking for authored timecode properties,
			// not just the ones that map to a particular node with channels.
			if (!bFoundTimecode)
			{
				TArray<FString> AnimatedPropertyNames;
				CurveAPI.GetNodeAnimatedPropertyNameArray(NodeName, AnimatedPropertyNames);
				for (const FString& AnimatedPropertyName : AnimatedPropertyNames)
				{
					if (!TimecodePropertyNames.Contains(AnimatedPropertyName))
					{
						continue;
					}

					bFoundTimecode = true;

					const int32 ChannelIndex = 0;
					const int32 CompositeIndex = 0;
					FRichCurve PropertyCurve;
					const bool bNegative = false;
					CurveAPI.GetCurveDataForSequencer(NodeName, AnimatedPropertyName, ChannelIndex, CompositeIndex, PropertyCurve, bNegative);

					const float EvalTime = 0.0f;
					const float DefaultValue = 0.0f;
					const float Value = PropertyCurve.Eval(EvalTime, DefaultValue);
					const int32 IntValue = FMath::TruncToInt(Value);

					const FName AnimatedPropertyNameAsFName(AnimatedPropertyName);

					if (AnimatedPropertyNameAsFName.IsEqual(TCHourAttrName))
					{
						Timecode.Hours = IntValue;
					}
					else if (AnimatedPropertyNameAsFName.IsEqual(TCMinuteAttrName))
					{
						Timecode.Minutes = IntValue;
					}
					else if (AnimatedPropertyNameAsFName.IsEqual(TCSecondAttrName))
					{
						Timecode.Seconds = IntValue;
					}
					else if (AnimatedPropertyNameAsFName.IsEqual(TCFrameAttrName))
					{
						Timecode.Frames = IntValue;
					}
				}
			}
		}

		if (bFoundTimecode)
		{
			for (UMovieSceneSection* Section : AllModifiedSections)
			{
				Section->TimecodeSource = FMovieSceneTimecodeSource(Timecode);
			}
		}

		// restore
		FObjectReader ObjReader(GetMutableDefault<UMovieSceneUserImportFBXSettings>(), OriginalSettings);
		FbxImporter->ReleaseScene();
	}

	ImportOptions->AnimationLengthImportType = OldAnimLengthType;
	ImportOptions->bConvertScene = bOldbConvertScene;
	ImportOptions->bConvertSceneUnit = bOldbConvertSceneUnit;
	ImportOptions->bForceFrontXAxis = bOldbForceFrontXAxis;
	ImportOptions->ImportUniformScale = OldUniformScale;;
	return bValid;
}

bool MovieSceneToolHelpers::ExportFBXFromControlRigChannels(const UMovieSceneSection* Section,
	const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings, const TArray<FName>& SelectedControlNames,
	const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (!Section)
	{
		return false;
	}

	UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();

	UFbxExportOption* ExportOptions = Exporter->GetExportOptions();
	ExportOptions->FbxExportCompatibility = ExportFBXControlRigSettings->FbxExportCompatibility;
	ExportOptions->bASCII = ExportFBXControlRigSettings->bASCII;
	ExportOptions->bForceFrontXAxis = ExportFBXControlRigSettings->bForceFrontXAxis;
	ExportOptions->bExportLocalTime = ExportFBXControlRigSettings->bExportLocalTime;
	
	Exporter->CreateDocument();
	Exporter->SetTransformBaking(true);
	Exporter->SetKeepHierarchy(true);

	TArray<FControlRigFbxNodeMapping> ChannelsMapping;

	// Map to make it easier to convert from Control Rig Channel Enum to Control Rig Type Proxy + Transform Channel Index
	const TMap<FControlRigChannelEnum, TPair<FFBXControlRigTypeProxyEnum, uint8>> ProxyMapping = {
		{ FControlRigChannelEnum::Vector2DX, { FFBXControlRigTypeProxyEnum::Vector2D, 0 } },
		{ FControlRigChannelEnum::Vector2DY, { FFBXControlRigTypeProxyEnum::Vector2D, 1 } },
		{ FControlRigChannelEnum::PositionX, { FFBXControlRigTypeProxyEnum::Position, 0 } },
		{ FControlRigChannelEnum::PositionY, { FFBXControlRigTypeProxyEnum::Position, 1 } },
		{ FControlRigChannelEnum::PositionZ, { FFBXControlRigTypeProxyEnum::Position, 2 } },
		{ FControlRigChannelEnum::RotatorX, { FFBXControlRigTypeProxyEnum::Rotator, 0 } },
		{ FControlRigChannelEnum::RotatorY, { FFBXControlRigTypeProxyEnum::Rotator, 1 } },
		{ FControlRigChannelEnum::RotatorZ, { FFBXControlRigTypeProxyEnum::Rotator, 2 } },
		{ FControlRigChannelEnum::ScaleX, { FFBXControlRigTypeProxyEnum::Scale, 0 } },
		{ FControlRigChannelEnum::ScaleY, { FFBXControlRigTypeProxyEnum::Scale, 1 } },
		{ FControlRigChannelEnum::ScaleZ, { FFBXControlRigTypeProxyEnum::Scale, 2 } }
	}; 

	for (const FControlToTransformMappings& ControlMapping : ExportFBXControlRigSettings->ControlChannelMappings)
	{
		FControlRigFbxNodeMapping ChannelMapping;
		
		if (ControlMapping.ControlChannel == FControlRigChannelEnum::Bool)
		{
			ChannelMapping.ChannelType = FMovieSceneBoolChannel::StaticStruct()->GetFName();
			ChannelMapping.ControlType = FFBXControlRigTypeProxyEnum::Bool;
		}
		else if (ControlMapping.ControlChannel == FControlRigChannelEnum::Enum)
		{
			ChannelMapping.ChannelType = FMovieSceneByteChannel::StaticStruct()->GetFName();
			ChannelMapping.ControlType = FFBXControlRigTypeProxyEnum::Integer;
		}
		else if (ControlMapping.ControlChannel == FControlRigChannelEnum::Integer)
		{
			ChannelMapping.ChannelType = FMovieSceneIntegerChannel::StaticStruct()->GetFName();
			ChannelMapping.ControlType = FFBXControlRigTypeProxyEnum::Integer;
		}
		else
		{
			ChannelMapping.ChannelType = FMovieSceneFloatChannel::StaticStruct()->GetFName();

			// Convert Control Channel Type to Control Type and Transform Component Index (0-2)
			if (const TPair<FFBXControlRigTypeProxyEnum, uint8>* Mapping = ProxyMapping.Find(ControlMapping.ControlChannel))
			{
				ChannelMapping.ControlType = Mapping->Key;
				ChannelMapping.ChannelAttrIndex = Mapping->Value;
			}
			else
			{
				ChannelMapping.ControlType = FFBXControlRigTypeProxyEnum::Float;
			}
		}

		// Convert FBXChannel Type to Transform Component Index (0-8)
		ChannelMapping.FbxAttrIndex = (uint8)ControlMapping.FBXChannel;
		
		ChannelMapping.bNegate = ControlMapping.bNegate;
		ChannelsMapping.Add(ChannelMapping);
	}

	// Filter only selected controls if required
	TArray<FName> ControlsFilter;
	if (ExportFBXControlRigSettings->bExportOnlySelectedControls)
	{
		ControlsFilter = SelectedControlNames;
	}

	Exporter->ExportControlRigSection(Section, ChannelsMapping, ControlsFilter, RootToLocalTransform);

	// Save to disk
	Exporter->WriteToFile(*ExportFBXControlRigSettings->ExportFileName);
	
	return true;
}

bool MovieSceneToolHelpers::ImportFBXIntoControlRigChannelsWithDialog(const TSharedRef<ISequencer>& InSequencer,TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels)
{
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("FBX (*.fbx)|*.fbx|");

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("MovieSceneTools", "ImportFBX", "Import FBX from...").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::FBX),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}
	if (!bOpen)
	{
		return false;
	}

	if (!OpenFilenames.Num())
	{
		return false;
	}
	
	const FText TitleText = NSLOCTEXT("MovieSceneTools", "ImportFBXTitleOnToControlRig", "Import FBX Onto Control Rig");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(500.f, 700.f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SControlRigImportFBXSettings> DialogWidget = SNew(SControlRigImportFBXSettings, InSequencer)
		.ImportFilename(OpenFilenames[0]);


	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();

	EFBXAnimationLengthImportType AnimLengthType = ImportOptions->AnimationLengthImportType;
	ImportOptions->AnimationLengthImportType = FBXALIT_ExportedTime;
	const FString FileExtension = FPaths::GetExtension(OpenFilenames[0]);
	if (!FbxImporter->ImportFromFile(*OpenFilenames[0], FileExtension, true))
	{
		ImportOptions->AnimationLengthImportType = AnimLengthType;
		if (NodeAndChannels)
		{
			delete NodeAndChannels;
		}
		FbxImporter->ReleaseScene();
		return false;
	}
	UnFbx::FFbxCurvesAPI CurveAPI;
	FbxImporter->PopulateAnimatedCurveData(CurveAPI);
	TArray<FString> AllNodeNames;
	CurveAPI.GetAllNodeNameArray(AllNodeNames);
	FbxAnimStack* AnimStack = FbxImporter->Scene->GetMember<FbxAnimStack>(0);

	FbxTimeSpan TimeSpan = FbxImporter->GetAnimationTimeSpan(FbxImporter->Scene->GetRootNode(), AnimStack);
	ImportOptions->AnimationLengthImportType = AnimLengthType;
	FbxImporter->ReleaseScene();
	DialogWidget->SetFileName(OpenFilenames[0]);
	FString FrameRateStr = FString::Printf(TEXT("%.2f"), FbxImporter->GetOriginalFbxFramerate());
	
	DialogWidget->SetFrameRate(FrameRateStr);
	FFrameRate FrameRate = InSequencer->GetFocusedTickResolution();
	FFrameNumber StartTime = FrameRate.AsFrameNumber(TimeSpan.GetStart().GetSecondDouble());
	FFrameNumber EndTime = FrameRate.AsFrameNumber(TimeSpan.GetStop().GetSecondDouble());
	DialogWidget->SetStartTime(StartTime);
	DialogWidget->SetEndTime(EndTime);
	DialogWidget->SetNodeNames(AllNodeNames);
	DialogWidget->SetNodeAndChannels(NodeAndChannels);
	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);

	return true;

}

bool MovieSceneToolHelpers::ExportFBXFromControlRigChannelsWithDialog(const TSharedRef<ISequencer>& InSequencer, UMovieSceneTrack* Track)
{
	FString ExportFilename;
		
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		TArray<FString> SavedFiles;

		const bool bFilePicked = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("MovieSceneTools", "ExportControlRigFBX", "Export Control Rig FBX" ).ToString(),
			*(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::FBX)),
			TEXT(""),
			TEXT("FBX document|*.fbx"),
			EFileDialogFlags::None,
			SavedFiles);

		if (!bFilePicked || SavedFiles.IsEmpty())
		{
			return false;
		}
		ExportFilename = SavedFiles[0];
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::FBX, FPaths::GetPath(ExportFilename)); 
	}

	const FText TitleText = NSLOCTEXT("MovieSceneTools", "ExportFBXFromControlRigTitle", "Export FBX from Control Rig");

	// Create the window to choose our options
	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(450.0f, 300.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	const TSharedRef<SControlRigExportFBXSettings> DialogWidget = SNew(SControlRigExportFBXSettings, InSequencer, Track)
		.ExportFilename(ExportFilename);

	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);

	return true;
}

double UnwindKeyValue(const double OldValue, double NewValue)
{
	while (NewValue - OldValue > 180.0f)
	{
		NewValue -= 360.0f;
	}
	while (NewValue - OldValue < -180.0f)
	{
		NewValue += 360.0f;
	}
	return NewValue;
}

void AddCorrectedKey(const FMovieSceneDoubleValue& CorrectedValue, const FFrameTime Time, TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData)
{
	const int32 KeyIndex = ChannelData.FindKey(Time.RoundToFrame());

	if (KeyIndex != INDEX_NONE)
	{
		FMovieSceneDoubleValue& KeyValue = ChannelData.GetValues()[KeyIndex];

		KeyValue.Value = CorrectedValue.Value;
		// Tangents need to be set to auto, so they can be corrected later, as their imported values may no longer be valid after correcting the transform
		if (KeyValue.TangentMode != RCTM_Auto && KeyValue.TangentMode != RCTM_SmartAuto)
		{
			KeyValue.TangentMode = RCTM_Auto;
		}
	}
	else
	{
		MovieSceneToolHelpers::SetOrAddKey(ChannelData, Time.RoundToFrame(), CorrectedValue);
	}
}

void CorrectForTransformOrigin(IMovieScenePlayer* Player, UMovieScene3DTransformSection* TransformSection, ISequencer* Sequencer)
{
	FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z")
	};

	FFrameRate FrameRate = TransformSection->GetTypedOuter<UMovieScene>()->GetTickResolution();

	// If importing from the Sequencer editor, retrieve the focused subsequence hierarchy, in order to get the subsequence transform origin overrides.
	// If not importing from the Sequencer editor, assume the transform is being imported onto the root sequence.
	TArray<FMovieSceneSequenceID> SubsequenceHierarchy = Sequencer ? Sequencer->GetSubSequenceHierarchy() : TArray { MovieSceneSequenceID::Root };

	UE::MovieScene::FSystemInterrogator Interrogator;
	
	const FMovieSceneRootEvaluationTemplateInstance& RootInstance = Player->GetEvaluationTemplate();
	const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());
	
	// Sub tracks can only be interrogated if the Sequencer and Hierarchy are valid (i.e. the import is triggered from the editor),
	// but the instance data can still be interrogated if they aren't, so this step can be safely skipped when importing from scripting.
	if (Sequencer && Hierarchy)
	{
		Interrogator.SetHierarchy(const_cast<FMovieSceneSequenceHierarchy*>(Hierarchy));

		FMovieSceneSequenceID OwningSequenceID = MovieSceneSequenceID::Root;
		for (const FMovieSceneSequenceID SubSequenceID : SubsequenceHierarchy)
		{
			if (const UMovieSceneSubSection* SubSection = Sequencer->FindSubSection(SubSequenceID))
			{
				if (UMovieSceneTrack* SubTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter()))
				{
					Interrogator.ImportTrack(SubTrack, UE::MovieScene::FInterrogationChannel::FromIndex(SubSection->GetRowIndex()), OwningSequenceID);
				}
			}
			OwningSequenceID = SubSequenceID;
		}
	}
	
	TSet<FFrameNumber> Times;

	for (int32 ChannelIndex = 0; ChannelIndex < 6; ++ChannelIndex)
	{
		TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannel = DoubleChannels[ChannelIndex];
		
		for (FFrameNumber Time : DoubleChannel.Get()->GetTimes())
		{
			Times.Add(Time);
		}
	}

	TArray<FFrameNumber> TimesElements = Times.Array();
	TimesElements.Sort();

	for (FFrameNumber Time : TimesElements)
	{
		Interrogator.AddInterrogation(Time);
	}
	
	const IMovieScenePlaybackClient*  Client = Player->GetPlaybackClient();
	const UObject* InstanceData = Client ? Client->GetInstanceData() : nullptr;
	
	TArray<FTransform> TransformOrigins;
	TArray<FTransform> CorrectedTransforms;

	Interrogator.Update();
	Interrogator.QueryTransformOrigins(TransformOrigins, SubsequenceHierarchy, InstanceData);
	
	// Gather the corrected transforms
	for (int32 TransformOriginIndex = 0; TransformOriginIndex < TransformOrigins.Num(); ++TransformOriginIndex)
	{
		FFrameTime Time = TimesElements[TransformOriginIndex];
		FTransform TransformOrigin = TransformOrigins[TransformOriginIndex];
		
		double LocationX = 0.0;
		double LocationY = 0.0;
		double LocationZ = 0.0;

		double RotationX = 0.0;
		double RotationY = 0.0;
		double RotationZ = 0.0;

		if (DoubleChannels[0].Get())
		{
			DoubleChannels[0].Get()->Evaluate(Time, LocationX);
		}

		if (DoubleChannels[1].Get())
		{
			DoubleChannels[1].Get()->Evaluate(Time, LocationY);
		}

		if (DoubleChannels[2].Get())
		{
			DoubleChannels[2].Get()->Evaluate(Time, LocationZ);
		}

		if (DoubleChannels[3].Get())
		{
			DoubleChannels[3].Get()->Evaluate(Time, RotationX);
		}

		if (DoubleChannels[4].Get())
		{
			DoubleChannels[4].Get()->Evaluate(Time, RotationY);
		}

		if (DoubleChannels[5].Get())
		{
			DoubleChannels[5].Get()->Evaluate(Time, RotationZ);
		}

		FVector EvaluatedLocation = FVector(LocationX, LocationY, LocationZ);
		FRotator EvaluatedRotation = FRotator(RotationY, RotationZ, RotationX);

		FTransform EvaluatedTransform = FTransform(EvaluatedRotation, EvaluatedLocation);

		EvaluatedTransform *= TransformOrigin.Inverse();

		CorrectedTransforms.Add(EvaluatedTransform);
	}

	TOptional<double> PreviousRotationX;
	TOptional<double> PreviousRotationY;
	TOptional<double> PreviousRotationZ;

	// Apply the correct transforms. This has to be done after gathering, so the corrected transforms don't interfere with evaluation.
	for (int32 CorrectedTransformIndex = 0; CorrectedTransformIndex < CorrectedTransforms.Num(); ++CorrectedTransformIndex)
	{
		FTransform CorrectedTransform = CorrectedTransforms[CorrectedTransformIndex];
		FFrameTime Time = TimesElements[CorrectedTransformIndex];
		
		if (DoubleChannels[0].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[0].Get()->GetData();
			AddCorrectedKey(FMovieSceneDoubleValue(CorrectedTransform.GetTranslation().X), Time, ChannelData);
		}

		if (DoubleChannels[1].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[1].Get()->GetData();
			AddCorrectedKey(FMovieSceneDoubleValue(CorrectedTransform.GetTranslation().Y), Time, ChannelData);
		}

		if (DoubleChannels[2].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[2].Get()->GetData();
			AddCorrectedKey(FMovieSceneDoubleValue(CorrectedTransform.GetTranslation().Z), Time, ChannelData);
		}

		if (DoubleChannels[3].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[3].Get()->GetData();
			FMovieSceneDoubleValue CorrectedValue = FMovieSceneDoubleValue(CorrectedTransform.GetRotation().Rotator().Roll);

			if (PreviousRotationX.IsSet())
			{
				CorrectedValue.Value = UnwindKeyValue(PreviousRotationX.GetValue(), CorrectedValue.Value);
			}

			PreviousRotationX = CorrectedValue.Value;

			AddCorrectedKey(CorrectedValue, Time, ChannelData);
		}

		if (DoubleChannels[4].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[4].Get()->GetData();
			FMovieSceneDoubleValue CorrectedValue = FMovieSceneDoubleValue(CorrectedTransform.GetRotation().Rotator().Pitch);

			if (PreviousRotationY.IsSet())
			{
				CorrectedValue.Value = UnwindKeyValue(PreviousRotationY.GetValue(), CorrectedValue.Value);
			}

			PreviousRotationY = CorrectedValue.Value;

			AddCorrectedKey(CorrectedValue, Time, ChannelData);
		}

		if (DoubleChannels[5].Get())
		{
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[5].Get()->GetData();
			FMovieSceneDoubleValue CorrectedValue = FMovieSceneDoubleValue(CorrectedTransform.GetRotation().Rotator().Yaw);

			if (PreviousRotationZ.IsSet())
			{
				CorrectedValue.Value = UnwindKeyValue(PreviousRotationZ.GetValue(), CorrectedValue.Value);
			}

			PreviousRotationZ = CorrectedValue.Value;

			AddCorrectedKey(CorrectedValue, Time, ChannelData);
		}
	}

	for (TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannel : DoubleChannels)
	{
		const UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetDefault<UMovieSceneUserImportFBXSettings>();
		if (ImportFBXSettings->bReduceKeys)
		{
			FKeyDataOptimizationParams Params;
			Params.Tolerance = ImportFBXSettings->ReduceKeysTolerance;
			Params.DisplayRate = FrameRate;
			DoubleChannel.Get()->Optimize(Params);
		}
		DoubleChannel.Get()->AutoSetTangents();
	}
}

bool ImportFBXTransform(FString NodeName, FGuid ObjectBinding, UnFbx::FFbxCurvesAPI& CurveAPI, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, ISequencer* Sequencer)
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	const UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetDefault<UMovieSceneUserImportFBXSettings>();

	// Look for transforms explicitly
	FRichCurve Translation[3];
	FRichCurve EulerRotation[3];
	FRichCurve Scale[3];
	FTransform DefaultTransform;
	const bool bUseSequencerCurve = true;
	CurveAPI.GetConvertedTransformCurveData(NodeName, Translation[0], Translation[1], Translation[2], EulerRotation[0], EulerRotation[1], EulerRotation[2], Scale[0], Scale[1], Scale[2], DefaultTransform, bUseSequencerCurve, ImportFBXSettings->ImportUniformScale);

 	UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectBinding); 
	if (!TransformTrack)
	{
		MovieScene->Modify();
		TransformTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(ObjectBinding);
	}
	TransformTrack->Modify();

	bool bSectionAdded = false;
	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->FindSection(0));
	if (TransformSection && !ImportFBXSettings->bReplaceTransformTrack)
	{
		TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());
		TransformSection->SetRowIndex(TransformTrack->GetMaxRowIndex()+1);
		TransformTrack->AddSection(*TransformSection);
		bSectionAdded = true;
	}
	else
	{
		TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->FindOrAddSection(0, bSectionAdded));
	}

	if (!TransformSection)
	{
		return false;
	}

	TransformSection->Modify();

	FFrameRate FrameRate = TransformSection->GetTypedOuter<UMovieScene>()->GetTickResolution();

	if (bSectionAdded)
	{
		TransformSection->SetRange(TRange<FFrameNumber>::All());
	}

	FVector Location = DefaultTransform.GetLocation(), Rotation = DefaultTransform.GetRotation().Euler(), Scale3D = DefaultTransform.GetScale3D();
	
	FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
	};

	if (DoubleChannels[0].Get())
	{
		DoubleChannels[0].Get()->SetDefault(Location.X);
		ImportTransformChannelToDouble(Translation[0], DoubleChannels[0].Get(), FrameRate, false, true);
	}

	if (DoubleChannels[1].Get())
	{
		DoubleChannels[1].Get()->SetDefault(Location.Y);
		ImportTransformChannelToDouble(Translation[1], DoubleChannels[1].Get(), FrameRate, false, true);
	}
	if (DoubleChannels[2].Get())
	{
		DoubleChannels[2].Get()->SetDefault(Location.Z);
		ImportTransformChannelToDouble(Translation[2], DoubleChannels[2].Get(), FrameRate, false, true);
	}

	if (DoubleChannels[3].Get())
	{
		DoubleChannels[3].Get()->SetDefault(Rotation.X);
		ImportTransformChannelToDouble(EulerRotation[0], DoubleChannels[3].Get(), FrameRate, false, true);
	}
	if (DoubleChannels[4].Get())
	{
		const bool bNegateTangents = ImportFBXSettings->bForceFrontXAxis;
		DoubleChannels[4].Get()->SetDefault(Rotation.Y);
		ImportTransformChannelToDouble(EulerRotation[1], DoubleChannels[4].Get(), FrameRate, bNegateTangents, true);
	}
	if (DoubleChannels[5].Get())
	{
		DoubleChannels[5].Get()->SetDefault(Rotation.Z);
		ImportTransformChannelToDouble(EulerRotation[2], DoubleChannels[5].Get(), FrameRate, false, true);
	}

	if (DoubleChannels[6].Get())
	{
		DoubleChannels[6].Get()->SetDefault(Scale3D.X);
		ImportTransformChannelToDouble(Scale[0], DoubleChannels[6].Get(), FrameRate, false, true);
	}
	if (DoubleChannels[7].Get())
	{
		DoubleChannels[7].Get()->SetDefault(Scale3D.Y);
		ImportTransformChannelToDouble(Scale[1], DoubleChannels[7].Get(), FrameRate, false, true);
	}
	if (DoubleChannels[8].Get())
	{
		DoubleChannels[8].Get()->SetDefault(Scale3D.Z);
		ImportTransformChannelToDouble(Scale[2], DoubleChannels[8].Get(), FrameRate, false, true);
	}

	if(ImportFBXSettings->bCorrectForTransformOrigin)
	{
		CorrectForTransformOrigin(Player, TransformSection, Sequencer);
	}

	return true;
}

bool MovieSceneToolHelpers::ImportFBXNode(FString NodeName, UnFbx::FFbxCurvesAPI& CurveAPI, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, FGuid ObjectBinding, ISequencer* Sequencer)
{
	// Look for animated float properties
	TArray<FString> AnimatedPropertyNames;
	CurveAPI.GetNodeAnimatedPropertyNameArray(NodeName, AnimatedPropertyNames);
		
	for (auto AnimatedPropertyName : AnimatedPropertyNames)
	{
		ImportFBXProperty(NodeName, AnimatedPropertyName, ObjectBinding, CurveAPI, InSequence, Player, TemplateID);
	}
	
	ImportFBXTransform(NodeName, ObjectBinding, CurveAPI, InSequence, Player, Sequencer);

	// Custom static string properties
	TArray<TPair<FString, FString> > CustomPropertyPairs;
	CurveAPI.GetCustomStringPropertyArray(NodeName, CustomPropertyPairs);

	for (TPair<FString, FString>& CustomProperty : CustomPropertyPairs)
	{
		FMovieSceneToolsModule::Get().ImportStringProperty(CustomProperty.Key, CustomProperty.Value, ObjectBinding, InSequence->GetMovieScene());
	}

	return true;
}

void MovieSceneToolHelpers::GetCameras( FbxNode* Parent, TArray<FbxCamera*>& Cameras )
{
	FbxCamera* Camera = Parent->GetCamera();
	if( Camera )
	{
		Cameras.Add(Camera);
	}

	int32 NodeCount = Parent->GetChildCount();
	for ( int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex )
	{
		FbxNode* Child = Parent->GetChild( NodeIndex );
		GetCameras(Child, Cameras);
	}
}

FbxCamera* FindCamera( FbxNode* Parent )
{
	FbxCamera* Camera = Parent->GetCamera();
	if( !Camera )
	{
		int32 NodeCount = Parent->GetChildCount();
		for ( int32 NodeIndex = 0; NodeIndex < NodeCount && !Camera; ++NodeIndex )
		{
			FbxNode* Child = Parent->GetChild( NodeIndex );
			Camera = Child->GetCamera();
		}
	}

	return Camera;
}

FbxNode* RetrieveObjectFromName(const TCHAR* ObjectName, FbxNode* Root)
{
	if (!Root)
	{
		return nullptr;
	}
	
	for (int32 ChildIndex=0;ChildIndex<Root->GetChildCount();++ChildIndex)
	{
		FbxNode* Node = Root->GetChild(ChildIndex);
		if (Node)
		{
			FString NodeName = FString(Node->GetName());

			if ( !FCString::Strcmp(ObjectName,UTF8_TO_TCHAR(Node->GetName())))
			{
				return Node;
			}

			if (FbxNode* NextNode = RetrieveObjectFromName(ObjectName,Node))
			{
				return NextNode;
			}
		}
	}

	return nullptr;
}

void MovieSceneToolHelpers::CopyCameraProperties(FbxCamera* CameraNode, AActor* InCameraActor)
{
	float FieldOfView;
	float FocalLength;

	if (CameraNode->GetApertureMode() == FbxCamera::eFocalLength)
	{
		FocalLength = CameraNode->FocalLength.Get();
		FieldOfView = CameraNode->ComputeFieldOfView(FocalLength);
	}
	else
	{
		FieldOfView = CameraNode->FieldOfView.Get();
		FocalLength = CameraNode->ComputeFocalLength(FieldOfView);
	}

	float ApertureWidth = CameraNode->GetApertureWidth();
	float ApertureHeight = CameraNode->GetApertureHeight();

	UCameraComponent* CameraComponent = nullptr;

	if (ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(InCameraActor))
	{
		CameraComponent = CineCameraActor->GetCineCameraComponent();

		UCineCameraComponent* CineCameraComponent = CineCameraActor->GetCineCameraComponent();
		CineCameraComponent->Filmback.SensorWidth = FUnitConversion::Convert(ApertureWidth, EUnit::Inches, EUnit::Millimeters);
		CineCameraComponent->Filmback.SensorHeight = FUnitConversion::Convert(ApertureHeight, EUnit::Inches, EUnit::Millimeters);
		CineCameraComponent->FocusSettings.ManualFocusDistance = CameraNode->FocusDistance;
		if (FocalLength < CineCameraComponent->LensSettings.MinFocalLength)
		{
			CineCameraComponent->LensSettings.MinFocalLength = FocalLength;
		}
		if (FocalLength > CineCameraComponent->LensSettings.MaxFocalLength)
		{
			CineCameraComponent->LensSettings.MaxFocalLength = FocalLength;
		}
		CineCameraComponent->CurrentFocalLength = FocalLength;
	}
	else if (ACameraActor* CameraActor = Cast<ACameraActor>(InCameraActor))
	{
		CameraComponent = CameraActor->GetCameraComponent();
	}

	if (!CameraComponent)
	{
		return;
	}

	CameraComponent->SetProjectionMode(CameraNode->ProjectionType.Get() == FbxCamera::ePerspective ? ECameraProjectionMode::Perspective : ECameraProjectionMode::Orthographic);
	CameraComponent->SetAspectRatio(CameraNode->AspectWidth.Get() / CameraNode->AspectHeight.Get());
	CameraComponent->SetOrthoNearClipPlane(CameraNode->NearPlane.Get());
	CameraComponent->SetOrthoFarClipPlane(CameraNode->FarPlane.Get());
	CameraComponent->SetOrthoWidth(CameraNode->OrthoZoom.Get());
	CameraComponent->SetFieldOfView(FieldOfView);
}

FString MovieSceneToolHelpers::GetCameraName(FbxCamera* InCamera)
{
	FbxNode* CameraNode = InCamera->GetNode();
	if (CameraNode)
	{
		return CameraNode->GetName();
	}

	return InCamera->GetName();
}


void MovieSceneToolHelpers::ImportFBXCameraToExisting(UnFbx::FFbxImporter* FbxImporter, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, TMap<FGuid, FString>& InObjectBindingMap, bool bMatchByNameOnly, bool bNotifySlate)
{
	if (FApp::IsUnattended() || GIsRunningUnattendedScript)
	{
		bNotifySlate = false;
	}

	UMovieScene* MovieScene = InSequence->GetMovieScene();

	for (auto InObjectBinding : InObjectBindingMap)
	{
		TArrayView<TWeakObjectPtr<>> BoundObjects = Player->FindBoundObjects(InObjectBinding.Key,TemplateID);

		FString ObjectName = InObjectBinding.Value;
		FbxCamera* CameraNode = nullptr;
		FbxNode* Node = RetrieveObjectFromName(*ObjectName, FbxImporter->Scene->GetRootNode());
		if (Node)
		{
			CameraNode = FindCamera(Node);
		}

		if (!CameraNode)
		{
			if (bMatchByNameOnly)
			{
				if (bNotifySlate)
				{
					FNotificationInfo Info(FText::Format(NSLOCTEXT("MovieSceneTools", "NoMatchingCameraError", "Failed to find any matching camera for {0}"), FText::FromString(ObjectName)));
					Info.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
				}
				continue;
			}

			CameraNode = FindCamera(FbxImporter->Scene->GetRootNode());
			if (CameraNode)
			{
				if (bNotifySlate)
				{
					FString CameraName = GetCameraName(CameraNode);
					FNotificationInfo Info(FText::Format(NSLOCTEXT("MovieSceneTools", "NoMatchingCameraWarning", "Failed to find any matching camera for {0}. Importing onto first camera from fbx {1}"), FText::FromString(ObjectName), FText::FromString(CameraName)));
					Info.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_None);
				}
			}
		}

		if (!CameraNode)
		{
			continue;
		}

		float FieldOfView;
		float FocalLength;

		if (CameraNode->GetApertureMode() == FbxCamera::eFocalLength)
		{
			FocalLength = CameraNode->FocalLength.Get();
			FieldOfView = CameraNode->ComputeFieldOfView(FocalLength);
		}
		else
		{
			FieldOfView = CameraNode->FieldOfView.Get();
			FocalLength = CameraNode->ComputeFocalLength(FieldOfView);
		}

		for (TWeakObjectPtr<>& WeakObject : BoundObjects)
		{
			UObject* FoundObject = WeakObject.Get();
			if (FoundObject && FoundObject->GetClass()->IsChildOf(ACameraActor::StaticClass()))
			{
				CopyCameraProperties(CameraNode, Cast<AActor>(FoundObject));

				UCameraComponent* CameraComponent = nullptr;
				FName TrackName;
				float TrackValue;

				if (ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(FoundObject))
				{
					CameraComponent = CineCameraActor->GetCineCameraComponent();
					TrackName = TEXT("CurrentFocalLength");
					TrackValue = FocalLength;
				}
				else if (ACameraActor* CameraActor = Cast<ACameraActor>(FoundObject))
				{
					CameraComponent = CameraActor->GetCameraComponent();
					TrackName = TEXT("FieldOfView");
					TrackValue = FieldOfView;
				}
				else
				{
					continue;
				}

				// Set the default value of the current focal length or field of view section
				//FGuid PropertyOwnerGuid = Player->GetHandleToObject(CameraComponent);
				FGuid PropertyOwnerGuid = GetHandleToObject(CameraComponent, InSequence, Player, TemplateID);

				if (!PropertyOwnerGuid.IsValid())
				{
					continue;
				}

				// If copying properties to a spawnable object, the template object must be updated
				MovieSceneHelpers::CopyObjectTemplate(InSequence, InObjectBinding.Key, FoundObject, Player->GetSharedPlaybackState());

				UMovieSceneFloatTrack* FloatTrack = MovieScene->FindTrack<UMovieSceneFloatTrack>(PropertyOwnerGuid, TrackName);
				if (FloatTrack)
				{
					FloatTrack->Modify();
					FloatTrack->RemoveAllAnimationData();

					bool bSectionAdded = false;
					UMovieSceneFloatSection* FloatSection = Cast<UMovieSceneFloatSection>(FloatTrack->FindOrAddSection(0, bSectionAdded));
					if (!FloatSection)
					{
						continue;
					}

					FloatSection->Modify();

					if (bSectionAdded)
					{
						FloatSection->SetRange(TRange<FFrameNumber>::All());
					}

					FloatSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0)->SetDefault(TrackValue);
				}
			}
		}
	}
}

void ImportFBXCamera(UnFbx::FFbxImporter* FbxImporter, UMovieSceneSequence* InSequence, ISequencer& InSequencer,  TMap<FGuid, FString>& InObjectBindingMap, bool bMatchByNameOnly, bool bCreateCameras)
{
	bool bNotifySlate = !FApp::IsUnattended() && !GIsRunningUnattendedScript;

	UMovieScene* MovieScene = InSequence->GetMovieScene();

	TArray<FbxCamera*> AllCameras;
	MovieSceneToolHelpers::GetCameras(FbxImporter->Scene->GetRootNode(), AllCameras);

	if (AllCameras.Num() == 0)
	{
		return;
	}

	if (bCreateCameras)
	{
		UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;

		// Find unmatched cameras
		TArray<FbxCamera*> UnmatchedCameras;
		for (auto Camera : AllCameras)
		{
			FString NodeName = MovieSceneToolHelpers::GetCameraName(Camera);

			bool bMatched = false;
			for (auto InObjectBinding : InObjectBindingMap)
			{		
				FString ObjectName = InObjectBinding.Value;
				if (ObjectName == NodeName)
				{
					// Look for a valid bound object, otherwise need to create a new camera and assign this binding to it
					bool bFoundBoundObject = false;
					TArrayView<TWeakObjectPtr<>> BoundObjects = InSequencer.FindBoundObjects(InObjectBinding.Key, InSequencer.GetFocusedTemplateID());
					for (auto BoundObject : BoundObjects)
					{
						if (BoundObject.IsValid())
						{
							bFoundBoundObject = true;
							break;
						}
					}

					if (!bFoundBoundObject)
					{
						if (bNotifySlate)
						{
							FNotificationInfo Info(FText::Format(NSLOCTEXT("MovieSceneTools", "NoBoundObjectsError", "Existing binding has no objects. Creating a new camera and binding for {0}"), FText::FromString(ObjectName)));
							Info.ExpireDuration = 5.0f;
							FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
						}
					}
				}
			}

			if (!bMatched)
			{
				UnmatchedCameras.Add(Camera);
			}
		}

		// If there are new cameras, clear the object binding map so that we're only assigning values to the newly created cameras
		if (UnmatchedCameras.Num() != 0)
		{
			InObjectBindingMap.Reset();
			bMatchByNameOnly = true;
		}

		// Add any unmatched cameras
		for (auto UnmatchedCamera : UnmatchedCameras)
		{
			FString CameraName = MovieSceneToolHelpers::GetCameraName(UnmatchedCamera);

			AActor* NewCamera = nullptr;
			if (UnmatchedCamera->GetApertureMode() == FbxCamera::eFocalLength)
			{
				FActorSpawnParameters SpawnParams;
				NewCamera = World->SpawnActor<ACineCameraActor>(SpawnParams);
				NewCamera->SetActorLabel(*CameraName);
			}
			else
			{
				FActorSpawnParameters SpawnParams;
				NewCamera = World->SpawnActor<ACameraActor>(SpawnParams);
				NewCamera->SetActorLabel(*CameraName);
			}

			// Copy camera properties before adding default tracks so that initial camera properties match and can be restored after sequencer finishes
			MovieSceneToolHelpers::CopyCameraProperties(UnmatchedCamera, NewCamera);

			TArray<TWeakObjectPtr<AActor> > NewCameras;
			NewCameras.Add(NewCamera);
			TArray<FGuid> NewCameraGuids = InSequencer.AddActors(NewCameras);

			if (NewCameraGuids.Num())
			{
				InObjectBindingMap.Add(NewCameraGuids[0]);
				InObjectBindingMap[NewCameraGuids[0]] = CameraName;
			}
		}
	}
	
	MovieSceneToolHelpers::ImportFBXCameraToExisting(FbxImporter, InSequence, &InSequencer, InSequencer.GetFocusedTemplateID(), InObjectBindingMap, bMatchByNameOnly, true);
}

FGuid FindCameraGuid(FbxCamera* Camera, TMap<FGuid, FString>& InObjectBindingMap)
{
	FString CameraName = MovieSceneToolHelpers::GetCameraName(Camera);

	for (auto& Pair : InObjectBindingMap)
	{
		if (Pair.Value == CameraName)
		{
			return Pair.Key;
		}
	}
	return FGuid();
}

UMovieSceneCameraCutTrack* GetCameraCutTrack(UMovieScene* InMovieScene)
{
	// Get the camera cut
	UMovieSceneTrack* CameraCutTrack = InMovieScene->GetCameraCutTrack();
	if (CameraCutTrack == nullptr)
	{
		InMovieScene->Modify();
		CameraCutTrack = InMovieScene->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
	}
	return CastChecked<UMovieSceneCameraCutTrack>(CameraCutTrack);
}

void ImportCameraCut(UnFbx::FFbxImporter* FbxImporter, UMovieScene* InMovieScene, TMap<FGuid, FString>& InObjectBindingMap)
{
	// Find a camera switcher
	FbxCameraSwitcher* CameraSwitcher = FbxImporter->Scene->GlobalCameraSettings().GetCameraSwitcher();
	if (CameraSwitcher == nullptr)
	{
		return;
	}
	// Get the animation layer
	FbxAnimStack* AnimStack = FbxImporter->Scene->GetMember<FbxAnimStack>(0);
	if (AnimStack == nullptr)
	{
		return;
	}
	FbxAnimLayer* AnimLayer = AnimStack->GetMember<FbxAnimLayer>(0);
	if (AnimLayer == nullptr)
	{
		return;
	}

	// The camera switcher camera index refer to depth-first found order of the camera in the FBX
	TArray<FbxCamera*> AllCameras;
	MovieSceneToolHelpers::GetCameras(FbxImporter->Scene->GetRootNode(), AllCameras);

	UMovieSceneCameraCutTrack* CameraCutTrack = GetCameraCutTrack(InMovieScene);
	FFrameRate FrameRate = CameraCutTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();

	FbxAnimCurve* AnimCurve = CameraSwitcher->CameraIndex.GetCurve(AnimLayer);
	if (AnimCurve)
	{
		for (int i = 0; i < AnimCurve->KeyGetCount(); ++i)
		{
			FbxAnimCurveKey key = AnimCurve->KeyGet(i);
			int value = (int)key.GetValue() - 1;
			if (value >= 0 && value < AllCameras.Num())
			{
				FGuid CameraGuid = FindCameraGuid(AllCameras[value], InObjectBindingMap);
				if (CameraGuid != FGuid())
				{
					CameraCutTrack->AddNewCameraCut(UE::MovieScene::FRelativeObjectBindingID(CameraGuid), (key.GetTime().GetSecondDouble() * FrameRate).RoundToFrame());
				}
			}
		}
	}
}

class SMovieSceneImportFBXSettings : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SMovieSceneImportFBXSettings) {}
		SLATE_ARGUMENT(FString, ImportFilename)
		SLATE_ARGUMENT(UMovieSceneSequence*, Sequence)
		SLATE_ARGUMENT(ISequencer*, Sequencer)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Import FBX Settings";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
				.Text(NSLOCTEXT("MovieSceneTools", "ImportFBXButtonText", "Import"))
				.OnClicked(this, &SMovieSceneImportFBXSettings::OnImportFBXClicked)
			]
			
		];

		ImportFilename = InArgs._ImportFilename;
		Sequence = InArgs._Sequence;
		Sequencer = InArgs._Sequencer;

		UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXSettings>();
		DetailView->SetObject(ImportFBXSettings);
	}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(Sequence);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("SMovieSceneImportFBXSettings");
	}

	void SetObjectBindingMap(const TMap<FGuid, FString>& InObjectBindingMap)
	{
		ObjectBindingMap = InObjectBindingMap;
	}

	void SetCreateCameras(TOptional<bool> bInCreateCameras)
	{
		bCreateCameras = bInCreateCameras;
	}

private:

	FReply OnImportFBXClicked()
	{
		
		UMovieSceneUserImportFBXSettings* ImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXSettings>();
		FEditorDirectories::Get().SetLastDirectory( ELastDirectory::FBX, FPaths::GetPath( ImportFilename ) ); // Save path as default for next time.

		if (!Sequence || !Sequence->GetMovieScene() || Sequence->GetMovieScene()->IsReadOnly())
		{
			return FReply::Unhandled();
		}

		FFBXInOutParameters InOutParams;
		if (!MovieSceneToolHelpers::ReadyFBXForImport(ImportFilename, ImportFBXSettings,InOutParams))
		{
			return FReply::Unhandled();
		}

		const FScopedTransaction Transaction(NSLOCTEXT("MovieSceneTools", "ImportFBXTransaction", "Import FBX"));
		UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

		bool bMatchByNameOnly = ImportFBXSettings->bMatchByNameOnly;
		if (ObjectBindingMap.Num() == 1 && bMatchByNameOnly)
		{
			UE_LOGF(LogMovieScene, Display, "Fbx Import: Importing onto one selected binding, disabling match by name only.");
			bMatchByNameOnly = false;
		}

		// Import static cameras first
		ImportFBXCamera(FbxImporter, Sequence, *Sequencer, ObjectBindingMap, bMatchByNameOnly, bCreateCameras.IsSet() ? bCreateCameras.GetValue() : ImportFBXSettings->bCreateCameras);

		UObject* PlaybackContext = Sequencer->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		bool bValid = MovieSceneToolHelpers::ImportFBXIfReady(World, Sequence, Sequencer, Sequencer->GetFocusedTemplateID(), ObjectBindingMap, ImportFBXSettings, InOutParams, Sequencer);
	
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if ( Window.IsValid() )
		{
			Window->RequestDestroyWindow();
		}

		return bValid ? FReply::Handled() : FReply::Unhandled();
	}

	TSharedPtr<IDetailsView> DetailView;
	FString ImportFilename;
	TObjectPtr<UMovieSceneSequence> Sequence;
	ISequencer* Sequencer;
	TMap<FGuid, FString> ObjectBindingMap;
	TOptional<bool> bCreateCameras;

};

bool MovieSceneToolHelpers::ReadyFBXForImport(const FString&  ImportFilename, UMovieSceneUserImportFBXSettings* ImportFBXSettings, FFBXInOutParameters& OutParams)
{
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

	UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();
	OutParams.bConvertSceneBackup = ImportOptions->bConvertScene;
	OutParams.bConvertSceneUnitBackup = ImportOptions->bConvertSceneUnit;
	OutParams.bForceFrontXAxisBackup = ImportOptions->bForceFrontXAxis;
	OutParams.ImportUniformScaleBackup = ImportOptions->ImportUniformScale;

	ImportOptions->bIsImportCancelable = false;
	ImportOptions->bConvertScene = true;
	ImportOptions->bConvertSceneUnit = ImportFBXSettings->bConvertSceneUnit;
	ImportOptions->bForceFrontXAxis = ImportFBXSettings->bForceFrontXAxis;
	ImportOptions->ImportUniformScale = ImportFBXSettings->ImportUniformScale;

	const FString FileExtension = FPaths::GetExtension(ImportFilename);
	if (!FbxImporter->ImportFromFile(*ImportFilename, FileExtension, true))
	{
		// Log the error message and fail the import.
		FbxImporter->ReleaseScene();
		ImportOptions->bConvertScene = OutParams.bConvertSceneBackup;
		ImportOptions->bConvertSceneUnit = OutParams.bConvertSceneUnitBackup;
		ImportOptions->bForceFrontXAxis = OutParams.bForceFrontXAxisBackup;
		ImportOptions->ImportUniformScale = OutParams.ImportUniformScaleBackup;
		return false;
	}
	return true;
}

bool ImportFBXOntoControlRigs(UWorld* World, UMovieScene* MovieScene, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID,
	TMap<FGuid, FString>& ObjectBindingMap, const TArray<FString>& ControRigControlNames , UMovieSceneUserImportFBXSettings* ImportFBXSettings,
	UMovieSceneUserImportFBXControlRigSettings*  Settings)
{
	UMovieSceneUserImportFBXSettings* CurrentImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXSettings>();
	TArray<uint8> OriginalSettings;
	FObjectWriter ObjWriter(CurrentImportFBXSettings, OriginalSettings);

	CurrentImportFBXSettings->bMatchByNameOnly = ImportFBXSettings->bMatchByNameOnly;
	CurrentImportFBXSettings->bForceFrontXAxis = ImportFBXSettings->bForceFrontXAxis;
	CurrentImportFBXSettings->bCreateCameras = ImportFBXSettings->bCreateCameras;
	CurrentImportFBXSettings->bReduceKeys = ImportFBXSettings->bReduceKeys;
	CurrentImportFBXSettings->ReduceKeysTolerance = ImportFBXSettings->ReduceKeysTolerance;
	CurrentImportFBXSettings->bConvertSceneUnit = ImportFBXSettings->bConvertSceneUnit;
	CurrentImportFBXSettings->ImportUniformScale = ImportFBXSettings->ImportUniformScale;


	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

	return true;
}

bool MovieSceneToolHelpers::ImportFBXIfReady(UWorld* World, UMovieSceneSequence* Sequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, TMap<FGuid, FString>& ObjectBindingMap, UMovieSceneUserImportFBXSettings* ImportFBXSettings,
	const FFBXInOutParameters& InParams, ISequencer* Sequencer)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	UMovieSceneUserImportFBXSettings* CurrentImportFBXSettings = GetMutableDefault<UMovieSceneUserImportFBXSettings>();
	TArray<uint8> OriginalSettings;
	FObjectWriter ObjWriter(CurrentImportFBXSettings, OriginalSettings);

	CurrentImportFBXSettings->bMatchByNameOnly = ImportFBXSettings->bMatchByNameOnly;
	CurrentImportFBXSettings->bForceFrontXAxis = ImportFBXSettings->bForceFrontXAxis;
	CurrentImportFBXSettings->bCreateCameras = ImportFBXSettings->bCreateCameras;
	CurrentImportFBXSettings->bReduceKeys = ImportFBXSettings->bReduceKeys;
	CurrentImportFBXSettings->ReduceKeysTolerance = ImportFBXSettings->ReduceKeysTolerance;
	CurrentImportFBXSettings->bConvertSceneUnit = ImportFBXSettings->bConvertSceneUnit;
	CurrentImportFBXSettings->ImportUniformScale = ImportFBXSettings->ImportUniformScale;
	CurrentImportFBXSettings->bReplaceTransformTrack = ImportFBXSettings->bReplaceTransformTrack;
	CurrentImportFBXSettings->bCorrectForTransformOrigin = ImportFBXSettings->bCorrectForTransformOrigin;
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

	UnFbx::FFbxCurvesAPI CurveAPI;
	FbxImporter->PopulateAnimatedCurveData(CurveAPI);
	TArray<FString> AllNodeNames;
	CurveAPI.GetAllNodeNameArray(AllNodeNames);

	// Import a camera cut track if cams were created, do it after populating curve data ensure only one animation layer, if any
	ImportCameraCut(FbxImporter, MovieScene, ObjectBindingMap);

	FString RootNodeName = FbxImporter->Scene->GetRootNode()->GetName();

	// First try matching by name
	for (int32 NodeIndex = 0; NodeIndex < AllNodeNames.Num(); )
	{
		FString NodeName = AllNodeNames[NodeIndex];
		if (RootNodeName == NodeName)
		{
			++NodeIndex;
			continue;
		}

		bool bFoundMatch = false;
		for (auto It = ObjectBindingMap.CreateIterator(); It; ++It)
		{
			if (FCString::Strcmp(*It.Value().ToUpper(), *NodeName.ToUpper()) == 0)
			{
				MovieSceneToolHelpers::ImportFBXNode(NodeName, CurveAPI, Sequence, Player, TemplateID, It.Key(), Sequencer);

				It.RemoveCurrent();
				AllNodeNames.RemoveAt(NodeIndex);

				bFoundMatch = true;
				break;
			}
		}

		if (bFoundMatch)
		{
			continue;
		}

		++NodeIndex;
	}

	// Otherwise, get the first available node that hasn't been imported onto yet
	if (!ImportFBXSettings->bMatchByNameOnly)
	{
		for (int32 NodeIndex = 0; NodeIndex < AllNodeNames.Num(); )
		{
			FString NodeName = AllNodeNames[NodeIndex];
			if (RootNodeName == NodeName)
			{
				++NodeIndex;
				continue;
			}

			auto It = ObjectBindingMap.CreateConstIterator();
			if (It)
			{
				MovieSceneToolHelpers::ImportFBXNode(NodeName, CurveAPI, Sequence, Player, TemplateID, It.Key(), Sequencer);

				UE_LOGF(LogMovieScene, Warning, "Fbx Import: Failed to find any matching node for (%ls). Defaulting to first available (%ls).", *NodeName, *It.Value());
				ObjectBindingMap.Remove(It.Key());
				AllNodeNames.RemoveAt(NodeIndex);
				continue;
			}

			++NodeIndex;
		}
	}

	for (FString NodeName : AllNodeNames)
	{
		UE_LOGF(LogMovieScene, Warning, "Fbx Import: Failed to find any matching node for (%ls).", *NodeName);
	}

	// restore
	FObjectReader ObjReader(GetMutableDefault<UMovieSceneUserImportFBXSettings>(), OriginalSettings);

	FbxImporter->ReleaseScene();
	UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();
	ImportOptions->bConvertScene = InParams.bConvertSceneBackup;
	ImportOptions->bConvertSceneUnit = InParams.bConvertSceneUnitBackup;
	ImportOptions->bForceFrontXAxis = InParams.bForceFrontXAxisBackup;
	ImportOptions->ImportUniformScale = InParams.ImportUniformScaleBackup;
	return true;
}

bool MovieSceneToolHelpers::ImportFBXWithDialog(UMovieSceneSequence* InSequence, ISequencer& InSequencer, const TMap<FGuid, FString>& InObjectBindingMap, TOptional<bool> bCreateCameras)
{
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("FBX (*.fbx)|*.fbx|");

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("MovieSceneTools", "ImportFBX", "Import FBX from...").ToString(), 
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::FBX),
			TEXT(""), 
			*ExtensionStr,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}
	if (!bOpen)
	{
		return false;
	}

	if (!OpenFilenames.Num())
	{
		return false;
	}

	const FText TitleText = NSLOCTEXT("MovieSceneTools", "ImportFBXTitle", "Import FBX");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(450.0f, 300.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SMovieSceneImportFBXSettings> DialogWidget = SNew(SMovieSceneImportFBXSettings)
		.ImportFilename(OpenFilenames[0])
		.Sequence(InSequence)
		.Sequencer(&InSequencer);
	DialogWidget->SetObjectBindingMap(InObjectBindingMap);
	DialogWidget->SetCreateCameras(bCreateCameras);
	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);

	return true;
}

bool MovieSceneToolHelpers::HasHiddenMobility(const UClass* ObjectClass)
{
	if (ObjectClass)
	{
		static const FName NAME_HideCategories(TEXT("HideCategories"));
		if (ObjectClass->HasMetaData(NAME_HideCategories))
		{
			if (ObjectClass->GetMetaData(NAME_HideCategories).Contains(TEXT("Mobility")))
			{
				return true;
			}
		}
	}

	return false;
}

const FMovieSceneEvaluationTrack* MovieSceneToolHelpers::GetEvaluationTrack(ISequencer *Sequencer, const FGuid& TrackSignature)
{
	FMovieSceneRootEvaluationTemplateInstance& Instance = Sequencer->GetEvaluationTemplate();
	FMovieSceneCompiledDataID SubDataID = Instance.GetCompiledDataManager()->GetSubDataID(Instance.GetCompiledDataID(), Sequencer->GetFocusedTemplateID());

	{
		const FMovieSceneEvaluationTemplate* Template  = SubDataID.IsValid() ? Instance.GetCompiledDataManager()->FindTrackTemplate(SubDataID) : nullptr;
		const FMovieSceneEvaluationTrack*    EvalTrack = Template ? Template->FindTrack(TrackSignature) : nullptr;
		if (EvalTrack)
		{
			return EvalTrack;
		}
	}
	return nullptr;
}

void ExportLevelMesh(UnFbx::FFbxExporter* Exporter, ULevel* Level, IMovieScenePlayer* Player, const TArray<FGuid>& Bindings, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef& Template)
{
	// Get list of actors based upon bindings...
	const bool bSelectedOnly = (Bindings.Num()) != 0;

	const bool bSaveAnimSeq = false; //force off saving any AnimSequences since this can conflict when we export the level sequence animations.

	TArray<AActor*> ActorToExport;

	int32 ActorCount = Level->Actors.Num();
	for (int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex)
	{
		AActor* Actor = Level->Actors[ActorIndex];
		if (Actor != NULL)
		{
			FGuid ExistingGuid = Player->FindObjectId(*Actor, Template);
			if (ExistingGuid.IsValid() && (!bSelectedOnly || Bindings.Contains(ExistingGuid)))
			{
				ActorToExport.Add(Actor);
			}
		}
	}

	// Export the persistent level and all of it's actors
	Exporter->ExportLevelMesh(Level, !bSelectedOnly, ActorToExport, NodeNameAdapter, bSaveAnimSeq);
}

//5.5 deprecrated
bool MovieSceneToolHelpers::ExportFBX(UWorld* World, UMovieScene* MovieScene, IMovieScenePlayer* Player, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef& Template, const FString& InFBXFileName, FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (MovieScene)
	{
		if (UMovieSceneSequence* MovieSceneSequence = MovieScene->GetTypedOuter< UMovieSceneSequence>())
		{
			FAnimExportSequenceParameters AESP;
			AESP.MovieSceneSequence = MovieSceneSequence;
			AESP.RootMovieSceneSequence = MovieSceneSequence;
			AESP.Player = Player;
			AESP.RootToLocalTransform = RootToLocalTransform;
			return MovieSceneToolHelpers::ExportFBX(World, AESP, Bindings, Tracks, NodeNameAdapter, Template, InFBXFileName);
		}
	}
	return false;
}

bool MovieSceneToolHelpers::ExportFBX(UWorld* World, const FAnimExportSequenceParameters& AESP, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef& Template, const FString& InFBXFileName)
{
	UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();

	Exporter->CreateDocument();
	Exporter->SetTransformBaking(false);
	Exporter->SetKeepHierarchy(true);

	if (World)
	{
		ExportLevelMesh(Exporter, World->PersistentLevel, AESP.Player, Bindings, NodeNameAdapter, Template);

		// Export streaming levels and actors
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel)
			{
				if (ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					ExportLevelMesh(Exporter, Level, AESP.Player, Bindings, NodeNameAdapter, Template);
				}
			}
		}
	}

	Exporter->ExportLevelSequence(AESP.MovieSceneSequence, AESP.RootMovieSceneSequence, Bindings, AESP.Player, NodeNameAdapter, Template, AESP.RootToLocalTransform);

	//Export given tracks
	Exporter->ExportLevelSequenceTracks(AESP.MovieSceneSequence, AESP.RootMovieSceneSequence, AESP.Player, Template, nullptr, nullptr, Tracks, AESP.RootToLocalTransform);

	// Save to disk
	Exporter->WriteToFile(*InFBXFileName);

	return true;
}
static void TickLiveLink(ILiveLinkClient* LiveLinkClient, TMap<FGuid, ELiveLinkSourceMode>&  SourceAndMode)
{

	//This first bit lookes for a Sequencer Live Link Source which can show up any frame and we need to set it to Latest mode
	if (LiveLinkClient)
	{
		TArray<FGuid> Sources = LiveLinkClient->GetSources();
		for (const FGuid& Guid : Sources)
		{
			FText SourceTypeText = LiveLinkClient->GetSourceType(Guid);
			FString SourceTypeStr = SourceTypeText.ToString();
			if (SourceTypeStr.Contains(TEXT("Sequencer Live Link")))
			{
				ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Guid);
				if (Settings)
				{
					if (Settings->Mode != ELiveLinkSourceMode::Latest)
					{
						SourceAndMode.Add(Guid, Settings->Mode);
						Settings->Mode = ELiveLinkSourceMode::Latest;
					}
				}
			}
		}
	
		LiveLinkClient->ForceTick();
	}
}

static void TickFrame(const FFrameNumber& FrameNumber,float DeltaTime, UMovieScene* MovieScene, UnFbx::FLevelSequenceAnimTrackAdapter& AnimTrackAdapter,
	const TArray<IMovieSceneToolsAnimationBakeHelper*>& BakeHelpers, const TArray< USkeletalMeshComponent*>& SkelMeshComps, 
	ILiveLinkClient* LiveLinkClient, TMap<FGuid, ELiveLinkSourceMode>& SourceAndMode)
{
	//Begin records a frame so need to set things up first
	for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
	{
		if (BakeHelper)
		{
			BakeHelper->PreEvaluation(MovieScene, FrameNumber);
		}
	}
	// This will call UpdateSkelPose on the skeletal mesh component to move bones based on animations in the sequence
	int32 Index = FrameNumber.Value;
	AnimTrackAdapter.UpdateAnimation(Index);

	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	if (World)
	{
		const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.EvaluateAllConstraints();
	}

	for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
	{
		if (BakeHelper)
		{
			BakeHelper->PostEvaluation(MovieScene, FrameNumber);
		}
	}
	//Live Link source can show up at any time so we unfortunately need to check for it
	TickLiveLink(LiveLinkClient, SourceAndMode);

	// Update space bases so new animation position has an effect.
	for (USkeletalMeshComponent* SkelMeshComp : SkelMeshComps)
	{
		SkelMeshComp->SetForcedLOD(1); //we need to do this every tick since a restore state may happen while evaluating and bummp us out ot a different LOD
		SkelMeshComp->TickAnimation(DeltaTime, false);

		SkelMeshComp->RefreshBoneTransforms();
		SkelMeshComp->RefreshFollowerComponents();
		SkelMeshComp->UpdateComponentToWorld();
		SkelMeshComp->FinalizeBoneTransform();
		SkelMeshComp->MarkRenderTransformDirty();
		SkelMeshComp->MarkRenderDynamicDataDirty();
	}

}
bool MovieSceneToolHelpers::BakeToSkelMeshToCallbacks(UMovieScene* MovieScene, IMovieScenePlayer* Player,
	USkeletalMeshComponent* InSkelMeshComp, FMovieSceneSequenceIDRef& Template, FMovieSceneSequenceTransform& RootToLocalTransform, UAnimSeqExportOption* ExportOptions,
	FInitAnimationCB InitCallback, FStartAnimationCB StartCallback, FTickAnimationCB TickCallback, FEndAnimationCB EndCallback)
{
	return false;
}


bool MovieSceneToolHelpers::BakeToSkelMeshToCallbacks(const FAnimExportSequenceParameters& AESP, USkeletalMeshComponent* InSkelMeshComp, UAnimSeqExportOption* ExportOptions,
	FInitAnimationCB InitCallback, FStartAnimationCB StartCallback, FTickAnimationCB TickCallback, FEndAnimationCB EndCallback)
{
	UMovieScene* MovieScene = AESP.MovieSceneSequence->GetMovieScene();
	TArray< USkeletalMeshComponent*> SkelMeshComps;
	if (!ExportOptions)
	{
		UE_LOGF(LogMovieScene, Warning, " MovieSceneToolHelpers::BakeToSkelMesh functions require a valid AnimSeqExportOption");
		return false;
	}
	if (ExportOptions->bEvaluateAllSkeletalMeshComponents)
	{
		AActor* Actor = InSkelMeshComp->GetTypedOuter<AActor>();
		if (Actor)
		{
			Actor->GetComponents(SkelMeshComps, false);
		}
	}
	else
	{
		SkelMeshComps.Add(InSkelMeshComp);
	}
	//if we have no allocated bone space transforms something wrong so try to recalc them,only need to do this on the recorded skelmesh
	if (InSkelMeshComp->GetBoneSpaceTransforms().Num() <= 0)
	{
		InSkelMeshComp->RecalcRequiredBones(0);
		if (InSkelMeshComp->GetBoneSpaceTransforms().Num() <= 0)
		{
			UE_LOGF(LogMovieScene, Error, "Error Ba");
			return false;
		}
	}

	UnFbx::FLevelSequenceAnimTrackAdapter::FAnimTrackSettings AnimTrackSettings;
	AnimTrackSettings.MovieScenePlayer = AESP.Player;
	AnimTrackSettings.MovieSceneSequence = AESP.MovieSceneSequence;
	AnimTrackSettings.RootMovieSceneSequence = AESP.RootMovieSceneSequence;
	AnimTrackSettings.RootToLocalTransform = AESP.RootToLocalTransform;
	AnimTrackSettings.bForceUseOfMovieScenePlaybackRange = AESP.bForceUseOfMovieScenePlaybackRange;

	UnFbx::FLevelSequenceAnimTrackAdapter AnimTrackAdapter(AnimTrackSettings);
	if (ExportOptions->bUseCustomTimeRange)
	{
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayResolution = ExportOptions->CustomDisplayRate;
		const FFrameNumber StartFrameInTick = FFrameRate::TransformTime(FFrameTime(ExportOptions->CustomStartFrame), DisplayResolution, TickResolution).FloorToFrame();
		FFrameNumber EndFrameInTick = FFrameRate::TransformTime(FFrameTime(ExportOptions->CustomEndFrame), DisplayResolution, TickResolution).CeilToFrame();
		if (EndFrameInTick < StartFrameInTick)
		{
			EndFrameInTick = StartFrameInTick;
		}
		AnimTrackAdapter.SetRange(StartFrameInTick,EndFrameInTick);
	}
	if (ExportOptions->bUseCustomFrameRate)
	{
		AnimTrackAdapter.SetFrameRate(ExportOptions->CustomFrameRate);
	}

	int32 LocalStartFrame = AnimTrackAdapter.GetLocalStartFrame();
	int32 AnimationLength = AnimTrackAdapter.GetLength();
	float FrameRate = AnimTrackAdapter.GetFrameRate();
	float DeltaTime = 1.0f / FrameRate;

	//If we are running with a live link track we need to do a few things.
	// 1. First test to see if we have one, only way to really do that is to see if we have a source that has the `Sequencer Live Link Track`.  We also evalute the first frame in case we are out of range and the sources aren't created yet.
	// 2. Make sure Sequencer.AlwaysSendInterpolated.LiveLink is non-zero, and then set it back to zero if it's not.
	// 3. For each live link sequencer source we need to set the ELiveLinkSourceMode to Latest so that we just get the latest and don't use engine/timecode for any interpolation.
	ILiveLinkClient* LiveLinkClient = nullptr;
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	TMap<FGuid, ELiveLinkSourceMode> SourceAndMode;
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		LiveLinkClient = &ModularFeatures.GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	}
	TOptional<int32> SequencerAlwaysSenedLiveLinkInterpolated;
	IConsoleVariable* CVarAlwaysSendInterpolatedLiveLink = IConsoleManager::Get().FindConsoleVariable(TEXT("Sequencer.AlwaysSendInterpolatedLiveLink"));
	if (CVarAlwaysSendInterpolatedLiveLink)
	{
		SequencerAlwaysSenedLiveLinkInterpolated = CVarAlwaysSendInterpolatedLiveLink->GetInt();
		CVarAlwaysSendInterpolatedLiveLink->Set(1, ECVF_SetByConsole);
	}

	const TArray<IMovieSceneToolsAnimationBakeHelper*>&  BakeHelpers = FMovieSceneToolsModule::Get().GetAnimationBakeHelpers();
	for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
	{
		if (BakeHelper)
		{
			BakeHelper->StartBaking(MovieScene);
		}
	}

	InitCallback.ExecuteIfBound();
	//if we have delay frames run them first at the LocalStartFrame - ExportOptions->WarmupFrames;
	if (ExportOptions->DelayBeforeStart > 0)
	{
		FFrameNumber FrameNumber = FFrameNumber(LocalStartFrame) - ExportOptions->WarmUpFrames;
		for (int32 Index = 0; Index < ExportOptions->DelayBeforeStart; ++Index)
		{
			TickFrame(FrameNumber, DeltaTime, MovieScene, AnimTrackAdapter, BakeHelpers, SkelMeshComps, LiveLinkClient, SourceAndMode);
		}

	}
	//if we have warmup frames
	if (ExportOptions->WarmUpFrames > 0)
	{
		for (int32 Index = -ExportOptions->WarmUpFrames.Value; Index < 0; ++Index)
		{
			FFrameNumber FrameNumber = FFrameNumber(LocalStartFrame) - FFrameNumber(Index);
			TickFrame(FrameNumber, DeltaTime, MovieScene, AnimTrackAdapter, BakeHelpers, SkelMeshComps, LiveLinkClient, SourceAndMode);
		}
	}
	
	//BeginRecording, which happens in the Start Callback below, records a frame so need to set things up first at first frame, even if we had any delay or warmup
	TickFrame(LocalStartFrame, DeltaTime, MovieScene, AnimTrackAdapter, BakeHelpers, SkelMeshComps, LiveLinkClient, SourceAndMode);
	StartCallback.ExecuteIfBound(LocalStartFrame);

	for (int32 FrameCount = 1; FrameCount <= AnimationLength; ++FrameCount)
	{
		int32 LocalIndex = LocalStartFrame + FrameCount;
		FFrameNumber LocalFrame(LocalIndex);
		TickFrame(LocalFrame, DeltaTime, MovieScene, AnimTrackAdapter, BakeHelpers, SkelMeshComps, LiveLinkClient, SourceAndMode);
		TickCallback.ExecuteIfBound(DeltaTime, LocalFrame);
	}

	for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
	{
		if (BakeHelper)
		{
			BakeHelper->StopBaking(MovieScene);
		}
	}
	EndCallback.ExecuteIfBound();

	//now do any sequencer live link cleanup
	if (LiveLinkClient)
	{
		for (TPair<FGuid, ELiveLinkSourceMode>& Item : SourceAndMode)
		{
			ULiveLinkSourceSettings* Settings = LiveLinkClient->GetSourceSettings(Item.Key);
			if (Settings)
			{
				Settings->Mode = Item.Value;
			}
		}
	}

	if (SequencerAlwaysSenedLiveLinkInterpolated.IsSet() && CVarAlwaysSendInterpolatedLiveLink)
	{
		CVarAlwaysSendInterpolatedLiveLink->Set(0, ECVF_SetByConsole);
	}
	return true;
}

//5.5 deprecated
bool MovieSceneToolHelpers::ExportToAnimSequence(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, UMovieScene* MovieScene, IMovieScenePlayer* Player,
	USkeletalMeshComponent* SkelMesh, FMovieSceneSequenceIDRef& Template, FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (MovieScene)
	{
		FAnimExportSequenceParameters AESP;
		AESP.Player = Player;
		AESP.RootToLocalTransform = RootToLocalTransform;
		UMovieSceneSequence* OwnerSceneSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
		AESP.MovieSceneSequence = OwnerSceneSequence;
		AESP.RootMovieSceneSequence = OwnerSceneSequence;
		return MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, ExportOptions, AESP, SkelMesh);
	}
	else
	{
		return false;
	}
}

/**
 * Retrieves a string from the MovieSceneSequence corresponding to the source takename.
 * 
 * It tries to find the UTakeMetaData slate via reflection, since UTakeMetaData is defined in a plugin.
 * If that fails, it returns the name of the MovieSceneSequence.
 */
static FString GetSlateFromMovieSceneSequence(UMovieSceneSequence* MovieSceneSequence)
{
	if (!MovieSceneSequence)
	{
		return FString();
	}

	ULevelSequence* LevelSequence = Cast<ULevelSequence>(MovieSceneSequence);

	if (!LevelSequence)
	{
		return MovieSceneSequence->GetName();
	}

	// Find the UTakeMetaData class dynamically, since it is defined in a plugin.
	static UClass* TakeMetaDataClass = FindObject<UClass>(nullptr, TEXT("/Script/TakesCore.TakeMetaData"), EFindObjectFlags::None);
	if (!TakeMetaDataClass)
	{
		return LevelSequence->GetName();
	}

	if (UObject* MetaDataObject = LevelSequence->FindMetaDataByClass(TakeMetaDataClass))
	{
		if (const FProperty* SlateProperty = TakeMetaDataClass->FindPropertyByName("Slate"))
		{
			if (const FStrProperty* SlateStrProperty = CastField<FStrProperty>(SlateProperty))
			{
				return SlateStrProperty->GetPropertyValue_InContainer(MetaDataObject);
			}
			else
			{
				UE_LOGF(LogMovieScene, Error, "TakesMetaData::Slate property was found, but it unexpectedly was not a String property.");
			}
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "TakesMetaData class was found, but it unexpectedly did not have a Slate property.");
		}
	}

	return LevelSequence->GetName();
}

bool MovieSceneToolHelpers::ExportToAnimSequence(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FAnimExportSequenceParameters& AESP,
	USkeletalMeshComponent* SkelMeshComp)
{
	if (AnimSequence == nullptr || ExportOptions == nullptr || AESP.MovieSceneSequence == nullptr || AESP.RootMovieSceneSequence == nullptr || SkelMeshComp == nullptr)
	{
		UE_LOGF(LogMovieScene, Error, "MovieSceneToolHelpers::ExportToAnimSequence All parameters must be valid.");
		return false;
	}

	FAnimRecorderInstance AnimationRecorder;
	const FFrameRate SampleRate = ExportOptions->bUseCustomFrameRate ? ExportOptions->CustomFrameRate :  AESP.MovieSceneSequence->GetMovieScene()->GetDisplayRate();
	// Used by function that retrieves the current timecode 
	FQualifiedFrameTime CurrentTimecode;

	// Prefer the rate overriden by the user, otherwise use the project setting.
	if (ExportOptions->bTimecodeRateOverride)
	{
		CurrentTimecode.Rate = ExportOptions->OverrideTimecodeRate;
	}
	else if (GEngine)
	{
		CurrentTimecode.Rate = GEngine->GenerateDefaultTimecodeFrameRate;
	}

	FInitAnimationCB InitCallback = FInitAnimationCB::CreateLambda([&AnimationRecorder, SampleRate, ExportOptions, SkelMeshComp, AnimSequence]
		{
			FAnimationRecordingSettings RecordingSettings;
			RecordingSettings.SampleFrameRate = SampleRate;
			RecordingSettings.Interpolation = ExportOptions->Interpolation;
			RecordingSettings.InterpMode = ExportOptions->CurveInterpolation;
			if (ExportOptions->CurveInterpolation == ERichCurveInterpMode::RCIM_Constant ||
				ExportOptions->CurveInterpolation == ERichCurveInterpMode::RCIM_None)
			{
				RecordingSettings.TangentMode = ERichCurveTangentMode::RCTM_None;
			}
			else
			{
				RecordingSettings.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			RecordingSettings.Length = 0;
			RecordingSettings.bRemoveRootAnimation = false;
			RecordingSettings.bCheckDeltaTimeAtBeginning = false;
			RecordingSettings.bSetRetargetSourceAsset = ExportOptions->bSetRetargetSourceAsset;
			RecordingSettings.bRecordTransforms = ExportOptions->bExportTransforms;
			RecordingSettings.bRecordMorphTargets = ExportOptions->bExportMorphTargets;
			RecordingSettings.bRecordAttributeCurves = ExportOptions->bExportAttributeCurves;
			RecordingSettings.bRecordMaterialCurves = ExportOptions->bExportMaterialCurves;
			RecordingSettings.bSkipCurvesWithZeroValue = ExportOptions->bSkipCurvesWithZeroValue;
			RecordingSettings.bRecordInWorldSpace = ExportOptions->bRecordInWorldSpace;
			RecordingSettings.IncludeAnimationNames = ExportOptions->IncludeAnimationNames;
			RecordingSettings.ExcludeAnimationNames = ExportOptions->ExcludeAnimationNames;
			RecordingSettings.bRemoveExcludedCurves = ExportOptions->bRemoveExcludedCurves;
			RecordingSettings.bTransactRecording = false; //don't do this it breaks undo's
			AnimationRecorder.Init(SkelMeshComp, AnimSequence, nullptr, RecordingSettings);
		});

	FStartAnimationCB StartCallback = FStartAnimationCB::CreateLambda([SkelMeshComp, &AnimationRecorder, &CurrentTimecode, &SampleRate, ExportOptions](FFrameNumber FrameNumber)
		{
			if (ExportOptions->bBakeTimecode)
			{
				ensureMsgf(AnimationRecorder.Recorder.IsValid(), TEXT("Unable to bake timecode due to Invalid Recorder."));

				if (AnimationRecorder.Recorder.IsValid())
				{
					// Update the time because AnimationRecorder.BeginRecording() will key the first frame.
					CurrentTimecode.Time = FQualifiedFrameTime(FFrameTime(FrameNumber), SampleRate).ConvertTo(CurrentTimecode.Rate);

					AnimationRecorder.Recorder->SetCurrentFrameTimeGetter([&CurrentTimecode]() -> TOptional<FQualifiedFrameTime>
						{
							return CurrentTimecode;
						});
				}
			}

			AnimationRecorder.BeginRecording();
			SkelMeshComp->UpdateLODStatus();
		});

	FTickAnimationCB TickCallback = FTickAnimationCB::CreateLambda([&AnimationRecorder, &CurrentTimecode, SampleRate, ExportOptions](float DeltaTime, FFrameNumber FrameNumber)
		{
			// Update current timecode, which will be read during AnimationRecorder via the custom current time getter.
			if (ExportOptions->bBakeTimecode)
			{
				CurrentTimecode.Time = FQualifiedFrameTime(FFrameTime(FrameNumber), SampleRate).ConvertTo(CurrentTimecode.Rate);
			}

			AnimationRecorder.Update(DeltaTime);
		});

	FEndAnimationCB EndCallback = FEndAnimationCB::CreateLambda([&AnimationRecorder, ExportOptions, AnimSequence, SkelMeshComp, AESP]
		{
			if (ExportOptions->bBakeTimecode)
			{
				FProcessRecordedTimeParams TimeParams;
				
				// Use the timecode attribute names in the animation settings for consistency.
				if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
				{
					TimeParams.HoursName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.HourAttributeName.ToString();
					TimeParams.MinutesName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.MinuteAttributeName.ToString();
					TimeParams.SecondsName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SecondAttributeName.ToString();
					TimeParams.FramesName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.FrameAttributeName.ToString();
					TimeParams.SubFramesName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SubframeAttributeName.ToString();
					TimeParams.RateName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.RateAttributeName.ToString();
					TimeParams.SlateName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.TakenameAttributeName.ToString();

					TimeParams.Slate = GetSlateFromMovieSceneSequence(AESP.MovieSceneSequence);
				}

				AnimationRecorder.ProcessRecordedTimes(AnimSequence, SkelMeshComp, FTimecodeBoneMethod(), TimeParams);
			}

			const bool bShowAnimationAssetCreatedToast = false;
			AnimationRecorder.FinishRecording(bShowAnimationAssetCreatedToast);

			// Clean up the getter since it references a local variable and AnimationRecorder.Recorder is managed by a shared pointer.
			if (ExportOptions->bBakeTimecode && AnimationRecorder.Recorder.IsValid())
			{
				AnimationRecorder.Recorder->SetCurrentFrameTimeGetter([]() -> TOptional<FQualifiedFrameTime>
					{
						return FApp::GetCurrentFrameTime();
					});
			}
		});

	MovieSceneToolHelpers::BakeToSkelMeshToCallbacks(AESP, SkelMeshComp, ExportOptions,
		InitCallback, StartCallback, TickCallback, EndCallback);

	return AnimSequence->GetDataModel()->HasBeenPopulated();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // Use FAllSpawnableRestoreState

FSpawnableRestoreState::FSpawnableRestoreState(UMovieScene* MovieScene)
	: FSpawnableRestoreState(MovieScene, nullptr)
{

}

FSpawnableRestoreState::FSpawnableRestoreState(UMovieScene* MovieScene, TSharedPtr<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState)
	: bWasChanged(false)
	, WeakMovieScene(MovieScene)
	, SharedPlaybackState(InSharedPlaybackState)
{
	for (int32 SpawnableIndex = 0; SpawnableIndex < WeakMovieScene->GetSpawnableCount(); ++SpawnableIndex)
	{
		FMovieSceneSpawnable& Spawnable = WeakMovieScene->GetSpawnable(SpawnableIndex);

		FAllSpawnableRestoreState::SaveStateForSpawnable(bWasChanged, SpawnOwnershipMap, WeakMovieScene, Spawnable.GetGuid(), Spawnable.GetSpawnOwnership(), [&Spawnable](ESpawnOwnership SpawnOwnership) { Spawnable.SetSpawnOwnership(SpawnOwnership);});
	}

	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackStateToUse = SharedPlaybackState;
	if (!SharedPlaybackStateToUse)
	{
		SharedPlaybackStateToUse = MovieSceneHelpers::CreateTransientSharedPlaybackState(GEditor->GetEditorWorldContext().World(), WeakMovieScene->GetTypedOuter<UMovieSceneSequence>());
	}

	ensure(SharedPlaybackStateToUse.IsValid());
	if (UMovieSceneSequence* Sequence = WeakMovieScene->GetTypedOuter<UMovieSceneSequence>())
	{
		if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
			{
				if (BindingReference.CustomBinding)
				{
					if (UMovieSceneSpawnableBindingBase* SpawnableBinding = BindingReference.CustomBinding->AsSpawnable(SharedPlaybackStateToUse.ToSharedRef()))
					{
						FAllSpawnableRestoreState::SaveStateForSpawnable(bWasChanged, SpawnOwnershipMap, WeakMovieScene, BindingReference.ID, SpawnableBinding->SpawnOwnership, [SpawnableBinding](ESpawnOwnership SpawnOwnership) { SpawnableBinding->SpawnOwnership = SpawnOwnership; });
					}								
				}
			}
		}
	}
}
FSpawnableRestoreState::~FSpawnableRestoreState()
{
	if (bWasChanged)
	{
		GEditor->EndTransaction();
	}

	if (!bWasChanged || !WeakMovieScene.IsValid())
	{
		return;
	}

	// Restore spawnable owners
	for (int32 SpawnableIndex = 0; SpawnableIndex < WeakMovieScene->GetSpawnableCount(); ++SpawnableIndex)
	{
		FMovieSceneSpawnable& Spawnable = WeakMovieScene->GetSpawnable(SpawnableIndex);
		if (ESpawnOwnership* SpawnOwnership = SpawnOwnershipMap.Find(Spawnable.GetGuid()))
		{
			Spawnable.SetSpawnOwnership(*SpawnOwnership);
		}
	}

	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackStateToUse = SharedPlaybackState;
	if (!SharedPlaybackStateToUse)
	{
		// Supporting deprecated path where we have no SharedPlaybackState
		UE::MovieScene::FSharedPlaybackStateCreateParams CreateParams;
		CreateParams.PlaybackContext = GEditor->GetEditorWorldContext().World();
		UMovieSceneSequence* ThisSequence = WeakMovieScene->GetTypedOuter<UMovieSceneSequence>();
		SharedPlaybackStateToUse = MakeShared<UE::MovieScene::FSharedPlaybackState>(*ThisSequence, CreateParams);

		FMovieSceneEvaluationState State;
		SharedPlaybackStateToUse->AddCapabilityRaw(&State);
		State.AssignSequence(MovieSceneSequenceID::Root, *ThisSequence, SharedPlaybackStateToUse.ToSharedRef());
	}
	ensure(SharedPlaybackStateToUse.IsValid());

	if (UMovieSceneSequence* Sequence = WeakMovieScene->GetTypedOuter<UMovieSceneSequence>())
	{
		if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
			{
				if (BindingReference.CustomBinding)
				{
					if (UMovieSceneSpawnableBindingBase* SpawnableBinding = BindingReference.CustomBinding->AsSpawnable(SharedPlaybackStateToUse.ToSharedRef()))
					{
						if (ESpawnOwnership* SpawnOwnership = SpawnOwnershipMap.Find(BindingReference.ID))
						{
							SpawnableBinding->SpawnOwnership = *SpawnOwnership;
						}
					}
				}
			}
		}
	}

	// Restore modified spawned sections
	const bool bOrigSquelchTransactionNotification = GEditor->bSquelchTransactionNotification;
	GEditor->bSquelchTransactionNotification = true;
	const bool bOrigIgnoreSelectionChange = GEditor->bIgnoreSelectionChange;
	GEditor->bIgnoreSelectionChange = true;
	const bool bOrigSuspendBroadcastPostUndoRedo = GEditor->bSuspendBroadcastPostUndoRedo;
	GEditor->bSuspendBroadcastPostUndoRedo = true;

	GEditor->UndoTransaction(false);

	GEditor->bSquelchTransactionNotification = bOrigSquelchTransactionNotification;
	GEditor->bIgnoreSelectionChange = bOrigIgnoreSelectionChange;
	GEditor->bSuspendBroadcastPostUndoRedo = bOrigSuspendBroadcastPostUndoRedo;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FAllSpawnableRestoreState::SaveStateForSpawnable(bool& bInOutWasChanged, TMap<FGuid, ESpawnOwnership>& InOutSpawnOwnershipMap, TWeakObjectPtr<UMovieScene>& WeakMovieScene, FGuid BindingGuid, ESpawnOwnership PreviousSpawnOwnership, TFunction<void(ESpawnOwnership)> SpawnOwnershipSetter)
{
	if (WeakMovieScene.IsValid() == false)
	{
		return;
	}
	// Spawnable could be in a subscene, so temporarily override it to persist throughout
	InOutSpawnOwnershipMap.Add(BindingGuid, PreviousSpawnOwnership);

	SpawnOwnershipSetter(ESpawnOwnership::RootSequence);

	const UMovieSceneSpawnTrack* SpawnTrack = WeakMovieScene->FindTrack<UMovieSceneSpawnTrack>(BindingGuid);
	if (SpawnTrack && !SpawnTrack->IsEmpty())
	{
		if (UMovieSceneSpawnSection* SpawnSection = Cast<UMovieSceneSpawnSection>(SpawnTrack->GetAllSections()[0]))
		{
			// no need to modify the spawned state if it's constant
			auto ShouldSectionBeModified = [SpawnSection]()
			{
				const FMovieSceneBoolChannel& SpawnedChannel = SpawnSection->GetChannel();
				const TArrayView<const bool>& Values = SpawnedChannel.GetValues();
				if (!Values.IsEmpty())
				{
					for (int32 Index = 1; Index < Values.Num(); Index++)
					{
						if (Values[Index] != Values[Index-1])
						{
							return true;
						}
					}
				}
				return false;
			};
		
			if (ShouldSectionBeModified())
			{
				// Start a transaction that will be undone later for the modifications to the spawn track
				if (!bInOutWasChanged)
				{
					GEditor->BeginTransaction(NSLOCTEXT("MovieSceneToolHelpers", "SpwanableRestoreState", "SpawnableRestoreState"));
					bInOutWasChanged = true;
				}

				// spawn it by default 
				SpawnSection->Modify();
				SpawnSection->GetChannel().SetDefault(true);
			}
		}
	}

	UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = WeakMovieScene->FindTrack<UMovieSceneBindingLifetimeTrack>(BindingGuid);
	if (BindingLifetimeTrack && BindingLifetimeTrack->GetAllSections().Num() > 0)
	{
		// Start a transaction that will be undone later for the modifications to the binding lifetime track
		if (!bInOutWasChanged)
		{
			GEditor->BeginTransaction(NSLOCTEXT("MovieSceneToolHelpers", "SpwanableRestoreState", "SpawnableRestoreState"));
		}

		bInOutWasChanged = true;

		BindingLifetimeTrack->Modify();
		BindingLifetimeTrack->RemoveAllAnimationData();
		// Add a single infinite section
		UMovieSceneSection* NewSection = NewObject<UMovieSceneSection>(BindingLifetimeTrack, UMovieSceneBindingLifetimeSection::StaticClass(), NAME_None, RF_Transactional);
		check(NewSection);
		NewSection->SetRange(TRange<FFrameNumber>::All());
		BindingLifetimeTrack->AddSection(*NewSection);
	}
}

FAllSpawnableRestoreState::FAllSpawnableRestoreState(TSharedPtr<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState, UMovieScene* InMovieScene)
	: SharedPlaybackState(InSharedPlaybackState)
{

	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackStateToUse = SharedPlaybackState;
	if (!SharedPlaybackStateToUse)
	{
		UE_LOGF(LogMovieScene, Warning, "FAllSpawnableRestoreState requires a valid FSharedPlaybackState");
		return;
	}
	if (InMovieScene)
	{
		FPerMovieScene PerMovieScene;
		PerMovieScene.WeakMovieScene = InMovieScene;
		PerMovieScene.SequenceID = MovieSceneSequenceID::Root;
		OwnershipMaps.Add(InMovieScene, PerMovieScene);
	}
	else
	{
		const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackStateToUse->GetHierarchy();
		if (!Hierarchy)
		{
			if (UMovieSceneSequence* Sequence = SharedPlaybackStateToUse->GetSequence(MovieSceneSequenceID::Root))
			{
				if (UMovieScene* MovieScene = Sequence->GetMovieScene())
				{
					FPerMovieScene PerMovieScene;
					PerMovieScene.WeakMovieScene = MovieScene;
					PerMovieScene.SequenceID = MovieSceneSequenceID::Root;
					OwnershipMaps.Add(MovieScene, PerMovieScene);
				}
			}
			else
			{
				UE_LOGF(LogMovieScene, Warning, "FAllSpawnableRestoreState requires a valid Root Sequence from FSharedPlaybackState");
				return;
			}
		}
		else
		{
			TArray<FMovieSceneSequenceID> SequenceIDs;
			Hierarchy->AllSubSequenceIDs(SequenceIDs);
			TArray<TWeakObjectPtr<UMovieSceneSequence>> Sequences;
			for (const FMovieSceneSequenceID& SequenceID : SequenceIDs)
			{
				if (UMovieSceneSequence* Sequence = SharedPlaybackStateToUse->GetSequence(SequenceID))
				{
					if (UMovieScene* MovieScene = Sequence->GetMovieScene())
					{
						FPerMovieScene PerMovieScene;
						PerMovieScene.WeakMovieScene = MovieScene;
						PerMovieScene.SequenceID = SequenceID;
						OwnershipMaps.Add(MovieScene, PerMovieScene);
					}
				}
			}
		}
	}

	for (TPair <TWeakObjectPtr<UMovieScene>, FPerMovieScene>& OwnershipMap : OwnershipMaps)
	{
		if (OwnershipMap.Key.IsValid() == false)
		{
			continue;
		}
		for (int32 SpawnableIndex = 0; SpawnableIndex < OwnershipMap.Key->GetSpawnableCount(); ++SpawnableIndex)
		{
			FMovieSceneSpawnable& Spawnable = OwnershipMap.Key->GetSpawnable(SpawnableIndex);

			FAllSpawnableRestoreState::SaveStateForSpawnable(bWasChanged, OwnershipMap.Value.SpawnOwnershipMap, OwnershipMap.Key, Spawnable.GetGuid(), Spawnable.GetSpawnOwnership(), [&Spawnable](ESpawnOwnership SpawnOwnership) { Spawnable.SetSpawnOwnership(SpawnOwnership); });
		}

		ensure(SharedPlaybackStateToUse.IsValid());
		if (UMovieSceneSequence* Sequence = OwnershipMap.Key->GetTypedOuter<UMovieSceneSequence>())
		{
			if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
			{
				for (FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
				{
					if (BindingReference.CustomBinding)
					{
						if (UMovieSceneSpawnableBindingBase* SpawnableBinding = BindingReference.CustomBinding->AsSpawnable(SharedPlaybackStateToUse.ToSharedRef()))
						{
							FAllSpawnableRestoreState::SaveStateForSpawnable(bWasChanged, OwnershipMap.Value.SpawnOwnershipMap, OwnershipMap.Key, BindingReference.ID, SpawnableBinding->SpawnOwnership, [SpawnableBinding](ESpawnOwnership SpawnOwnership) { SpawnableBinding->SpawnOwnership = SpawnOwnership; });
						}
					}
				}
			}
		}
	}
}

FAllSpawnableRestoreState::~FAllSpawnableRestoreState()
{
	if (bWasChanged)
	{
		GEditor->EndTransaction();
	}

	if (!bWasChanged || OwnershipMaps.IsEmpty())
	{
		return;
	}

	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackStateToUse = SharedPlaybackState;

	for (TPair <TWeakObjectPtr<UMovieScene>, FPerMovieScene>& OwnershipMap : OwnershipMaps)
	{
		if (OwnershipMap.Key.IsValid() == false)
		{
			continue;
		}
		// Restore spawnable owners
		for (int32 SpawnableIndex = 0; SpawnableIndex < OwnershipMap.Key->GetSpawnableCount(); ++SpawnableIndex)
		{
			FMovieSceneSpawnable& Spawnable = OwnershipMap.Key->GetSpawnable(SpawnableIndex);
			if (ESpawnOwnership* SpawnOwnership = OwnershipMap.Value.SpawnOwnershipMap.Find(Spawnable.GetGuid()))
			{
				Spawnable.SetSpawnOwnership(*SpawnOwnership);
			}
		}

		if (!SharedPlaybackStateToUse)
		{
			// Supporting deprecated path where we have no SharedPlaybackState
			UE::MovieScene::FSharedPlaybackStateCreateParams CreateParams;
			CreateParams.PlaybackContext = GEditor->GetEditorWorldContext().World();
			UMovieSceneSequence* ThisSequence = OwnershipMap.Key->GetTypedOuter<UMovieSceneSequence>();
			SharedPlaybackStateToUse = MakeShared<UE::MovieScene::FSharedPlaybackState>(*ThisSequence, CreateParams);

			FMovieSceneEvaluationState State;
			SharedPlaybackStateToUse->AddCapabilityRaw(&State);
			State.AssignSequence(MovieSceneSequenceID::Root, *ThisSequence, SharedPlaybackStateToUse.ToSharedRef());
		}
		ensure(SharedPlaybackStateToUse.IsValid());

		if (UMovieSceneSequence* Sequence = OwnershipMap.Key->GetTypedOuter<UMovieSceneSequence>())
		{
			if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
			{
				for (FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
				{
					if (BindingReference.CustomBinding)
					{
						if (UMovieSceneSpawnableBindingBase* SpawnableBinding = BindingReference.CustomBinding->AsSpawnable(SharedPlaybackStateToUse.ToSharedRef()))
						{
							if (ESpawnOwnership* SpawnOwnership = OwnershipMap.Value.SpawnOwnershipMap.Find(BindingReference.ID))
							{
								SpawnableBinding->SpawnOwnership = *SpawnOwnership;
							}
						}
					}
				}
			}
		}
	}

	// Restore modified spawned sections
	const bool bOrigSquelchTransactionNotification = GEditor->bSquelchTransactionNotification;
	GEditor->bSquelchTransactionNotification = true;
	const bool bOrigIgnoreSelectionChange = GEditor->bIgnoreSelectionChange;
	GEditor->bIgnoreSelectionChange = true;
	const bool bOrigSuspendBroadcastPostUndoRedo = GEditor->bSuspendBroadcastPostUndoRedo;
	GEditor->bSuspendBroadcastPostUndoRedo = true;

	GEditor->UndoTransaction(false);

	GEditor->bSquelchTransactionNotification = bOrigSquelchTransactionNotification;
	GEditor->bIgnoreSelectionChange = bOrigIgnoreSelectionChange;
	GEditor->bSuspendBroadcastPostUndoRedo = bOrigSuspendBroadcastPostUndoRedo;
}

void MovieSceneToolHelpers::GetParents(TArray<const UObject*>& Parents, const UObject* InObject)
{
	const AActor* Actor = Cast<AActor>(InObject);
	if (Actor)
	{
		Parents.Emplace(Actor);
		const AActor* ParentActor = Actor->GetAttachParentActor();
		if (ParentActor)
		{
			GetParents(Parents, ParentActor);
		}
	}
}
/** This is not that scalable moving forward with stuff like the control rig , need a better caching solution there */
bool MovieSceneToolHelpers::GetParentTM(FTransform& CurrentRefTM, const TSharedPtr<ISequencer>& Sequencer, UObject* ParentObject, FFrameTime KeyTime)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return false;
	}

	FGuid ObjectBinding = Sequencer->FindCachedObjectId(*ParentObject, Sequencer->GetFocusedTemplateID());
	if (!ObjectBinding.IsValid())
	{
		return false;
	}

	const FMovieSceneBinding* Binding = Sequence->GetMovieScene()->FindBinding(ObjectBinding);
	if (!Binding)
	{
		return false;
	}
	//TODO this doesn't handle blended sections at all
	for (const UMovieSceneTrack* Track : Binding->GetTracks())
	{
		const UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(Track);
		if (!TransformTrack)
		{
			continue;
		}

		//we used to loop between sections here and only evaluate if we are in a section, this will give us wrong transfroms though
		//when in between or outside of the section range. We still want to evaluate, though it is heavy.

		const FMovieSceneEvaluationTrack* EvalTrack = MovieSceneToolHelpers::GetEvaluationTrack(Sequencer.Get(), TransformTrack->GetSignature());
		if (EvalTrack)
		{
			FVector ParentKeyPos;
			FRotator ParentKeyRot;
			GetLocationAtTime(EvalTrack, ParentObject, KeyTime, ParentKeyPos, ParentKeyRot, Sequencer);
			CurrentRefTM = FTransform(ParentKeyRot, ParentKeyPos);
			return true;
		}

	}

	return false;
}

FTransform MovieSceneToolHelpers::GetRefFrameFromParents(const TSharedPtr<ISequencer>& Sequencer, const TArray<const UObject*>& Parents, FFrameTime KeyTime)
{
	FTransform RefTM = FTransform::Identity;
	FTransform ParentRefTM = FTransform::Identity;

	for (const UObject* Object : Parents)
	{
		const AActor* Actor = Cast<AActor>(Object);
		if (Actor != nullptr)
		{
			if (Actor->GetRootComponent() != nullptr && Actor->GetRootComponent()->GetAttachParent() != nullptr)
			{
				//Always get local ref tm since we don't know which parent is in the sequencer or not.
				if (!GetParentTM(ParentRefTM, Sequencer, Actor->GetRootComponent()->GetAttachParent()->GetOwner(), KeyTime))
				{
					AActor* Parent = Actor->GetRootComponent()->GetAttachParent()->GetOwner();
					if (Parent && Parent->GetRootComponent())
					{
						ParentRefTM = Parent->GetRootComponent()->GetRelativeTransform();
					}
					else
					{
						continue;
					}
				}
				RefTM = ParentRefTM * RefTM;
			}
		}
		else
		{
			const USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
			FTransform CurrentRefTM = FTransform::Identity;
			UObject* ParentObject = SceneComponent->GetAttachParent() == SceneComponent->GetOwner()->GetRootComponent() ? static_cast<UObject*>(SceneComponent->GetOwner()) : SceneComponent->GetAttachParent();

			if (SceneComponent->GetAttachParent() != nullptr)
			{
				if (!GetParentTM(CurrentRefTM, Sequencer, ParentObject, KeyTime))
				{
					CurrentRefTM = RefTM * SceneComponent->GetAttachParent()->GetRelativeTransform();
				}
			}
			RefTM = CurrentRefTM * RefTM;
		}
	}
	return RefTM;
}

void MovieSceneToolHelpers::GetLocationAtTime(const FMovieSceneEvaluationTrack* Track, UObject* Object, FFrameTime KeyTime, FVector& KeyPos, FRotator& KeyRot, const TSharedPtr<ISequencer>& Sequencer)
{
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// TODO: Reimplement trajectory rendering
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	UE_MOVIESCENE_TODO(Reimplement trajectory rendering)
}

USkeletalMeshComponent* MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(UObject* BoundObject)
{
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(Component))
			{
				return SkeletalMeshComp;
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return SkeletalMeshComponent;
		}
	}
	return nullptr;
}
static void GetSequenceSceneComponentWorldTransforms(USceneComponent* SceneComponent, IMovieScenePlayer* Player, UMovieSceneSequence* InSequence, FMovieSceneSequenceIDRef Template, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutTransforms)
{

	// Hack: static system interrogator for now to avoid re-allocating UObjects all the time
	static UE::MovieScene::FSystemInterrogator Interrogator;
	Interrogator.Reset();

	Interrogator.ImportTransformHierarchy(SceneComponent, Player, Template);

	if (Frames[0] < Frames[Frames.Num() - 1])
	{
		for (const FFrameNumber& FrameNumber : Frames)
		{
			const FFrameTime FrameTime(FrameNumber);
			Interrogator.AddInterrogation(FrameTime);
		}
		Interrogator.Update();

		Interrogator.QueryWorldSpaceTransforms(SceneComponent, OutTransforms);
	}
	else //time is backward we need to do swap things around
	{
		for (int32 Index = Frames.Num() - 1; Index >= 0; --Index)
		{
			const FFrameTime FrameTime(Frames[Index]);
			Interrogator.AddInterrogation(FrameTime);
		}
		Interrogator.Update();

		TArray<FTransform> WorldTransforms;
		Interrogator.QueryWorldSpaceTransforms(SceneComponent, WorldTransforms);
		OutTransforms.SetNum(0);
		OutTransforms.Reserve(WorldTransforms.Num());
		for (int32 Index2 = WorldTransforms.Num() - 1; Index2 >= 0; --Index2)
		{
			OutTransforms.Add(WorldTransforms[Index2]);
		}
	}
	Interrogator.Reset();
}

static void GetSequencerActorWorldTransforms(IMovieScenePlayer* Player, FMovieSceneSequenceTransform RootToLocalTransform, UMovieSceneSequence* InSequence, FMovieSceneSequenceIDRef Template, const FActorForWorldTransforms& ActorSelection, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutTransforms)
{
	USceneComponent* SceneComponent = nullptr;
	AActor* Actor = ActorSelection.Actor.Get();
	if (Actor)
	{
		SceneComponent = Actor->GetRootComponent();
	}
	else
	{
		SceneComponent = ActorSelection.Component.IsValid() ? Cast<USceneComponent>(ActorSelection.Component.Get()) : nullptr;
		if (SceneComponent)
		{
			Actor = SceneComponent->GetTypedOuter<AActor>();
		}
	}
	
	if(Actor && SceneComponent)
	{
		USkeletalMeshComponent* SkelMeshComp =  Cast<USkeletalMeshComponent>(SceneComponent);

		if (!SkelMeshComp)
		{
			SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(Actor);
		}

		if (UMovieScene* MovieScene = InSequence->GetMovieScene())
		{
			OutTransforms.SetNum(Frames.Num());

			FFrameRate TickResolution = MovieScene->GetTickResolution();
			const TArray<IMovieSceneToolsAnimationBakeHelper*>& BakeHelpers = FMovieSceneToolsModule::Get().GetAnimationBakeHelpers();

			for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
			{
				if (BakeHelper)
				{
					BakeHelper->StartBaking(MovieScene);
				}
			}

			const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
			const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
			
			for (int32 Index = 0; Index < Frames.Num(); ++Index)
			{
				FFrameNumber FrameNumber = Frames[Index];
				FFrameTime GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(FrameNumber).Get(FrameNumber); //player evals in root time so need to go back to it.

				for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
				{
					if (BakeHelper)
					{
						BakeHelper->PreEvaluation(MovieScene, FrameNumber);
					}
				}

				FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);
				if (Index == 0) // similar with baking first time in we need to evaluate twice (think due to double buffering that happens with skel mesh components).
				{
					Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
				}
				Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);


				{
					const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
					const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(Actor->GetWorld());
					Controller.EvaluateAllConstraints();
				}
					
				for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
				{
					if (BakeHelper)
					{
						BakeHelper->PostEvaluation(MovieScene, FrameNumber);
					}
				}

				AActor* Parent = ActorSelection.Actor.Get();
				while (Parent)
				{
					TArray<USkeletalMeshComponent*> MeshComps;
					Parent->GetComponents(MeshComps, true);

					for (USkeletalMeshComponent* MeshComp : MeshComps)
					{
						MeshComp->TickAnimation(0.03f, false);
						MeshComp->RefreshBoneTransforms();
						MeshComp->RefreshFollowerComponents();
						MeshComp->UpdateComponentToWorld();
						MeshComp->FinalizeBoneTransform();
						MeshComp->MarkRenderTransformDirty();
						MeshComp->MarkRenderDynamicDataDirty();
					}

					Parent = Parent->GetAttachParentActor();
				}

				OutTransforms[Index] = (SkelMeshComp && ActorSelection.SocketName != NAME_None)
					? SkelMeshComp->GetSocketTransform(ActorSelection.SocketName)
					: SceneComponent->GetComponentToWorld();

			}

			for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
			{
				if (BakeHelper)
				{
					BakeHelper->StopBaking(MovieScene);
				}
			}
		}	
	}
}

void MovieSceneToolHelpers::RunSequencerTasks(
	ISequencer* Sequencer,
	UWorld* World,
	const TArray<FFrameNumber>& Frames,
	const TArrayView<TFunction<void(int32)>>& Tasks)
{
	if (Frames.IsEmpty() || Tasks.IsEmpty())
	{
		return;
	}

	UMovieSceneSequence* MovieSceneSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}
	
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
	const FMovieSceneInverseSequenceTransform InverseSequenceTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();
	
	FMovieSceneToolsAnimationBakingWrapper BakeWrapper(MovieScene);
	BakeWrapper.Start();

	const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
	const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
	
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber FrameNumber = Frames[Index];
		FFrameTime GlobalTime = InverseSequenceTransform.TryTransformTime(FrameNumber).Get(FrameNumber); //player evals in root time so need to go back to it.

		BakeWrapper.PreEvaluate(FrameNumber);

		FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
		if (Index == 0) // similar with baking first time in we need to evaluate twice (think due to double buffering that happens with skel mesh components).
		{
			EvaluationTemplate.EvaluateSynchronousBlocking(Context);
		}
		EvaluationTemplate.EvaluateSynchronousBlocking(Context);

		if (World)
		{
			const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			Controller.EvaluateAllConstraints();
		}

		BakeWrapper.PostEvaluate(FrameNumber);

		for (const TFunction<void(int32)>& Task: Tasks)
		{
			Task(Index);
		}
	}

	BakeWrapper.Stop();
}

void MovieSceneToolHelpers::RunSequencerTask(
		ISequencer* Sequencer,
		UWorld* World,
		const TArray<FFrameNumber>& Frames,
		const TFunctionRef<void(int32)>& Task)
{
	if (Frames.IsEmpty())
	{
		return;
	}

	UMovieSceneSequence* MovieSceneSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}
	
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
	const FMovieSceneInverseSequenceTransform InverseSequenceTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();
	
	FMovieSceneToolsAnimationBakingWrapper BakeWrapper(MovieScene);
	BakeWrapper.Start();

	const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
	const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber FrameNumber = Frames[Index];
		FFrameTime GlobalTime = InverseSequenceTransform.TryTransformTime(FrameNumber).Get(FrameNumber); //player evals in root time so need to go back to it.

		BakeWrapper.PreEvaluate(FrameNumber);

		FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
		if (Index == 0) // similar with baking first time in we need to evaluate twice (think due to double buffering that happens with skel mesh components).
		{
			EvaluationTemplate.EvaluateSynchronousBlocking(Context);
		}
		EvaluationTemplate.EvaluateSynchronousBlocking(Context);

		if (World)
		{
			const UE::Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			Controller.EvaluateAllConstraints();
		}

		BakeWrapper.PostEvaluate(FrameNumber);

		Task(Index);
	}

	BakeWrapper.Stop();
}

static void GetNonSequencerActorWorldTransforms(IMovieScenePlayer* Player, FMovieSceneSequenceTransform RootToLocalTransform, UMovieSceneSequence* InSequence, FMovieSceneSequenceIDRef Template, const FActorForWorldTransforms& ActorSelection, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutTransforms)
{
	FName SocketName = ActorSelection.SocketName;
	AActor* Actor = ActorSelection.Actor.Get();
	FActorForWorldTransforms NewActorSelection = ActorSelection;
	FTransform WorldTransform = FTransform::Identity;
	if (Actor)
	{
		do
		{
			FGuid ActorHandle = GetHandleToObject(Actor, InSequence, Player, Template,false);
			if (ActorHandle.IsValid() && Frames.Num() > 0)
			{
				if (InSequence->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(ActorHandle))
				{
					GetSequencerActorWorldTransforms(Player, RootToLocalTransform, InSequence, Template, NewActorSelection, Frames, OutTransforms);
					for (FTransform& OutTransform : OutTransforms)
					{
						OutTransform = WorldTransform * OutTransform;
					}
					return;
				}
			}

			if (Actor->GetRootComponent()->DoesSocketExist(SocketName))
			{
				WorldTransform = WorldTransform * Actor->GetRootComponent()->GetSocketTransform(SocketName);
			}
			else
			{
				WorldTransform = WorldTransform * Actor->GetRootComponent()->GetRelativeTransform();
			}
			SocketName = Actor->GetAttachParentSocketName();
			Actor = Actor->GetAttachParentActor();
			NewActorSelection.Actor = Actor;
			if (Actor)
			{
				NewActorSelection.SocketName = SocketName;
			}
		} while (Actor);
		//if we get to here world transform is what we want 
	}
	int32 NumFrames = Frames.Num() > 0 ? Frames.Num() : 1; //if no frames passed in we actually have one
	OutTransforms.SetNumUninitialized(NumFrames);
	for (FTransform& OutTransform : OutTransforms)
	{
		OutTransform = WorldTransform;
	}
}

void MovieSceneToolHelpers::GetActorWorldTransforms(ISequencer* Sequencer, const FActorForWorldTransforms& ActorSelection, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutWorldTransforms)
{
	FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
	const bool bAvoidEvaluates = (Frames.Num() == 0) || (Frames.Num() == 1 && Frames[0] == Sequencer->GetLocalTime().Time.RoundToFrame());
	FGuid ObjectHandle = Sequencer->GetHandleToObject(ActorSelection.Actor.Get(), false);
	if (!bAvoidEvaluates && ObjectHandle.IsValid())
	{
		if (Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(ObjectHandle))
		{
			GetSequencerActorWorldTransforms(Sequencer, Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped(), Sequencer->GetFocusedMovieSceneSequence(), Template, ActorSelection, Frames, OutWorldTransforms);
			return;
		}
	}

	ObjectHandle = Sequencer->GetHandleToObject(ActorSelection.Component.Get(), false);
	if (!bAvoidEvaluates && ObjectHandle.IsValid())
	{
		if (Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(ObjectHandle))
		{
			GetSequencerActorWorldTransforms(Sequencer, Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped(), Sequencer->GetFocusedMovieSceneSequence(), Template, ActorSelection, Frames, OutWorldTransforms);
			return;
		}
	}
	if (bAvoidEvaluates)
	{
		TArray<FFrameNumber> NoFrame; //this will make sure we don't evaluate
		GetNonSequencerActorWorldTransforms(Sequencer, Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped(), Sequencer->GetFocusedMovieSceneSequence(), Template, ActorSelection, NoFrame, OutWorldTransforms);
	}
	else
	{
		GetNonSequencerActorWorldTransforms(Sequencer, Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped(), Sequencer->GetFocusedMovieSceneSequence(), Template, ActorSelection, Frames, OutWorldTransforms);
	}
}

bool MovieSceneToolHelpers::IsValidAsset(UMovieSceneSequence* Sequence, const FAssetData& InAssetData)
{
	if (!Sequence)
	{
		return false;
	}

	FAssetReferenceFilterContext AssetReferenceFilterContext;
	AssetReferenceFilterContext.AddReferencingAsset(FAssetData(Sequence));

	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);

	FText FailureReason;
	if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(InAssetData, &FailureReason))
	{
		UE_LOGF(LogMovieScene, Warning, "%ls cannot reference because: %ls", *GetNameSafe(Sequence), *FailureReason.ToString());
		return false;
	}
	return true;
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& ChannelData, FFrameNumber Time, float Value, const EMovieSceneKeyInterpolation Interpolation)
{
	int32 ExistingIndex = ChannelData.FindKey(Time);
	if (ExistingIndex != INDEX_NONE)
	{
		FMovieSceneFloatValue& FloatValue = ChannelData.GetValues()[ExistingIndex];
		FloatValue.Value = Value;
	}
	else
	{
		FMovieSceneFloatValue NewKey(Value);
		ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone;
		NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
		NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
		NewKey.Tangent.ArriveTangent = 0.0f;
		NewKey.Tangent.LeaveTangent = 0.0f;
		NewKey.Tangent.TangentWeightMode = WeightedMode;
		NewKey.Tangent.ArriveTangentWeight = 0.0f;
		NewKey.Tangent.LeaveTangentWeight = 0.0f;

		switch (Interpolation)
		{
			case EMovieSceneKeyInterpolation::SmartAuto:    
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_SmartAuto;
			}
			break;
			case EMovieSceneKeyInterpolation::Auto:  
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;
			case EMovieSceneKeyInterpolation::User: 
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_User;
			}
			break;
			case EMovieSceneKeyInterpolation::Break:  
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;

			case EMovieSceneKeyInterpolation::Linear:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Linear;
			}
			break;

			case EMovieSceneKeyInterpolation::Constant:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Constant;
			}
			break;

		}
		ChannelData.AddKey(Time, NewKey);
	}
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& ChannelData, FFrameNumber Time, const FMovieSceneFloatValue& Value)
{
	int32 ExistingIndex = ChannelData.FindKey(Time);
	if (ExistingIndex != INDEX_NONE)
	{
		FMovieSceneFloatValue& FloatValue = ChannelData.GetValues()[ExistingIndex];
		FloatValue.Value = Value.Value;
	}
	else
	{
		ChannelData.AddKey(Time, Value);
	}
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& ChannelData, FFrameNumber Time, float Value, 
		float ArriveTangent, float LeaveTangent, ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode, 
		FFrameRate FrameRate, ERichCurveTangentWeightMode WeightedMode, float ArriveTangentWeight, float LeaveTangentWeight)
{
	FMovieSceneFloatValue NewKey(Value);

	NewKey.InterpMode = InterpMode;
	NewKey.TangentMode = TangentMode;
	NewKey.Tangent.ArriveTangent = ArriveTangent / FrameRate.AsDecimal();
	NewKey.Tangent.LeaveTangent = LeaveTangent / FrameRate.AsDecimal();
	NewKey.Tangent.TangentWeightMode = WeightedMode;
	NewKey.Tangent.ArriveTangentWeight = ArriveTangentWeight;
	NewKey.Tangent.LeaveTangentWeight = LeaveTangentWeight;
	ChannelData.AddKey(Time, NewKey);
	MovieSceneToolHelpers::SetOrAddKey(ChannelData, Time, NewKey);
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& ChannelData, FFrameNumber Time, double Value, 
		float ArriveTangent, float LeaveTangent, ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode, 
		FFrameRate FrameRate, ERichCurveTangentWeightMode WeightedMode, float ArriveTangentWeight, float LeaveTangentWeight)
{
	FMovieSceneDoubleValue NewKey(Value);
	NewKey.InterpMode = InterpMode;
	NewKey.TangentMode = TangentMode;
	NewKey.Tangent.ArriveTangent = ArriveTangent / FrameRate.AsDecimal();
	NewKey.Tangent.LeaveTangent = LeaveTangent / FrameRate.AsDecimal();
	NewKey.Tangent.TangentWeightMode = WeightedMode;
	NewKey.Tangent.ArriveTangentWeight = ArriveTangentWeight;
	NewKey.Tangent.LeaveTangentWeight = LeaveTangentWeight;
	MovieSceneToolHelpers::SetOrAddKey(ChannelData, Time, NewKey);
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& ChannelData, FFrameNumber Time, FMovieSceneDoubleValue Value)
{
	int32 ExistingIndex = ChannelData.FindKey(Time);
	if (ExistingIndex != INDEX_NONE)
	{
		FMovieSceneDoubleValue& DoubleValue = ChannelData.GetValues()[ExistingIndex];
		DoubleValue.Value = Value.Value;
	}
	else
	{
		ChannelData.AddKey(Time, Value);
	}
}

void MovieSceneToolHelpers::SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& ChannelData, FFrameNumber Time, double Value, const EMovieSceneKeyInterpolation Interpolation)
{
	int32 ExistingIndex = ChannelData.FindKey(Time);
	if (ExistingIndex != INDEX_NONE)
	{
		FMovieSceneDoubleValue& DoubleValue = ChannelData.GetValues()[ExistingIndex];
		DoubleValue.Value = Value;
	}
	else
	{
		FMovieSceneDoubleValue NewKey(Value);
		ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone;
		NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
		NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
		NewKey.Tangent.ArriveTangent = 0.0f;
		NewKey.Tangent.LeaveTangent = 0.0f;
		NewKey.Tangent.TangentWeightMode = WeightedMode;
		NewKey.Tangent.ArriveTangentWeight = 0.0f;
		NewKey.Tangent.LeaveTangentWeight = 0.0f;

		switch (Interpolation)
		{
			case EMovieSceneKeyInterpolation::SmartAuto:    
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_SmartAuto;
			}
			break;
			case EMovieSceneKeyInterpolation::Auto:  
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;
			case EMovieSceneKeyInterpolation::User: 
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_User;
			}
			break;
			case EMovieSceneKeyInterpolation::Break:  
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;

			case EMovieSceneKeyInterpolation::Linear:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Linear;
			}
			break;

			case EMovieSceneKeyInterpolation::Constant:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Constant;
			}
			break;

		}
		ChannelData.AddKey(Time, NewKey);
	}
}

void MovieSceneToolHelpers::GetActorWorldTransforms(IMovieScenePlayer* Player, UMovieSceneSequence* InSequence, FMovieSceneSequenceIDRef Template, const FActorForWorldTransforms& ActorSelection, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutWorldTransforms)
{
	FGuid ActorHandle = GetHandleToObject(ActorSelection.Actor.Get(), InSequence, Player, Template,false);
	FGuid ComponentHandle = GetHandleToObject(ActorSelection.Component.Get(), InSequence, Player, Template,false);
	FMovieSceneSequenceTransform RootToLocalTransform;
	if (ActorHandle.IsValid() || ComponentHandle.IsValid())
	{
		//we can have handles but if they don't have a transform track the interrogator will return identity
		bool bHaveTransformTrack = false;
		if (ActorHandle.IsValid())
		{
			if (InSequence->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(ActorHandle))
			{
				bHaveTransformTrack = true;
			}
		}
		if (ComponentHandle.IsValid())
		{
			if (InSequence->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(ComponentHandle))
			{
				bHaveTransformTrack = true;
			}
		}
		if (bHaveTransformTrack)
		{
			GetSequencerActorWorldTransforms(Player, RootToLocalTransform,InSequence, Template, ActorSelection, Frames, OutWorldTransforms);
		}
		else
		{
			GetNonSequencerActorWorldTransforms(Player, RootToLocalTransform, InSequence, Template, ActorSelection, Frames, OutWorldTransforms);
		}
	}
	else
	{
		GetNonSequencerActorWorldTransforms(Player, RootToLocalTransform, InSequence, Template, ActorSelection, Frames, OutWorldTransforms);
	}
}

void MovieSceneToolHelpers::GetActorParents(const FActorForWorldTransforms& ActorSelection,
	TArray<FActorForWorldTransforms>& OutParentActors)
{
	if (ActorSelection.Actor.IsValid())
	{
		//has component so parent is next component
		if (ActorSelection.Component.IsValid() && ActorSelection.Component->GetAttachParent())
		{
			FActorForWorldTransforms OutParentActor;
			OutParentActor.Actor = ActorSelection.Actor;
			OutParentActor.Component = ActorSelection.Component->GetAttachParent();
			OutParentActors.AddUnique(OutParentActor);
			GetActorParents(OutParentActor, OutParentActors);
		}
		else if (AActor* ParentActor = ActorSelection.Actor->GetAttachParentActor())
		{
			FActorForWorldTransforms OutParentActor;
			OutParentActor.Actor = ParentActor;
			OutParentActors.AddUnique(OutParentActor);
			GetActorParents(OutParentActor, OutParentActors);
		}
	}
}

void MovieSceneToolHelpers::GetActorParentsWithAttachments(ISequencer* Sequencer, const FActorForWorldTransforms& ActorSelection,
	TArray<FActorForWorldTransforms>& OutParentActors)
{
	if (Sequencer == nullptr)
	{
		return;
	}

	UMovieScene* FocusedMovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (ActorSelection.Actor.IsValid())
	{
		FGuid Guid;
		if (ActorSelection.Component.Get())
		{
			Guid = Sequencer->GetHandleToObject(ActorSelection.Component.Get(), false);
		}
		if (!Guid.IsValid())
		{
			Guid = Sequencer->GetHandleToObject(ActorSelection.Actor.Get(), false);
		}
		if (Guid.IsValid())
		{
			for (UMovieSceneTrack* Track : FocusedMovieScene->FindTracks(UMovieScene3DConstraintTrack::StaticClass(), Guid))
			{
				if (UMovieScene3DConstraintTrack* ConstraintTrack = Cast<UMovieScene3DConstraintTrack>(Track))
				{
					for (UMovieSceneSection* ConstraintSection : ConstraintTrack->GetAllSections())
					{
						FMovieSceneObjectBindingID ConstraintBindingID = (Cast<UMovieScene3DConstraintSection>(ConstraintSection))->GetConstraintBindingID();
						if (auto BoundObjectsView = ConstraintBindingID.ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer); BoundObjectsView.Num())
						{
							TWeakObjectPtr<> ParentObject = BoundObjectsView[0];
							FActorForWorldTransforms OutParentActor;
							OutParentActor.Actor = Cast<AActor>(ParentObject.Get());;
							OutParentActors.AddUnique(OutParentActor);
							GetActorParents(OutParentActor, OutParentActors);
						}
					}
				}
			}
		}
		else
		{
			//has component so parent is next component
			if (ActorSelection.Component.IsValid() && ActorSelection.Component->GetAttachParent())
			{
				FActorForWorldTransforms OutParentActor;
				OutParentActor.Actor = ActorSelection.Actor;
				OutParentActor.Component = ActorSelection.Component->GetAttachParent();
				OutParentActors.AddUnique(OutParentActor);
				GetActorParents(OutParentActor, OutParentActors);
			}
			else if (AActor* ParentActor = ActorSelection.Actor->GetAttachParentActor())
			{
				FActorForWorldTransforms OutParentActor;
				OutParentActor.Actor = ParentActor;
				OutParentActors.AddUnique(OutParentActor);
				GetActorParents(OutParentActor, OutParentActors);
			}
		}
	}
}

void MovieSceneToolHelpers::GetActorsAndParentsKeyFrames(ISequencer* Sequencer, const FActorForWorldTransforms& InActor,
	const FFrameNumber& StartFrame, const FFrameNumber& EndFrame, TSortedMap<FFrameNumber, FFrameNumber>& OutFrameMap)
{
	UMovieSceneSequence* Sequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	TArray<FFrameNumber> FramesToUse;
	TArray<FActorForWorldTransforms> ParentActors;
	ParentActors.Add(InActor);
	GetActorParentsWithAttachments(Sequencer, InActor, ParentActors);

	for (FActorForWorldTransforms& Actor : ParentActors)
	{
		FGuid Guid;
		if (Actor.Component.Get())
		{
			Guid = Sequencer->GetHandleToObject(Actor.Component.Get(), false);
		}
		if (!Guid.IsValid())
		{
			Guid = Sequencer->GetHandleToObject(Actor.Actor.Get(), false);
		}
		if (Guid.IsValid())
		{
			for (UMovieSceneTrack* Track : MovieScene->FindTracks(UMovieScene3DTransformTrack::StaticClass(), Guid))
			{	
				if (Track)
				{
					for (UMovieSceneSection* Section : Track->GetAllSections())
					{
						if (Section)
						{
							TArrayView<FMovieSceneDoubleChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
							TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
								Channels, StartFrame, EndFrame);
							for (FFrameNumber& FrameNumber : TransformFrameTimes)
							{
								OutFrameMap.Add(FrameNumber, FrameNumber);
							}
						}
					}
				}
			}
		}
	}
}

void MovieSceneToolHelpers::CalculateFramesBetween(
	const UMovieScene* MovieScene,
	FFrameNumber StartFrame,
	FFrameNumber EndFrame,
	int32 FrameInc,
	TArray<FFrameNumber>& OutFrames)
{
	const bool bReverse = StartFrame > EndFrame;
	if (bReverse)
	{
		Swap(StartFrame, EndFrame);
	}

	//if the end frame is the last frame, we move it back one tick otherwise it's possible when evaluating it won't evaluate.
	if (EndFrame == MovieScene->GetPlaybackRange().GetUpperBoundValue())
	{
		--EndFrame;
	}
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayResolution = MovieScene->GetDisplayRate();

	const FFrameNumber StartTimeInDisplay = FFrameRate::TransformTime(FFrameTime(StartFrame), TickResolution, DisplayResolution).FloorToFrame();
	const FFrameNumber EndTimeInDisplay = FFrameRate::TransformTime(FFrameTime(EndFrame), TickResolution, DisplayResolution).CeilToFrame();

	OutFrames.Reserve(EndTimeInDisplay.Value - StartTimeInDisplay.Value + 1);

	if (bReverse)
	{
		int32 FrameIndex = 0;
		for (FFrameNumber DisplayFrameNumber = EndTimeInDisplay; DisplayFrameNumber >= StartTimeInDisplay; --DisplayFrameNumber)
		{
			if (FrameIndex % FrameInc == 0)
			{
				FFrameNumber TickFrameNumber = FFrameRate::TransformTime(FFrameTime(DisplayFrameNumber), DisplayResolution, TickResolution).FrameNumber;
				OutFrames.Add(TickFrameNumber);
			}
			++FrameIndex;
		}
	}
	else
	{
		int32 FrameIndex = 0;
		for (FFrameNumber DisplayFrameNumber = StartTimeInDisplay; DisplayFrameNumber <= EndTimeInDisplay; ++DisplayFrameNumber)
		{
			if (FrameIndex % FrameInc == 0)
			{
				FFrameNumber TickFrameNumber = FFrameRate::TransformTime(FFrameTime(DisplayFrameNumber), DisplayResolution, TickResolution).FrameNumber;
				OutFrames.Add(TickFrameNumber);
			}
			++FrameIndex;
		}
	}
}

bool MovieSceneToolHelpers::OptimizeSection(const FKeyDataOptimizationParams& InParams, UMovieSceneSection* InSection)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	for (FMovieSceneFloatChannel* Channel : FloatChannels)
	{
		Channel->Optimize(InParams);
	}
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
	{
		Channel->Optimize(InParams);
	}
	return true;
}
void MovieSceneToolHelpers::SplitSectionsByBlendType(EMovieSceneBlendType BlendType, const TArray<UMovieSceneSection*>& InSections, TArray<UMovieSceneSection*>& Sections, TArray<UMovieSceneSection*>& OutBlendSections)
{
	for (UMovieSceneSection* InSection : InSections)
	{
		if (InSection &&
			InSection->IsActive() &&
			InSection->GetBlendType().IsValid())
		{
			if (InSection->GetBlendType().Get() == BlendType)
			{
				OutBlendSections.Add(InSection);
			}
			else 
			{
				Sections.Add(InSection);
			}
		}
	}
}

bool MovieSceneToolHelpers::CollapseSection(TSharedPtr<ISequencer>& SequencerPtr, UMovieSceneTrack* OwnerTrack, TArray<UMovieSceneSection*> Sections,
	const FBakingAnimationKeySettings& InSettings)
{
	if (SequencerPtr.IsValid() && Sections.Num() > 1 && OwnerTrack)
	{
		TRange<FFrameNumber> Range(InSettings.StartFrame, InSettings.EndFrame);
		//we get the  first absolute section or if no absolute sections, first additive,
		// that's the one we collapse onto
		UMovieSceneSection* BaseSection = nullptr;
		
		FScopedTransaction Transaction(NSLOCTEXT("MovieSceneTools", "CollapseAllSections", "Collapse All Sections"));
		bool bTrackModified = false;
		for (int32 SectionIndex = Sections.Num() - 1; SectionIndex > 0; --SectionIndex)
		{
			BaseSection = Sections[SectionIndex - 1];
			UMovieSceneSection* Section = Sections[SectionIndex];
			if (BaseSection && Section && BaseSection->GetBlendType().IsValid() && Section->GetBlendType().IsValid())
			{
				if (bTrackModified == false)
				{
					OwnerTrack->Modify();
					bTrackModified = true;
				}
				BaseSection->Modify();
				TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
				if (Section->IsActive())//active sections merge them
				{
					TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = BaseSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
					TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = BaseSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
					if (BaseDoubleChannels.Num() > 0)
					{
						const int StartIndex = 0;
						const int EndIndex = BaseDoubleChannels.Num() - 1;

						MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
							Section, StartIndex, EndIndex, StartIndex, EndIndex);
						MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections);

					}
					else if (BaseFloatChannels.Num() > 0)
					{
						const int StartIndex = 0;
						const int EndIndex = BaseFloatChannels.Num() - 1;
						MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
							Section, StartIndex, EndIndex, StartIndex, EndIndex);
						MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections);
					}
				}

				//if merging override onto an additive we change it to an additive
				if (Section->GetBlendType().Get() == EMovieSceneBlendType::Override &&
					(BaseSection->GetBlendType().Get() == EMovieSceneBlendType::Additive))
				{
					BaseSection->SetBlendType(EMovieSceneBlendType::Override);
				}
				TArray<UMovieSceneSection*> AllSections = OwnerTrack->GetAllSections();
				int32 TrackSectionIndex = INDEX_NONE;
				if (AllSections.Find(Section, TrackSectionIndex))
				{
					if (TrackSectionIndex != INDEX_NONE)
					{
						OwnerTrack->RemoveSectionAt(TrackSectionIndex);
					}
				}
			}
		}
		if (InSettings.bReduceKeys && BaseSection)
		{
			FKeyDataOptimizationParams Params;
			Params.bAutoSetInterpolation = true;
			Params.Tolerance = InSettings.Tolerance;
			FMovieSceneChannelProxy& ChannelProxy = BaseSection->GetChannelProxy();
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = ChannelProxy.GetChannels<FMovieSceneFloatChannel>();

			for (FMovieSceneFloatChannel* Channel : FloatChannels)
			{
				Channel->Optimize(Params); //should also auto tangent
			}
			TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = ChannelProxy.GetChannels<FMovieSceneDoubleChannel>();
			for (FMovieSceneDoubleChannel* DChannel : DoubleChannels)
			{
				DChannel->Optimize(Params); //should also auto tangent
			}

		}
		//reset everything back
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
	return true;

}

UMovieScene3DTransformSection* MovieSceneToolHelpers::GetTransformSection(
	const ISequencer* InSequencer,
	const FGuid& InGuid,
	const FTransform& InDefaultTransform)
{
	if (!InSequencer || !InSequencer->GetFocusedMovieSceneSequence())
	{
		return nullptr;
	}

	UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}

	UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(InGuid);
	if (!TransformTrack)
	{
		return nullptr;
	}

	UMovieScene3DTransformSection* TransformSection =
		Cast<UMovieScene3DTransformSection>(TransformTrack->GetSectionToKey());
	if (TransformSection)
	{
		return TransformSection;
	}

	TransformSection = TransformTrack->GetAllSections().Num() > 0 ? Cast<UMovieScene3DTransformSection>(TransformTrack->GetAllSections()[0]) : nullptr;
	if (TransformSection)
	{
		return TransformSection;
	}

	//there really should be a section to key or at least one section, if not then we go through creating it.
	TransformTrack->Modify();

	static constexpr FFrameNumber Frame0;
	bool bSectionAdded = false;
	TransformSection =
		Cast<UMovieScene3DTransformSection>(TransformTrack->FindOrAddSection(Frame0, bSectionAdded));
	if (!TransformSection)
	{
		return nullptr;
	}

	TransformSection->Modify();
	if (bSectionAdded)
	{
		TransformSection->SetRange(TRange<FFrameNumber>::All());

		const FVector Location0 = InDefaultTransform.GetLocation();
		const FRotator Rotation0 = InDefaultTransform.GetRotation().Rotator();
		const FVector Scale3D0 = InDefaultTransform.GetScale3D();

		FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
		TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
			SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
		};

		if (DoubleChannels[0].Get())
		{
			DoubleChannels[0].Get()->SetDefault(Location0.X);
		}
		if (DoubleChannels[1].Get())
		{
			DoubleChannels[1].Get()->SetDefault(Location0.Y);
		}
		if (DoubleChannels[2].Get())
		{
			DoubleChannels[2].Get()->SetDefault(Location0.Z);
		}
		if (DoubleChannels[3].Get())
		{
			DoubleChannels[3].Get()->SetDefault(Rotation0.Roll);
		}
		if (DoubleChannels[4].Get())
		{
			DoubleChannels[4].Get()->SetDefault(Rotation0.Pitch);
		}
		if (DoubleChannels[5].Get())
		{
			DoubleChannels[5].Get()->SetDefault(Rotation0.Yaw);
		}
		if (DoubleChannels[6].Get())
		{
			DoubleChannels[6].Get()->SetDefault(Scale3D0.X);
		}
		if (DoubleChannels[7].Get())
		{
			DoubleChannels[7].Get()->SetDefault(Scale3D0.Y);
		}
		if (DoubleChannels[8].Get())
		{
			DoubleChannels[8].Get()->SetDefault(Scale3D0.Z);
		}
	}

	return TransformSection;
}

bool MovieSceneToolHelpers::AddTransformKeys(
	const UMovieScene3DTransformSection* InTransformSection,
	const TArray<FFrameNumber>& Frames,
	const TArray<FTransform>& InLocalTransforms,
	const EMovieSceneTransformChannel& InChannels)
{
	if (!InTransformSection)
	{
		return false;
	}

	if (Frames.IsEmpty() || Frames.Num() != InLocalTransforms.Num())
	{
		return false;
	}

	auto GetValue = [](const uint32 Index, const FVector& InLocation, const FRotator& InRotation, const FVector& InScale)
	{
		switch (Index)
		{
		case 0:
			return InLocation.X;
		case 1:
			return InLocation.Y;
		case 2:
			return InLocation.Z;
		case 3:
			return InRotation.Roll;
		case 4:
			return InRotation.Pitch;
		case 5:
			return InRotation.Yaw;
		case 6:
			return InScale.X;
		case 7:
			return InScale.Y;
		case 8:
			return InScale.Z;
		default:
			ensure(false);
			break;
		}
		return 0.0;
	};

	const bool bKeyTranslation = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Translation);
	const bool bKeyRotation = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Rotation);
	const bool bKeyScale = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Scale);

	TArray<uint32> ChannelsIndexToKey;
	if (bKeyTranslation)
	{
		ChannelsIndexToKey.Append({ 0,1,2 });
	}
	if (bKeyRotation)
	{
		ChannelsIndexToKey.Append({ 3,4,5 });
	}
	if (bKeyScale)
	{
		ChannelsIndexToKey.Append({ 6,7,8 });
	}

	FMovieSceneChannelProxy& SectionChannelProxy = InTransformSection->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
	};

	// set default
	const FTransform& LocalTransform0 = InLocalTransforms[0];
	const FVector Location0 = LocalTransform0.GetLocation();
	const FRotator Rotation0 = LocalTransform0.GetRotation().Rotator();
	const FVector Scale3D0 = LocalTransform0.GetScale3D();

	for (int32 ChannelIndex = 0; ChannelIndex < 9; ChannelIndex++)
	{
		if (DoubleChannels[ChannelIndex].Get())
		{
			if (!DoubleChannels[ChannelIndex].Get()->GetDefault().IsSet())
			{
				const double Value = GetValue(ChannelIndex, Location0, Rotation0, Scale3D0);
				DoubleChannels[ChannelIndex].Get()->SetDefault(Value);
			}
		}
	}

	// add keys
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		const FFrameNumber& Frame = Frames[Index];
		const FTransform& LocalTransform = InLocalTransforms[Index];

		const FVector Location = LocalTransform.GetLocation();
		const FRotator Rotation = LocalTransform.GetRotation().Rotator();
		const FVector Scale3D = LocalTransform.GetScale3D();

		for (const int32 ChannelIndex : ChannelsIndexToKey)
		{
			if (DoubleChannels[ChannelIndex].Get())
			{
				const double Value = GetValue(ChannelIndex, Location, Rotation, Scale3D);
				TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex].Get()->GetData();
				MovieSceneToolHelpers::SetOrAddKey(ChannelData, Frame, Value);
			}
		}
	}
	//need to handle euler flips.. 3,4,5 is rotation
	if (bKeyRotation)
	{
		//figure out start/endframe, frames may not be in order
		FFrameNumber StartTime = Frames[0], EndTime = Frames[0];
		for (const FFrameNumber& Frame : Frames)
		{
			if (Frame < StartTime)
			{
				StartTime = Frame;
			}
			else if (Frame > EndTime)
			{
				EndTime = Frame;
			}
		}
		TSortedMap<FFrameNumber, FFrameNumber> FrameSet;
		TRange<FFrameNumber> WithinRange(0, 0);
		WithinRange.SetLowerBoundValue(StartTime);
		WithinRange.SetUpperBoundValue(EndTime);
		TArray<FFrameNumber> KeyTimes;
		TArray<FKeyHandle> KeyHandles;
		for (int32 ChannelIndex  =  3; ChannelIndex <=5;  ++ChannelIndex)
		{
			KeyTimes.SetNum(0);
			KeyHandles.SetNum(0);
			if (DoubleChannels[ChannelIndex].Get())
			{
				TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex].Get()->GetData();
				ChannelData.GetKeys(WithinRange, &KeyTimes, &KeyHandles);
				for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num() - 1; ++KeyIndex)
				{
					const int32 ValueIndex = ChannelData.GetIndex(KeyHandles[KeyIndex]);
					const int32 NextValueIndex = ChannelData.GetIndex(KeyHandles[KeyIndex + 1]);

					TArrayView<const FMovieSceneDoubleValue> Values = DoubleChannels[ChannelIndex].Get()->GetValues(); //re-get the array since the winding may change values
					double Value = Values[ValueIndex].Value;
					double NextValue = Values[NextValueIndex].Value;
					FMath::WindRelativeAnglesDegrees(Value,NextValue);
					AssignValue(DoubleChannels[ChannelIndex].Get(), KeyHandles[KeyIndex + 1], NextValue);
				}
			}
		}
	}


	//now we need to set auto tangents
	for (const int32 ChannelIndex : ChannelsIndexToKey)
	{
		if (DoubleChannels[ChannelIndex].Get())
		{
			DoubleChannels[ChannelIndex].Get()->AutoSetTangents();
		}
	}

	return true;
}

void MovieSceneToolHelpers::AddTransformKeysForObject(UObject* Object, EMovieSceneTransformChannel ChannelToKey)
{
	if (F3DTransformTrackEditor::CurrentEditor)
	{
		F3DTransformTrackEditor::CurrentEditor->AddTransformKeysForObject(Object, ChannelToKey, ESequencerKeyMode::ManualKeyForced);
	}
}


void AddUnwoundKey(FMovieSceneDoubleChannel& Channel, FFrameNumber Time, double Value)
{
	int32 Index = Channel.AddLinearKey(Time, Value);

	TArrayView<FMovieSceneDoubleValue> Values = Channel.GetData().GetValues();
	if (Index >= 1)
	{
		const double PreviousValue = Values[Index - 1].Value;
		double NewValue = Value;

		while (NewValue - PreviousValue > 180.0f)
		{
			NewValue -= 360.f;
		}
		while (NewValue - PreviousValue < -180.0f)
		{
			NewValue += 360.f;
		}

		Values[Index].Value = NewValue;
	}
}

void MovieSceneToolHelpers::ImportAnimSequenceTransforms(const TSharedRef<ISequencer>& Sequencer, const FAssetData& Asset, UMovieScene3DTransformTrack* TransformTrack)
{
	if (!IsValid(TransformTrack))
	{
		return;
	}

	FSlateApplication::Get().DismissAllMenus();

	FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();

	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset.GetAsset());

	// find object binding to recover any component transforms we need to incorporate (for characters)
	FTransform InvComponentTransform;
	UMovieSceneSequence* MovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
	if(MovieSceneSequence)
	{
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		if(MovieScene)
		{
			FGuid ObjectBinding;
			if(MovieScene->FindTrackBinding(*TransformTrack, ObjectBinding))
			{
				const UClass* ObjectClass = nullptr;
				if(FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectBinding))
				{
					ObjectClass = Spawnable->GetObjectTemplate()->GetClass();
				}
				else if(FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBinding))
				{
					ObjectClass = Possessable->GetPossessedObjectClass();
				}

				if(ObjectClass)
				{
					const ACharacter* Character = Cast<const ACharacter>(ObjectClass->GetDefaultObject(false));
					if(Character)
					{
						const USkeletalMeshComponent* SkeletalMeshComponent = Character->GetMesh();
						FTransform MeshRelativeTransform = SkeletalMeshComponent->GetRelativeTransform();
						InvComponentTransform = MeshRelativeTransform.GetRelativeTransform(SkeletalMeshComponent->GetOwner()->GetTransform()).Inverse();
					}
				}
			}
		}
	}

	if (AnimSequence && AnimSequence->GetDataModel()->GetNumBoneTracks() > 0)
	{
		const FScopedTransaction Transaction( NSLOCTEXT( "Sequencer", "ImportAnimSequenceTransforms", "Import Anim Sequence Transforms" ) );

		TransformTrack->Modify();

		UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());

		Section->SetBlendType(EMovieSceneBlendType::Additive);
		Section->SetMask(EMovieSceneTransformChannel::Translation | EMovieSceneTransformChannel::Rotation);
		
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		// We know that there aren't any channel overrides (all 9 channels are double channels) because we're working
		// on a brand new transform section.
		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		// Set default translation and rotation
		for (int32 Index = 0; Index < 6; ++Index)
		{
			DoubleChannels[Index]->SetDefault(0.f);
		}
		// Set default scale
		for (int32 Index = 6; Index < 9; ++Index)
		{
			DoubleChannels[Index]->SetDefault(1.f);
		}

		TransformTrack->AddSection(*Section);

		if (Section->TryModify())
		{
			struct FTempTransformKey
			{
				FTransform Transform;
				FRotator WoundRotation;
				float Time;
			};

			float MinTime = FLT_MAX;

			TArray<FTempTransformKey> TempKeys;

			TArray<FName> BoneTrackNames;
			AnimSequence->GetDataModelInterface()->GetBoneTrackNames(BoneTrackNames);

			TArray<FTransform> BoneTransforms;
			AnimSequence->GetDataModelInterface()->GetBoneTrackTransforms(BoneTrackNames[0], BoneTransforms);

			for(int32 KeyIndex = 0; KeyIndex < BoneTransforms.Num(); KeyIndex++)
			{
				FTempTransformKey TempKey;
				TempKey.Time = AnimSequence->GetTimeAtFrame(KeyIndex);
				TempKey.Transform = BoneTransforms[KeyIndex];

				// apply component transform if any
				TempKey.Transform = InvComponentTransform * TempKey.Transform;

				TempKey.WoundRotation = TempKey.Transform.GetRotation().Rotator();

				TempKeys.Add(TempKey);

				MinTime = FMath::Min(MinTime, TempKey.Time);
			}

			int32 TransformCount = TempKeys.Num();
			for(int32 TransformIndex = 0; TransformIndex < TransformCount - 1; TransformIndex++)
			{
				FRotator& Rotator = TempKeys[TransformIndex].WoundRotation;
				FRotator& NextRotator = TempKeys[TransformIndex + 1].WoundRotation;

				FMath::WindRelativeAnglesDegrees(Rotator.Pitch, NextRotator.Pitch);
				FMath::WindRelativeAnglesDegrees(Rotator.Yaw, NextRotator.Yaw);
				FMath::WindRelativeAnglesDegrees(Rotator.Roll, NextRotator.Roll);
			}

			FFrameNumber MinKeyTime = (MinTime * TickResolution).RoundToFrame();

			TRange<FFrameNumber> Range = Section->GetRange();
			for(const FTempTransformKey& TempKey : TempKeys)
			{
				FFrameNumber KeyTime = (TempKey.Time * TickResolution).RoundToFrame();

				KeyTime = CurrentTime.Time.FrameNumber + (KeyTime - MinKeyTime);

				Range = TRange<FFrameNumber>::Hull(Range, TRange<FFrameNumber>(KeyTime));

				const FVector3f Translation = (FVector3f)TempKey.Transform.GetTranslation();
				const FVector3f Rotation = (FVector3f)TempKey.WoundRotation.Euler();
				const FVector3f Scale = (FVector3f)TempKey.Transform.GetScale3D();

				TArrayView<FMovieSceneDoubleChannel*> Channels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

				Channels[0]->AddLinearKey(KeyTime, Translation.X);
				Channels[1]->AddLinearKey(KeyTime, Translation.Y);
				Channels[2]->AddLinearKey(KeyTime, Translation.Z);

				AddUnwoundKey(*Channels[3], KeyTime, Rotation.X);
				AddUnwoundKey(*Channels[4], KeyTime, Rotation.Y);
				AddUnwoundKey(*Channels[5], KeyTime, Rotation.Z);

				Channels[6]->AddLinearKey(KeyTime, Scale.X);
				Channels[7]->AddLinearKey(KeyTime, Scale.Y);
				Channels[8]->AddLinearKey(KeyTime, Scale.Z);
			}

			Section->SetRange(Range);
			Section->SetRowIndex(FindAvailableRowIndex(TransformTrack, Section));

			Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
		}
	}
}

void MovieSceneToolHelpers::ImportAnimSequenceTransformsEnterPressed(const TSharedRef<ISequencer>& Sequencer, const TArray<FAssetData>& Asset, UMovieScene3DTransformTrack* TransformTrack)
{
	if (Asset.Num() > 0)
	{
		ImportAnimSequenceTransforms(Sequencer, Asset[0].GetAsset(), TransformTrack);
	}
}
