// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneToolsModule.h"

#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Curves/RichCurve.h"
#include "ISequencerCoreModule.h"
#include "ISequencerModule.h"
#include "ICurveEditorModule.h"
#include "MovieSceneToolsProjectSettingsCustomization.h"
#include "MovieSceneCVarOverridesPropertyTypeCustomization.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_FunctionEntry.h"
#include "KismetCompiler.h"

#include "Tracks/MovieSceneTransformTrack.h"

#include "Sections/MovieSceneEventSectionBase.h"
#include "Sections/MovieSceneParticleSection.h"

#include "Sections/BoolPropertySection.h"
#include "Sections/ColorPropertySection.h"

#include "Sections/MovieSceneColorSection.h"
#include "Tracks/MovieSceneColorTrack.h"

#include "Sections/MovieSceneBoolSection.h"
#include "Tracks/MovieSceneBoolTrack.h"

#include "TrackEditors/PropertyTrackEditors/BytePropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/RotatorPropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/VisibilityPropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/ActorReferencePropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/StringPropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/RegisteredPropertyTrackEditor.h"
#include "TrackEditors/PropertyTrackEditors/TextPropertyTrackEditor.h"

#include "TrackEditors/TransformTrackEditor.h"
#include "TrackEditors/CameraCutTrackEditor.h"
#include "TrackEditors/CinematicShotTrackEditor.h"
#include "TrackEditors/SlomoTrackEditor.h"
#include "TrackEditors/SubTrackEditor.h"
#include "TrackEditors/AudioTrackEditor.h"
#include "TrackEditors/SkeletalAnimationTrackEditor.h"
#include "TrackEditors/ParticleTrackEditor.h"
#include "TrackEditors/ParticleParameterTrackEditor.h"
#include "TrackEditors/AttachTrackEditor.h"
#include "TrackEditors/EventTrackEditor.h"
#include "TrackEditors/PathTrackEditor.h"
#include "TrackEditors/MaterialTrackEditor.h"
#include "TrackEditors/FadeTrackEditor.h"
#include "TrackEditors/SpawnTrackEditor.h"
#include "TrackEditors/LevelVisibilityTrackEditor.h"
#include "TrackEditors/DataLayerTrackEditor.h"
#include "TrackEditors/CameraShakeTrackEditor.h"
#include "TrackEditors/MaterialParameterCollectionTrackEditor.h"
#include "TrackEditors/ObjectPropertyTrackEditor.h"
#include "TrackEditors/PrimitiveMaterialTrackEditor.h"
#include "TrackEditors/CameraShakeSourceShakeTrackEditor.h"
#include "TrackEditors/CVarTrackEditor.h"
#include "TrackEditors/CustomPrimitiveDataTrackEditor.h"
#include "TrackEditors/BindingLifetimeTrackEditor.h"
#include "TrackEditors/TimeWarpTrackEditor.h"

#include "Channels/PerlinNoiseChannelInterface.h"

#include "MVVM/ViewModels/CameraCutTrackModel.h"
#include "MVVM/ViewModels/CinematicShotTrackModel.h"
#include "MVVM/ViewModels/BindingLifetimeTrackModel.h"
#include "MVVM/ViewModels/ScalingAnchorsModel.h"

#include "Decorations/MovieSceneScalingAnchors.h"

#include "MovieSceneBuiltInEasingFunctionCustomization.h"
#include "MovieSceneAlphaBlendOptionCustomization.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "MovieSceneDynamicBindingCustomization.h"
#include "MovieSceneEventCustomization.h"
#include "MovieSceneTimeWarpVariantCustomization.h"
#include "Conditions/MovieSceneConditionCustomization.h"
#include "SequencerClipboardReconciler.h"
#include "ClipboardTypes.h"
#include "ISettingsModule.h"
#include "PropertyEditorModule.h"
#include "IMovieSceneTools.h"
#include "IMovieSceneToolsTrackImporter.h"
#include "MovieSceneToolsProjectSettings.h"

#include "ISequencerChannelInterface.h"
#include "SequencerChannelInterface.h"
#include "Channels/BuiltInChannelEditors.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneEventChannel.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "Channels/MovieSceneCameraShakeSourceTriggerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannel.h"
#include "Channels/MovieSceneDoublePerlinNoiseChannel.h"
#include "Channels/EventChannelCurveModel.h"
#include "Channels/SCurveEditorEventChannelView.h"
#include "Channels/MovieSceneAudioTriggerChannel.h"
#include "ConstraintChannel.h"
#include "Channels/ConstraintChannelEditor.h"
#include "Channels/ConstraintChannelCurveModel.h"
#include "Channels/SCurveEditorKeyBarView.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/TransformPropertySection.h"


#include "Channels/MovieSceneDoublePerlinNoiseChannelContainer.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannelContainer.h"
#include "Channels/PerlinNoiseChannelDetailsCustomization.h"

#include "MovieSceneEventUtils.h"

#include "EntitySystem/MovieSceneEntityManager.h"
#include "EditorModeManager.h"
#include "EditModes/SkeletalAnimationTrackEditMode.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"

#include "LevelSequence.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "AnimSequenceLevelSequenceLink.h"
#include "EditorAnimUtils.h"
#include "Animation/AnimSequence.h"

#include "Transform/TransformableHandle.h"
#include "Constraints/ComponentConstraintChannelInterface.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "Bindings/MovieSceneReplaceableDirectorBlueprintBinding.h"
#include "Bindings/MovieSceneSpawnableDirectorBlueprintBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneDynamicBindingUtils.h"
#include "EditModes/SubTrackEditorMode.h"
#include "Kismet2/KismetEditorUtilities.h"

#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionUtils.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionCustomization.h"
#include "Conditions/MovieScenePlatformConditionCustomization.h"
#include "IMovieSceneRootMotionOffsetProvider.h"
#include "IMovieSceneEditingContextLayerResolver.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#define LOCTEXT_NAMESPACE "FMovieSceneToolsModule"

FName IMovieSceneRootMotionOffsetProvider::GetModularFeatureName()
{
	static FName Name(TEXT("MovieSceneRootMotionOffsetProvider"));
	return Name;
}

FName IMovieSceneEditingContextLayerResolver::GetModularFeatureName()
{
	static FName Name(TEXT("MovieSceneEditingContextLayerResolver"));
	return Name;
}

FTransform FSkeletalAnimationRootMotionOffsetProvider::GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) const
{
	if (!Linker)
	{
		return FTransform::Identity;
	}
	if (const UMovieSceneSkeletalAnimationSystem* System = Linker->FindSystem<UMovieSceneSkeletalAnimationSystem>())
	{
		return System->GetRootMotionOffset(AnimatedObject);
	}
	return FTransform::Identity;
}

void FSkeletalAnimationRootMotionOffsetProvider::UpdateRootMotionOffset(UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject)
{
	if (!Linker)
	{
		return;
	}
	if (UMovieSceneSkeletalAnimationSystem* System = Linker->FindSystem<UMovieSceneSkeletalAnimationSystem>())
	{
		System->UpdateRootMotionOffset(AnimatedObject);
	}
}

TAutoConsoleVariable<bool> CVarDuplicateLinkedAnimSequence(TEXT("Sequencer.DuplicateLinkedAnimSequence"), false, TEXT("When true when we duplicate a level sequence that has a linked anim sequence it will duplicate and link the anim sequencel, if false we leave any link alone."));


void FMovieSceneToolsModule::StartupModule()
{
	using namespace UE::Sequencer;

	if (GIsEditor)
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Editor", "Level Sequences",
				LOCTEXT("RuntimeSettingsName", "Level Sequences"),
				LOCTEXT("RuntimeSettingsDescription", "Configure project settings relating to Level Sequences"),
				GetMutableDefault<UMovieSceneToolsProjectSettings>()
			);
		}

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>( "Sequencer" );

		RegisterPropertyType(MakeShared<TRegisteredProperty<UMovieSceneBoolTrack, UMovieSceneBoolSection>>([](UMovieSceneSection* SectionObject, UMovieSceneTrack* Track, const FGuid& ObjectBinding, TWeakPtr<ISequencer> WeakSequencer){
			return TSharedPtr<ISequencerSection>(MakeShared<FBoolPropertySection>(*SectionObject));
		}));
		RegisterPropertyType(MakeShared<TRegisteredProperty<UMovieSceneTransformTrack, UMovieScene3DTransformSection>>([](UMovieSceneSection* SectionObject, UMovieSceneTrack* Track, const FGuid& ObjectBinding, TWeakPtr<ISequencer> WeakSequencer){
			return TSharedPtr<ISequencerSection>(MakeShared<FTransformSection>(*SectionObject, WeakSequencer));
		}));
		RegisterPropertyType(MakeShared<TRegisteredProperty<UMovieSceneColorTrack, UMovieSceneColorSection>>([](UMovieSceneSection* SectionObject, UMovieSceneTrack* Track, const FGuid& ObjectBinding, TWeakPtr<ISequencer> WeakSequencer){
			return TSharedPtr<ISequencerSection>(MakeShared<FColorPropertySection>(*SectionObject, ObjectBinding, WeakSequencer));
		}));


		// register property track editors
		BytePropertyTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FBytePropertyTrackEditor>();
		RotatorPropertyTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FRotatorPropertyTrackEditor>();
		VisibilityPropertyTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(FVisibilityPropertyTrackEditor::CreateTrackEditor));
		ActorReferencePropertyTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FActorReferencePropertyTrackEditor>();
		StringPropertyTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FStringPropertyTrackEditor>();
		TextPropertyTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FTextPropertyTrackEditor>();
		ObjectTrackCreateEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FObjectPropertyTrackEditor>();

		// register specialty track editors
		AnimationTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FSkeletalAnimationTrackEditor::CreateTrackEditor ) );
		AttachTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &F3DAttachTrackEditor::CreateTrackEditor ) );
		AudioTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FAudioTrackEditor::CreateTrackEditor ) );
		EventTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FEventTrackEditor::CreateTrackEditor ) );
		ParticleTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FParticleTrackEditor::CreateTrackEditor ) );
		ParticleParameterTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FParticleParameterTrackEditor::CreateTrackEditor ) );
		PathTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &F3DPathTrackEditor::CreateTrackEditor ) );
		CameraCutTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FCameraCutTrackEditor::CreateTrackEditor ) );
		CinematicShotTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FCinematicShotTrackEditor::CreateTrackEditor ) );
		SlomoTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FSlomoTrackEditor::CreateTrackEditor ) );
		SubTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FSubTrackEditor::CreateTrackEditor ) );
		TransformTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &F3DTransformTrackEditor::CreateTrackEditor ) );
		ComponentMaterialTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FComponentMaterialTrackEditor::CreateTrackEditor ) );
		FadeTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FFadeTrackEditor::CreateTrackEditor ) );
		SpawnTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FSpawnTrackEditor::CreateTrackEditor ) );
		LevelVisibilityTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FLevelVisibilityTrackEditor::CreateTrackEditor ) );
		DataLayerTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor( FOnCreateTrackEditor::CreateStatic( &FDataLayerTrackEditor::CreateTrackEditor ) );
		CameraShakeTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FCameraShakeTrackEditor::CreateTrackEditor));
		MPCTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FMaterialParameterCollectionTrackEditor::CreateTrackEditor));
		PrimitiveMaterialCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FPrimitiveMaterialTrackEditor::CreateTrackEditor));
		CameraShakeSourceShakeCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FCameraShakeSourceShakeTrackEditor::CreateTrackEditor));
		CVarTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FCVarTrackEditor::CreateTrackEditor));
		CustomPrimitiveDataTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FCustomPrimitiveDataTrackEditor::CreateTrackEditor));
		BindingLifetimeTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FBindingLifetimeTrackEditor::CreateTrackEditor));
		TimeWarpTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FTimeWarpTrackEditor::CreateTrackEditor));
		RegisteredPropertyTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FRegisteredPropertyTrackEditor::CreateTrackEditor));

		// register track models
		CameraCutTrackModelHandle = SequencerModule.RegisterTrackModel(FOnCreateTrackModel::CreateStatic(&FCameraCutTrackModel::CreateTrackModel));
		CinematicShotTrackModelHandle = SequencerModule.RegisterTrackModel(FOnCreateTrackModel::CreateStatic(&FCinematicShotTrackModel::CreateTrackModel));
		BindingLifetimeTrackModelHandle = SequencerModule.RegisterTrackModel(FOnCreateTrackModel::CreateStatic(&FBindingLifetimeTrackModel::CreateTrackModel));
		TimeWarpTrackModelHandle = SequencerModule.RegisterTrackModel(FOnCreateTrackModel::CreateStatic(&FTimeWarpTrackEditor::CreateTrackModel));

		RegisterClipboardConversions();

		// register details customization
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("MovieSceneToolsProjectSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FMovieSceneToolsProjectSettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("MovieSceneBuiltInEasingFunction", FOnGetDetailCustomizationInstance::CreateLambda(&MakeShared<FMovieSceneBuiltInEasingFunctionCustomization>));
		PropertyModule.RegisterCustomPropertyTypeLayout("EAlphaBlendOption", FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FAlphaBlendPropertyCustomization>));
		PropertyModule.RegisterCustomClassLayout("MovieSceneFloatPerlinNoiseChannelContainer", FOnGetDetailCustomizationInstance::CreateStatic(&FMovieSceneFloatPerlinNoiseChannelDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("MovieSceneDoublePerlinNoiseChannelContainer", FOnGetDetailCustomizationInstance::CreateStatic(&FMovieSceneDoublePerlinNoiseChannelDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneObjectBindingID", FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<FMovieSceneObjectBindingIDCustomization>));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneDynamicBinding", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneDynamicBindingCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneDirectorBlueprintConditionData", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneEvent", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneEventCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneCVarOverrides", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MovieScene::FCVarOverridesPropertyTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneTimeWarpVariant", FOnGetPropertyTypeCustomizationInstance::CreateLambda(&MakeShared<UE::MovieScene::FMovieSceneTimeWarpVariantCustomization>));
		PropertyModule.RegisterCustomPropertyTypeLayout("MovieSceneConditionContainer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneConditionCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("MovieScenePlatformCondition", FOnGetDetailCustomizationInstance::CreateStatic(&FMovieScenePlatformConditionCustomization::MakeInstance));


		SequencerModule.RegisterChannelInterface<FMovieSceneBoolChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneByteChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneIntegerChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneFloatChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneDoubleChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneStringChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneTextChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneParticleChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneActorReferenceChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneEventSectionData>();
		SequencerModule.RegisterChannelInterface<FMovieSceneObjectPathChannel>();
		SequencerModule.RegisterChannelInterface<FMovieSceneTimeWarpChannel>();

		SequencerModule.RegisterChannelInterface<FMovieSceneAudioTriggerChannel>();

		SequencerModule.RegisterChannelInterface<FMovieSceneEventChannel>();

		SequencerModule.RegisterChannelInterface<FMovieSceneCameraShakeSourceTriggerChannel>();

		SequencerModule.RegisterChannelInterface<FMovieSceneConstraintChannel>();

		SequencerModule.RegisterChannelInterface<FMovieSceneFloatPerlinNoiseChannel>(MakeUnique<TPerlinNoiseChannelInterface<UMovieSceneFloatPerlinNoiseChannelContainer>>());
		SequencerModule.RegisterChannelInterface<FMovieSceneDoublePerlinNoiseChannel>(MakeUnique<TPerlinNoiseChannelInterface<UMovieSceneDoublePerlinNoiseChannelContainer>>());

		ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>("CurveEditor");

		FEventChannelCurveModel::EventView = CurveEditorModule.RegisterView(FOnCreateCurveEditorView::CreateStatic(
			[](TWeakPtr<FCurveEditor> WeakCurveEditor) -> TSharedRef<SCurveEditorView>
			{
				return SNew(SCurveEditorEventChannelView, WeakCurveEditor);
			}
		));

		FConstraintChannelCurveModel::ViewID = CurveEditorModule.RegisterView(FOnCreateCurveEditorView::CreateStatic(
			[](TWeakPtr<FCurveEditor> WeakCurveEditor) -> TSharedRef<SCurveEditorView>
			{
				return SNew(SCurveEditorKeyBarView, WeakCurveEditor);
			}
		));

		ISequencerCoreModule& SequencerCoreModule = FModuleManager::Get().LoadModuleChecked<ISequencerCoreModule>("SequencerCore");
		SequencerCoreModule.RegisterModelType(UMovieSceneScalingAnchors::StaticClass(), FScalingAnchorsModel::ID);
	}

	FixupDynamicBindingPayloadParameterNameHandle = UMovieScene::FixupDynamicBindingPayloadParameterNameEvent.AddStatic(FixupPayloadParameterNameForDynamicBinding);
	FixupEventSectionPayloadParameterNameHandle = UMovieSceneEventSectionBase::FixupPayloadParameterNameEvent.AddStatic(FixupPayloadParameterNameForSection);
	UMovieSceneEventSectionBase::UpgradeLegacyEventEndpoint.BindStatic(UpgradeLegacyEventEndpointForSection);
	UMovieSceneEventSectionBase::PostDuplicateSectionEvent.BindStatic(PostDuplicateEventSection);
	UMovieSceneEventSectionBase::RemoveForCookEvent.BindStatic(RemoveForCookEventSection);
	UMovieScene::IsTrackClassAllowedEvent.BindStatic(IsTrackClassAllowed);
	UMovieScene::IsCustomBindingClassAllowedEvent.BindStatic(IsCustomBindingClassAllowed);
	UMovieScene::IsConditionClassAllowedEvent.BindStatic(IsConditionClassAllowed);
	ULevelSequence::PostDuplicateEvent.BindStatic(PostDuplicateEvent);
	FixupDynamicBindingsHandle = ULevelSequence::FixupDynamicBindingsEvent.AddStatic(FixupDynamicBindingsEvent); 
	FixupDirectorBlueprintConditionPayloadParameterNameHandle = UMovieScene::FixupDirectorBlueprintConditionPayloadParameterNameEvent.AddStatic(FixupPayloadParameterNameForDirectorBlueprintCondition);



	auto OnObjectsReplaced = [](const TMap<UObject*, UObject*>& ReplacedObjects)
	{
		// If a movie scene signed object is reinstanced, it has to be marked as modified
		// so that the data gets recompiled properly.
		// @todo: this might cause cook non-determinism, but we need to verify that separately
		for (const TTuple<UObject*, UObject*>& Pair : ReplacedObjects)
		{
			if (UMovieSceneSignedObject* SignedObject = Cast<UMovieSceneSignedObject>(Pair.Value))
			{
				SignedObject->MarkAsChanged();
			}
		}
	};

	OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddLambda(OnObjectsReplaced);

	FEditorModeRegistry::Get().RegisterMode<FSkeletalAnimationTrackEditMode>(
		FSkeletalAnimationTrackEditMode::ModeName,
		NSLOCTEXT("SkeletalAnimationTrackEditorMode", "SkelAnimTrackEditMode", "Skeletal Anim Track Mode"),
		FSlateIcon(),
		false);

	FEditorModeRegistry::Get().RegisterMode<FSubTrackEditorMode>(
		FSubTrackEditorMode::ModeName,
		NSLOCTEXT("SubTrackEditorMode", "SubTrackEditMode", "Sub Track Mode"),
		FSlateIcon(),
		false);

	// register UTransformableComponentHandle animatable interface
	FConstraintChannelInterfaceRegistry& ConstraintChannelInterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();
	ConstraintChannelInterfaceRegistry.RegisterConstraintChannelInterface<UTransformableComponentHandle>(MakeUnique<FComponentConstraintChannelInterface>());

	// Register legacy skeletal animation root motion offset provider
	IModularFeatures::Get().RegisterModularFeature(IMovieSceneRootMotionOffsetProvider::GetModularFeatureName(), &SkeletalAnimRootMotionProvider);
}

void FMovieSceneToolsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IMovieSceneRootMotionOffsetProvider::GetModularFeatureName(), &SkeletalAnimRootMotionProvider);
	UMovieScene::FixupDynamicBindingPayloadParameterNameEvent.Remove(FixupDynamicBindingPayloadParameterNameHandle);
	UMovieSceneEventSectionBase::FixupPayloadParameterNameEvent.Remove(FixupEventSectionPayloadParameterNameHandle);
	UMovieSceneEventSectionBase::UpgradeLegacyEventEndpoint = UMovieSceneEventSectionBase::FUpgradeLegacyEventEndpoint();
	UMovieSceneEventSectionBase::PostDuplicateSectionEvent = UMovieSceneEventSectionBase::FPostDuplicateEvent();
	UMovieSceneEventSectionBase::RemoveForCookEvent = UMovieSceneEventSectionBase::FRemoveForCookEvent();
	UMovieScene::IsTrackClassAllowedEvent = UMovieScene::FIsTrackClassAllowedEvent();
	UMovieScene::IsCustomBindingClassAllowedEvent = UMovieScene::FIsCustomBindingClassAllowedEvent();
	UMovieScene::IsConditionClassAllowedEvent = UMovieScene::FIsConditionClassAllowedEvent();
	ULevelSequence::PostDuplicateEvent = ULevelSequence::FPostDuplicateEvent();
	ULevelSequence::FixupDynamicBindingsEvent.Remove(FixupDynamicBindingsHandle);
	UMovieScene::FixupDirectorBlueprintConditionPayloadParameterNameEvent.Remove(FixupDirectorBlueprintConditionPayloadParameterNameHandle);

	if (ICurveEditorModule* CurveEditorModule = FModuleManager::GetModulePtr<ICurveEditorModule>("CurveEditor"))
	{
		CurveEditorModule->UnregisterView(FEventChannelCurveModel::EventView);
		CurveEditorModule->UnregisterView(FConstraintChannelCurveModel::ViewID);
	}

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Editor", "Level Sequences");
	}

	FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle);

	if (!FModuleManager::Get().IsModuleLoaded("Sequencer"))
	{
		return;
	}

	ISequencerModule& SequencerModule = FModuleManager::Get().GetModuleChecked<ISequencerModule>( "Sequencer" );

	// unregister property track editors
	SequencerModule.UnRegisterTrackEditor( BytePropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( RotatorPropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( VisibilityPropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( ActorReferencePropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( StringPropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( TextPropertyTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CameraShakeSourceShakeCreateEditorHandle );

	// unregister specialty track editors
	SequencerModule.UnRegisterTrackEditor( AnimationTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( AttachTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( AudioTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( EventTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( ParticleTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( ParticleParameterTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( PathTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CameraCutTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CinematicShotTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( SlomoTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( SubTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( TransformTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( ComponentMaterialTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( FadeTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( SpawnTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( LevelVisibilityTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( DataLayerTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CameraShakeTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( MPCTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( ObjectTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( PrimitiveMaterialCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CVarTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( CustomPrimitiveDataTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( BindingLifetimeTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( TimeWarpTrackCreateEditorHandle );
	SequencerModule.UnRegisterTrackEditor( RegisteredPropertyTrackCreateEditorHandle );

	// unregister track models
	SequencerModule.UnregisterTrackModel( CameraCutTrackModelHandle );
	SequencerModule.UnregisterTrackModel( CinematicShotTrackModelHandle );
	SequencerModule.UnregisterTrackModel( BindingLifetimeTrackModelHandle );
	SequencerModule.UnregisterTrackModel( TimeWarpTrackModelHandle );

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{	
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("MovieSceneToolsProjectSettings");
		PropertyModule.UnregisterCustomClassLayout("MovieSceneBuiltInEasingFunction");
		PropertyModule.UnregisterCustomPropertyTypeLayout("EAlphaBlendOption");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MovieSceneObjectBindingID");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MovieSceneEvent");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MovieSceneDynamicBinding");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MovieSceneDirectorBlueprintConditionData");
		PropertyModule.UnregisterCustomPropertyTypeLayout("MovieSceneConditionContainer");
		PropertyModule.UnregisterCustomClassLayout("MovieScenePlatformCondition");
	}

	FEditorModeRegistry::Get().UnregisterMode(FSkeletalAnimationTrackEditMode::ModeName);
	FEditorModeRegistry::Get().UnregisterMode(FSubTrackEditorMode::ModeName);

}

void FMovieSceneToolsModule::PostDuplicateEventSection(UMovieSceneEventSectionBase* Section)
{
	UMovieSceneSequence*       Sequence           = Section->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor     = FMovieSceneSequenceEditor::Find(Sequence);
	UBlueprint*                SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	FMovieSceneEventUtils::ClearInvalidEndpointsForEventSection(Section);

	if (SequenceDirectorBP)
	{
		// Always bind the event section onto the blueprint to ensure that we get another chance to upgrade when the BP compiles if this try wasn't successful
		FMovieSceneEventUtils::BindEventSectionToBlueprint(Section, SequenceDirectorBP);
	}
}

void FMovieSceneToolsModule::RemoveForCookEventSection(UMovieSceneEventSectionBase* Section)
{
	UMovieSceneSequence*       Sequence           = Section->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor     = FMovieSceneSequenceEditor::Find(Sequence);
	UBlueprint*                SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	if (SequenceDirectorBP)
	{
		FMovieSceneEventUtils::RemoveEndpointsForEventSection(Section, SequenceDirectorBP);
	}
}

//When we duplicate a ULevelSequence we check to see if there are any linked UAnimSequences in the asset user data,
//if so we either make a copy of the anim sequence, or leave it alone since a rename can also be a duplicate, the user will need to clean up this link later.
void FMovieSceneToolsModule::PostDuplicateEvent(ULevelSequence* LevelSequence)
{
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				const bool bDuplicateAnimSequence = CVarDuplicateLinkedAnimSequence.GetValueOnGameThread();
				if (bDuplicateAnimSequence)
				{
					for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
					{
						if (UAnimSequence* AnimSequence = Item.ResolveAnimSequence())
						{
							TArray<UAnimSequence*> AnimSequencesToDuplicate;
							AnimSequencesToDuplicate.Add(AnimSequence);
							UPackage* DestinationPackage = AnimSequence->GetPackage();
							EditorAnimUtils::FNameDuplicationRule NameRule;
							NameRule.FolderPath = FPackageName::GetLongPackagePath(AnimSequence->GetPathName()) / TEXT("");
							TMap<UAnimSequence*, UAnimSequence*> DuplicatedAnimAssets = EditorAnimUtils::DuplicateAssets<UAnimSequence>(AnimSequencesToDuplicate, DestinationPackage, &NameRule);
							for (TPair<UAnimSequence*, UAnimSequence*>& Duplicates : DuplicatedAnimAssets)
							{
								if (UAnimSequence* NewAnimSequence = Duplicates.Value)
								{
									Item.PathToAnimSequence = FSoftObjectPath(NewAnimSequence);
									if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(NewAnimSequence))
									{
										UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
										if (AnimLevelLink)
										{
											AnimLevelLink->SetLevelSequence(LevelSequence);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}


void FMovieSceneToolsModule::FixupDynamicBindingsEvent(ULevelSequence* LevelSequence)
{
	if (LevelSequence)
	{
		if (UBlueprint* DirectorBlueprint = LevelSequence->GetDirectorBlueprint())
		{
			FMovieSceneDynamicBindingUtils::EnsureBlueprintExtensionCreated(LevelSequence, DirectorBlueprint);
			FKismetEditorUtilities::CompileBlueprint(DirectorBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
		}
	}
}

bool FMovieSceneToolsModule::UpgradeLegacyEventEndpointForSection(UMovieSceneEventSectionBase* Section)
{
	UMovieSceneSequence*       Sequence           = Section->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor     = FMovieSceneSequenceEditor::Find(Sequence);
	UBlueprint*                SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	if (!SequenceDirectorBP)
	{
		return true;
	}

	// Always bind the event section onto the blueprint to ensure that we get another chance to upgrade when the BP compiles if this try wasn't successful
	FMovieSceneEventUtils::BindEventSectionToBlueprint(Section, SequenceDirectorBP);

	// We can't do this upgrade if we any of the function graphs are RF_NeedLoad
	for (UEdGraph* EdGraph : SequenceDirectorBP->FunctionGraphs)
	{
		if (EdGraph->HasAnyFlags(RF_NeedLoad))
		{
			return false;
		}
	}

	// All the function graphs have been loaded, which means this is a good time to perform legacy data upgrade
	for (FMovieSceneEvent& EntryPoint : Section->GetAllEntryPoints())
	{
		UK2Node* Endpoint = CastChecked<UK2Node>(EntryPoint.WeakEndpoint.Get(), ECastCheckedType::NullAllowed);
		if (!Endpoint)
		{
			if (UK2Node_FunctionEntry* LegacyFunctionEntry = Cast<UK2Node_FunctionEntry>(EntryPoint.FunctionEntry_DEPRECATED.Get()))
			{
				EntryPoint.WeakEndpoint = Endpoint = LegacyFunctionEntry;
			}

			// If we don't have an endpoint but do have legacy graph or node guids, we do the manual upgrade
			if (!Endpoint && EntryPoint.GraphGuid_DEPRECATED.IsValid())
			{
				if (EntryPoint.NodeGuid_DEPRECATED.IsValid())
				{
					if (TObjectPtr<UEdGraph> const* GraphPtr = Algo::FindBy(SequenceDirectorBP->UbergraphPages, EntryPoint.GraphGuid_DEPRECATED, &UEdGraph::GraphGuid))
					{
						TObjectPtr<UEdGraphNode> const* NodePtr  = Algo::FindBy((*GraphPtr)->Nodes, EntryPoint.NodeGuid_DEPRECATED, &UEdGraphNode::NodeGuid);
						if (NodePtr)
						{
							UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(*NodePtr);
							if (ensureMsgf(CustomEvent, TEXT("Encountered an event entry point node that is bound to something other than a custom event")))
							{
								CustomEvent->OnUserDefinedPinRenamed().AddUObject(Section, &UMovieSceneEventSectionBase::OnUserDefinedPinRenamed);
								EntryPoint.WeakEndpoint = Endpoint = CustomEvent;
							}
						}
					}
				}
				// If the node guid is invalid, this must be a function graph on the BP
				else if (TObjectPtr<UEdGraph> const* GraphPtr = Algo::FindBy(SequenceDirectorBP->FunctionGraphs, EntryPoint.GraphGuid_DEPRECATED, &UEdGraph::GraphGuid))
				{
					TObjectPtr<UEdGraphNode> const* NodePtr = Algo::FindByPredicate((*GraphPtr)->Nodes, [](UEdGraphNode* InNode){ return InNode && InNode->IsA<UK2Node_FunctionEntry>(); });
					if (NodePtr)
					{
						UK2Node_FunctionEntry* FunctionEntry = CastChecked<UK2Node_FunctionEntry>(*NodePtr);
						FunctionEntry->OnUserDefinedPinRenamed().AddUObject(Section, &UMovieSceneEventSectionBase::OnUserDefinedPinRenamed);
						EntryPoint.WeakEndpoint = Endpoint = FunctionEntry;
					}
				}

				if (Endpoint)
				{
					// Discover its bound object pin name from the node
					for (UEdGraphPin* Pin : Endpoint->Pins)
					{
						if (Pin->Direction == EGPD_Output && (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) )
						{
							EntryPoint.BoundObjectPinName = Pin->PinName;
							break;
						}
					}
				}
			}
		}

		// Set the compiled function name so that any immediate PostCompile steps find the correct function name
		if (Endpoint)
		{
			EntryPoint.CompiledFunctionName = Endpoint->GetGraph()->GetFName();
		}
	}

	// If the BP has already been compiled (eg regenerate on load) we must perform PostCompile fixup immediately since
	// We will not have had a chance to generate function entries. In this case we just bind directly to the already compiled functions.
	if (SequenceDirectorBP->bHasBeenRegenerated)
	{
		Section->OnPostCompile(SequenceDirectorBP);
	}

	return true;
}

bool IsMovieSceneClassAllowed(const UClass* InClass)
{
	if (!InClass)
	{
		return false;
	}

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
	TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
	FClassViewerInitializationOptions ClassViewerOptions = {};

	if (GlobalClassFilter.IsValid())
	{
		return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, InClass, ClassFilterFuncs);
	}

	return true;
}

bool FMovieSceneToolsModule::IsTrackClassAllowed(UClass* InClass)
{
	return IsMovieSceneClassAllowed(InClass);
}

bool FMovieSceneToolsModule::IsCustomBindingClassAllowed(UClass* InClass)
{
	// For now this has the same implementation as IsTrackClassAllowed, but has been kept a separate function and event in the case we want
	// different behavior here.

	return IsMovieSceneClassAllowed(InClass);
}


bool FMovieSceneToolsModule::IsConditionClassAllowed(const UClass* InClass)
{
	// For now this has the same implementation as IsTrackClassAllowed, but has been kept a separate function and event in the case we want
	// different behavior here.

	return IsMovieSceneClassAllowed(InClass);
}

void FMovieSceneToolsModule::FixupPayloadParameterNameForSection(UMovieSceneEventSectionBase* Section, UK2Node* InNode, FName OldPinName, FName NewPinName)
{
	check(Section && InNode);

	for (FMovieSceneEvent& EntryPoint : Section->GetAllEntryPoints())
	{
		if (EntryPoint.WeakEndpoint.Get() != InNode)
		{
			continue;
		}

		if (EntryPoint.BoundObjectPinName == OldPinName)
		{
			EntryPoint.BoundObjectPinName = NewPinName;
		}

		if (FMovieSceneEventPayloadVariable* Variable = EntryPoint.PayloadVariables.Find(OldPinName))
		{
			EntryPoint.PayloadVariables.Add(NewPinName, MoveTemp(*Variable));
			EntryPoint.PayloadVariables.Remove(OldPinName);
		}
	}
}

void FMovieSceneToolsModule::FixupPayloadParameterNameForDynamicBinding(UMovieScene* MovieScene, UK2Node* InNode, FName OldPinName, FName NewPinName)
{
	using namespace UE::MovieScene;

	check(MovieScene);

	auto FixupPayloadParameterName = [InNode, OldPinName, NewPinName](FMovieSceneDynamicBinding& DynamicBinding)
	{
		if (DynamicBinding.WeakEndpoint.Get() == InNode)
		{
			if (FMovieSceneDynamicBindingPayloadVariable* Variable = DynamicBinding.PayloadVariables.Find(OldPinName))
			{
				DynamicBinding.PayloadVariables.Add(NewPinName, MoveTemp(*Variable));
				DynamicBinding.PayloadVariables.Remove(OldPinName);
			}
		}
	};

	UMovieSceneSequence* ThisSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
	TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GEditor->GetEditorWorldContext().World(), ThisSequence);

	if (FMovieSceneBindingReferences* BindingReferences = MovieScene->GetTypedOuter<UMovieSceneSequence>()->GetBindingReferences())
	{
		for (FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
		{
			if (BindingReference.CustomBinding)
			{
				if (UMovieSceneReplaceableDirectorBlueprintBinding* ReplaceableDirectorBlueprintBinding = Cast<UMovieSceneReplaceableDirectorBlueprintBinding>(BindingReference.CustomBinding))
				{
					FixupPayloadParameterName(ReplaceableDirectorBlueprintBinding->DynamicBinding);
				}

				if (UMovieSceneSpawnableDirectorBlueprintBinding* SpawnableDirectorBlueprintBinding = Cast<UMovieSceneSpawnableDirectorBlueprintBinding>(BindingReference.CustomBinding->AsSpawnable(TransientPlaybackState)))
				{
					FixupPayloadParameterName(SpawnableDirectorBlueprintBinding->DynamicBinding);
				}
			}
		}
	}
}

void FMovieSceneToolsModule::FixupPayloadParameterNameForDirectorBlueprintCondition(UMovieScene* MovieScene, UK2Node* InNode, FName OldPinName, FName NewPinName)
{
	using namespace UE::MovieScene;

	check(MovieScene);

	auto FixupPayloadParameterName = [InNode, OldPinName, NewPinName](FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintConditionData)
	{
		if (DirectorBlueprintConditionData.WeakEndpoint.Get() == InNode)
		{
			if (FMovieSceneDirectorBlueprintConditionPayloadVariable* Variable = DirectorBlueprintConditionData.PayloadVariables.Find(OldPinName))
			{
				DirectorBlueprintConditionData.PayloadVariables.Add(NewPinName, MoveTemp(*Variable));
				DirectorBlueprintConditionData.PayloadVariables.Remove(OldPinName);
			}
		}
	};

	FMovieSceneDirectorBlueprintConditionUtils::IterateDirectorBlueprintConditions(MovieScene, FixupPayloadParameterName);
}

void FMovieSceneToolsModule::RegisterClipboardConversions()
{
	using namespace MovieSceneClipboard;

	DefineImplicitConversion<int32, uint8>();
	DefineImplicitConversion<int32, bool>();

	DefineImplicitConversion<uint8, int32>();
	DefineImplicitConversion<uint8, bool>();

	DefineExplicitConversion<int32, FMovieSceneFloatValue>([](const int32& In) -> FMovieSceneFloatValue { return FMovieSceneFloatValue(In);	});
	DefineExplicitConversion<uint8, FMovieSceneFloatValue>([](const uint8& In) -> FMovieSceneFloatValue { return FMovieSceneFloatValue(In);	});
	DefineExplicitConversion<FMovieSceneFloatValue, int32>([](const FMovieSceneFloatValue& In) -> int32 { return In.Value; 					});
	DefineExplicitConversion<FMovieSceneFloatValue, uint8>([](const FMovieSceneFloatValue& In) -> uint8 { return In.Value; 					});
	DefineExplicitConversion<FMovieSceneFloatValue, bool>([](const FMovieSceneFloatValue& In) -> bool	{ return !!In.Value; 				});

	DefineExplicitConversion<int32, FMovieSceneDoubleValue>([](const int32& In) -> FMovieSceneDoubleValue { return FMovieSceneDoubleValue(In);	});
	DefineExplicitConversion<uint8, FMovieSceneDoubleValue>([](const uint8& In) -> FMovieSceneDoubleValue { return FMovieSceneDoubleValue(In);	});
	DefineExplicitConversion<FMovieSceneDoubleValue, int32>([](const FMovieSceneDoubleValue& In) -> int32 { return In.Value; 					});
	DefineExplicitConversion<FMovieSceneDoubleValue, uint8>([](const FMovieSceneDoubleValue& In) -> uint8 { return In.Value; 					});
	DefineExplicitConversion<FMovieSceneDoubleValue, bool>([](const FMovieSceneDoubleValue& In) -> bool	  { return !!In.Value;					});

	FSequencerClipboardReconciler::AddTrackAlias("Location.X", "R");
	FSequencerClipboardReconciler::AddTrackAlias("Location.Y", "G");
	FSequencerClipboardReconciler::AddTrackAlias("Location.Z", "B");

	FSequencerClipboardReconciler::AddTrackAlias("Rotation.X", "R");
	FSequencerClipboardReconciler::AddTrackAlias("Rotation.Y", "G");
	FSequencerClipboardReconciler::AddTrackAlias("Rotation.Z", "B");

	FSequencerClipboardReconciler::AddTrackAlias("Scale.X", "R");
	FSequencerClipboardReconciler::AddTrackAlias("Scale.Y", "G");
	FSequencerClipboardReconciler::AddTrackAlias("Scale.Z", "B");

	FSequencerClipboardReconciler::AddTrackAlias("X", "R");
	FSequencerClipboardReconciler::AddTrackAlias("Y", "G");
	FSequencerClipboardReconciler::AddTrackAlias("Z", "B");
	FSequencerClipboardReconciler::AddTrackAlias("W", "A");
}

void FMovieSceneToolsModule::RegisterAnimationBakeHelper(IMovieSceneToolsAnimationBakeHelper* InBakeHelper)
{
	checkf(!BakeHelpers.Contains(InBakeHelper), TEXT("Bake Helper is already registered"));
	BakeHelpers.Add(InBakeHelper);
}

void FMovieSceneToolsModule::UnregisterAnimationBakeHelper(IMovieSceneToolsAnimationBakeHelper* InBakeHelper)
{
	checkf(BakeHelpers.Contains(InBakeHelper), TEXT("Bake Helper is not registered"));
	BakeHelpers.Remove(InBakeHelper);
}

void FMovieSceneToolsModule::RegisterTakeData(IMovieSceneToolsTakeData* InTakeData)
{
	checkf(!TakeDatas.Contains(InTakeData), TEXT("Take Data is already registered"));
	TakeDatas.Add(InTakeData);
}

void FMovieSceneToolsModule::UnregisterTakeData(IMovieSceneToolsTakeData* InTakeData)
{
	checkf(TakeDatas.Contains(InTakeData), TEXT("Take Data is not registered"));
	TakeDatas.Remove(InTakeData);
}

void FMovieSceneToolsModule::RegisterTrackImporter(IMovieSceneToolsTrackImporter* InTrackImporter)
{
	checkf(!TrackImporters.Contains(InTrackImporter), TEXT("Track Importer is already registered"));
	TrackImporters.Add(InTrackImporter);
}

void FMovieSceneToolsModule::UnregisterTrackImporter(IMovieSceneToolsTrackImporter* InTrackImporter)
{
	checkf(TrackImporters.Contains(InTrackImporter), TEXT("Take Importer is not registered"));
	TrackImporters.Remove(InTrackImporter);
}

bool FMovieSceneToolsModule::GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber)
{
	for (IMovieSceneToolsTakeData* TakeData : TakeDatas)
	{
		if (TakeData->GatherTakes(Section, AssetData, OutCurrentTakeNumber))
		{
			return true;
		}
	}

	return false;
}


bool FMovieSceneToolsModule::GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber)
{
	for (IMovieSceneToolsTakeData* TakeData : TakeDatas)
	{
		if (TakeData->GetTakeNumber(Section, AssetData, OutTakeNumber))
		{
			return true;
		}
	}

	return false;
}

bool FMovieSceneToolsModule::SetTakeNumber(const UMovieSceneSection* Section, uint32 InTakeNumber)
{
	for (IMovieSceneToolsTakeData* TakeData : TakeDatas)
	{
		if (TakeData->SetTakeNumber(Section, InTakeNumber))
		{
			return true;
		}
	}

	return false;
}

bool FMovieSceneToolsModule::ImportAnimatedProperty(const FString& InPropertyName, const FRichCurve& InCurve, FGuid InBinding, UMovieScene* InMovieScene)
{
	for (IMovieSceneToolsTrackImporter* TrackImporter : TrackImporters)
	{
		if (TrackImporter->ImportAnimatedProperty(InPropertyName, InCurve, InBinding, InMovieScene))
		{
			return true;
		}
	}

	return false;
}

bool FMovieSceneToolsModule::ImportStringProperty(const FString& InPropertyName, const FString& InStringValue, FGuid InBinding, UMovieScene* InMovieScene)
{
	for (IMovieSceneToolsTrackImporter* TrackImporter : TrackImporters)
	{
		if (TrackImporter->ImportStringProperty(InPropertyName, InStringValue, InBinding, InMovieScene))
		{
			return true;
		}
	}

	return false;
}

void FMovieSceneToolsModule::RegisterKeyStructInstancedPropertyTypeCustomizer(IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer* InCustomizer)
{
	checkf(!KeyStructInstancedPropertyTypeCustomizers.Contains(InCustomizer), TEXT("Key Struct Instanced Property Type Customizer is already registered"));
	KeyStructInstancedPropertyTypeCustomizers.Add(InCustomizer);
}

void FMovieSceneToolsModule::UnregisterKeyStructInstancedPropertyTypeCustomizer(IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer* InCustomizer)
{
	checkf(KeyStructInstancedPropertyTypeCustomizers.Contains(InCustomizer), TEXT("Key Struct Instanced Property Type Customizer is not registered"));
	KeyStructInstancedPropertyTypeCustomizers.Remove(InCustomizer);
}

void FMovieSceneToolsModule::CustomizeKeyStructInstancedPropertyTypes(TSharedRef<IStructureDetailsView> StructureDetailsView, TWeakObjectPtr<UMovieSceneSection> Section)
{
	for (IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer* Customizer : KeyStructInstancedPropertyTypeCustomizers)
	{
		if (Customizer)
		{
			Customizer->RegisterKeyStructInstancedPropertyTypeCustomization(StructureDetailsView, Section);
		}
	}
}

FMovieSceneToolsAnimationBakingWrapper::FMovieSceneToolsAnimationBakingWrapper(UMovieScene* InMovieScene)
	: MovieScene(InMovieScene)
	, BakeHelpers(FMovieSceneToolsModule::Get().GetAnimationBakeHelpers())
{
	ensure(InMovieScene);
}

void FMovieSceneToolsAnimationBakingWrapper::Start() const
{
	if (MovieScene)
	{
		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->StartBaking(MovieScene);
			}
		}
	}			
}

void FMovieSceneToolsAnimationBakingWrapper::PreEvaluate(const FFrameNumber& Frame) const
{
	if (MovieScene)
	{
		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->PreEvaluation(MovieScene, Frame);
			}
		}
	}
}

void FMovieSceneToolsAnimationBakingWrapper::PostEvaluate(const FFrameNumber& Frame) const
{
	if (MovieScene)
	{
		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->PostEvaluation(MovieScene, Frame);
			}
		}
	}
}

void FMovieSceneToolsAnimationBakingWrapper::Stop() const
{
	if (MovieScene)
	{
		for (IMovieSceneToolsAnimationBakeHelper* BakeHelper : BakeHelpers)
		{
			if (BakeHelper)
			{
				BakeHelper->StopBaking(MovieScene);
			}
		}
	}
}

IMPLEMENT_MODULE( FMovieSceneToolsModule, MovieSceneTools );

#undef LOCTEXT_NAMESPACE

