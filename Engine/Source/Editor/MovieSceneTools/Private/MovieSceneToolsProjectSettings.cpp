// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneToolsProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneToolsProjectSettings)


UMovieSceneToolsProjectSettings::UMovieSceneToolsProjectSettings()
	: DefaultStartTime(0.f)
	, DefaultDuration(5.f)
	, SubsequenceDirectory(TEXT("subsequences"))
	, ShotDirectory(TEXT("shots"))
	, SubsequencePrefix(TEXT("subsequence"))
	, ShotPrefix(TEXT("shot"))
	, FirstShotNumber(10)
	, ShotIncrement(10)
	, ShotNumDigits(4)
	, TakeNumDigits(2)
	, FirstTakeNumber(1)
	, TakeSeparator(TEXT("_"))
{ }
