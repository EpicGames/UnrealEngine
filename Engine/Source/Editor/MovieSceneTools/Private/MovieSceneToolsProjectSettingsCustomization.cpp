// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneToolsProjectSettingsCustomization.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneFwd.h"
#include "MovieSceneToolsProjectSettings.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

TSharedRef<IDetailCustomization> FMovieSceneToolsProjectSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneToolsProjectSettingsCustomization);
}

void FMovieSceneToolsProjectSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> TakeSeparatorProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMovieSceneToolsProjectSettings, TakeSeparator));
	TakeSeparatorProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FMovieSceneToolsProjectSettingsCustomization::OnTakeSeparatorUpdated));
}

void FMovieSceneToolsProjectSettingsCustomization::OnTakeSeparatorUpdated()
{
	UMovieSceneToolsProjectSettings* ProjectSettings = GetMutableDefault<UMovieSceneToolsProjectSettings>();

	FString TakeSeparator = ProjectSettings->TakeSeparator;

	// Make sure the take separator is a valid character
	FText OutErrorMessage;
	if (!FName(*TakeSeparator).IsValidXName(INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage))
	{
		UE_LOGF(LogMovieScene, Warning, "Invalid separator: %ls", *OutErrorMessage.ToString());
		ProjectSettings->TakeSeparator = TEXT("_");
	}
}
