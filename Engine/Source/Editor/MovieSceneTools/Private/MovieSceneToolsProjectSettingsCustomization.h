// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

class FMovieSceneToolsProjectSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void OnTakeSeparatorUpdated();
};
