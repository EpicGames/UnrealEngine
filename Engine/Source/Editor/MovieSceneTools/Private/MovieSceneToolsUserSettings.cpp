// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneToolsUserSettings.h"
#include "SceneManagement.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneToolsUserSettings)


UMovieSceneUserThumbnailSettings::UMovieSceneUserThumbnailSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	ThumbnailSize = FIntPoint(128, 72);
	bDrawThumbnails = true;
	Quality = EThumbnailQuality::Normal;
}

void UMovieSceneUserThumbnailSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMovieSceneUserThumbnailSettings, Quality))
	{
		BroadcastRedrawThumbnails();
	}
	
	ThumbnailSize.X = FMath::Clamp(ThumbnailSize.X, 1, 1024);
	ThumbnailSize.Y = FMath::Clamp(ThumbnailSize.Y, 1, 1024);

	SaveConfig();
}

void UMovieSceneUserThumbnailSettings::AddReferencedObjects(UObject* InThis, FReferenceCollector& InCollector)
{
	UMovieSceneUserThumbnailSettings* This = CastChecked<UMovieSceneUserThumbnailSettings>(InThis);
	if (This->ViewState.GetReference())
	{
		This->ViewState.GetReference()->AddReferencedObjects(InCollector);
	}
	Super::AddReferencedObjects(InThis, InCollector);
}

UMovieSceneUserImportFBXSettings::UMovieSceneUserImportFBXSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	bMatchByNameOnly = true;
	bForceFrontXAxis = false;
	bCreateCameras = true;
	bReplaceTransformTrack = true;
	bCorrectForTransformOrigin = false;
	bReduceKeys = true;
	ReduceKeysTolerance = 0.001f;
	bConvertSceneUnit = true;
	ImportUniformScale = 1.0f;
}


void SetControlRigIOChannelMappingPresets(TArray<FControlToTransformMappings>& ControlChannelMappings, const bool bMetaHuman)
{
	// Since we can't change the API unfortunately need to do this here.
	ControlChannelMappings.SetNum(0); //clear and reset
	FControlToTransformMappings Bool;
	Bool.bNegate = false;
	Bool.ControlChannel = FControlRigChannelEnum::Bool;
	Bool.FBXChannel = FTransformChannelEnum::TranslateX;
	ControlChannelMappings.Add(Bool);

	FControlToTransformMappings Float;
	Float.bNegate = false;
	Float.ControlChannel = FControlRigChannelEnum::Float;
	if (bMetaHuman)
	{
		Float.FBXChannel = FTransformChannelEnum::TranslateY;  // use Y for metahuman
	}
	else
	{
		Float.FBXChannel = FTransformChannelEnum::TranslateX;
	}
	ControlChannelMappings.Add(Float);

	FControlToTransformMappings Vector2DX;
	Vector2DX.bNegate = false;
	Vector2DX.ControlChannel = FControlRigChannelEnum::Vector2DX;
	Vector2DX.FBXChannel = FTransformChannelEnum::TranslateX;
	ControlChannelMappings.Add(Vector2DX);

	FControlToTransformMappings Vector2DY;
	Vector2DY.bNegate = false;
	Vector2DY.ControlChannel = FControlRigChannelEnum::Vector2DY;
	Vector2DY.FBXChannel = FTransformChannelEnum::TranslateY;
	ControlChannelMappings.Add(Vector2DY);

	FControlToTransformMappings PositionX;
	PositionX.bNegate = false;
	PositionX.ControlChannel = FControlRigChannelEnum::PositionX;
	PositionX.FBXChannel = FTransformChannelEnum::TranslateX;
	ControlChannelMappings.Add(PositionX);

	FControlToTransformMappings PositionY;
	PositionY.bNegate = false;
	PositionY.ControlChannel = FControlRigChannelEnum::PositionY;
	PositionY.FBXChannel = FTransformChannelEnum::TranslateY;
	ControlChannelMappings.Add(PositionY);

	FControlToTransformMappings PositionZ;
	PositionZ.bNegate = false;
	PositionZ.ControlChannel = FControlRigChannelEnum::PositionZ;
	PositionZ.FBXChannel = FTransformChannelEnum::TranslateZ;
	ControlChannelMappings.Add(PositionZ);

	FControlToTransformMappings RotatorX;
	RotatorX.bNegate = false;
	RotatorX.ControlChannel = FControlRigChannelEnum::RotatorX;
	RotatorX.FBXChannel = FTransformChannelEnum::RotateX;
	ControlChannelMappings.Add(RotatorX);

	FControlToTransformMappings RotatorY;
	RotatorY.bNegate = false;
	RotatorY.ControlChannel = FControlRigChannelEnum::RotatorY;
	RotatorY.FBXChannel = FTransformChannelEnum::RotateY;
	ControlChannelMappings.Add(RotatorY);

	FControlToTransformMappings RotatorZ;
	RotatorZ.bNegate = false;
	RotatorZ.ControlChannel = FControlRigChannelEnum::RotatorZ;
	RotatorZ.FBXChannel = FTransformChannelEnum::RotateZ;
	ControlChannelMappings.Add(RotatorZ);

	FControlToTransformMappings ScaleX;
	ScaleX.bNegate = false;
	ScaleX.ControlChannel = FControlRigChannelEnum::ScaleX;
	ScaleX.FBXChannel = FTransformChannelEnum::ScaleX;
	ControlChannelMappings.Add(ScaleX);

	FControlToTransformMappings ScaleY;
	ScaleY.bNegate = false;
	ScaleY.ControlChannel = FControlRigChannelEnum::ScaleY;
	ScaleY.FBXChannel = FTransformChannelEnum::ScaleY;
	ControlChannelMappings.Add(ScaleY);

	FControlToTransformMappings ScaleZ;
	ScaleZ.bNegate = false;
	ScaleZ.ControlChannel = FControlRigChannelEnum::ScaleZ;
	ScaleZ.FBXChannel = FTransformChannelEnum::ScaleZ;
	ControlChannelMappings.Add(ScaleZ);
}


UMovieSceneUserImportFBXControlRigSettings::UMovieSceneUserImportFBXControlRigSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	bForceFrontXAxis = false;
	bConvertSceneUnit = true;
	ImportUniformScale = 1.0f;
	bSpecifyTimeRange = false;
	StartTimeRange = 0;
	EndTimeRange = 0;
	TimeToInsertOrReplaceAnimation = 0;
	bInsertAnimation = true;
	bImportOntoSelectedControls = false;

	FControlFindReplaceString FirstFindReplace;
	FirstFindReplace.Find = FString("");
	FirstFindReplace.Replace = FString("");
	FindAndReplaceStrings.Add(FirstFindReplace);

	SetControlRigIOChannelMappingPresets(ControlChannelMappings, false);
}

void UMovieSceneUserImportFBXControlRigSettings::LoadControlMappingsFromPreset(bool bMetaHumanPreset)
{
	SetControlRigIOChannelMappingPresets(ControlChannelMappings, bMetaHumanPreset);
}

UMovieSceneUserExportFBXControlRigSettings::UMovieSceneUserExportFBXControlRigSettings(
	const FObjectInitializer& Initializer)
{
}

void UMovieSceneUserExportFBXControlRigSettings::LoadControlMappingsFromPreset(bool bMetaHumanPreset)
{
	SetControlRigIOChannelMappingPresets(ControlChannelMappings, bMetaHumanPreset);
}


/*
enum class ERigControlType : uint8
{
	Bool,
	Float,
	Vector2D,
	Position,
	Scale,
	Rotator,
	Transform,
	TransformNoScale,
	EulerTransform,
};

*/

