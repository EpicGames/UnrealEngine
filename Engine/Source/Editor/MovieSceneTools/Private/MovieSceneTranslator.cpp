// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTranslator.h"
#include "MovieScene.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "MovieSceneTimeHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundCue.h"
#include "EditorFramework/AssetImportData.h"

#define LOCTEXT_NAMESPACE "MovieSceneTranslator"

namespace
{
	bool SupportMultipleAudioTracks()
	{
		return true;
	}

	bool AudioSectionIsSoundWave(const UMovieSceneAudioSection* InAudioSection)
	{
		USoundBase* SoundBase = InAudioSection->GetSound();
		if (SoundBase == nullptr)
		{
			return false;
		}

		return (SoundBase->IsA<USoundWave>());
	}
}

FMovieSceneExportData::FMovieSceneExportData(const UMovieScene* InMovieScene, FFrameRate InFrameRate, uint32 InResX, uint32 InResY, int32 InHandleFrames, FString InSaveFilename, TSharedPtr<FMovieSceneTranslatorContext> InContext, FString InMovieExtension)
{
	if (InMovieScene == nullptr)
	{
		bExportDataIsValid = false;
		return;
	}

	ExportContext = InContext;
	FrameRate = InFrameRate;
	ResX = InResX;
	ResY = InResY;
	HandleFrames = InHandleFrames;
	SaveFilename = InSaveFilename;
	MovieExtension = InMovieExtension;

	// preferred sample rate in UE
	DefaultAudioSampleRate = 44100;
	// all audio in UE is has depth 16
	DefaultAudioDepth = 16;

	bExportDataIsValid = ConstructData(InMovieScene);
}

FMovieSceneExportData::FMovieSceneExportData()
{
	bExportDataIsValid = false;
}

FMovieSceneExportData::~FMovieSceneExportData()
{
}

bool FMovieSceneExportData::IsExportDataValid() const
{
	return bExportDataIsValid;
}

bool FMovieSceneExportData::ConstructData(const UMovieScene* InMovieScene)
{
	if (InMovieScene == nullptr) 
	{ 
		return false; 
	}

	SaveFilenamePath = FPaths::GetPath(SaveFilename);
	if (FPaths::IsRelative(SaveFilenamePath))
	{
		SaveFilenamePath = FPaths::ConvertRelativePathToFull(SaveFilenamePath);
	}

	return ConstructMovieSceneData(InMovieScene);
}

bool FMovieSceneExportData::ConstructMovieSceneData(const UMovieScene* InMovieScene)
{
	if (InMovieScene == nullptr)
	{
		return false;
	}

	MovieSceneData = MakeShared<FMovieSceneExportMovieSceneData>();

	FFrameRate TickResolution = InMovieScene->GetTickResolution();

	TRange<FFrameNumber> PlaybackRange = InMovieScene->GetPlaybackRange();

	if (PlaybackRange.HasLowerBound())
	{
		MovieSceneData->PlaybackRangeStartFrame = ConvertFrameTime(PlaybackRange.GetLowerBoundValue(), TickResolution, FrameRate).CeilToFrame();
	}
	else
	{
		UE_LOGF(LogMovieScene, Error, "Invalid condition: Movie scene playback range has infinite lower bound.");
		return false;
	}

	if (PlaybackRange.HasUpperBound())
	{
		MovieSceneData->PlaybackRangeEndFrame = ConvertFrameTime(PlaybackRange.GetUpperBoundValue(), TickResolution, FrameRate).CeilToFrame();
	}
	else
	{
		UE_LOGF(LogMovieScene, Error, "Invalid condition: Movie scene playback range has infinite upper bound.");
		return false;
	}

	MovieSceneData->Name = InMovieScene->GetOuter()->GetName();
	MovieSceneData->Path = InMovieScene->GetOuter()->GetPathName();
	MovieSceneData->TickResolution = TickResolution;
	MovieSceneData->Duration = ConvertFrameTime(UE::MovieScene::DiscreteSize(PlaybackRange), TickResolution, FrameRate).FrameNumber.Value;

	bool bFoundCinematicTrack = false;

	// sort audio tracks
	TMap<int32, TSharedPtr<FMovieSceneExportAudioData>> AudioTrackMap;

	const TArray<UMovieSceneTrack*> Tracks = InMovieScene->GetTracks();
	for (UMovieSceneTrack* Track : Tracks)
	{
		if (!bFoundCinematicTrack && Track->IsA(UMovieSceneCinematicShotTrack::StaticClass()))
		{
			const UMovieSceneCinematicShotTrack* CinematicTrack = Cast<UMovieSceneCinematicShotTrack>(Track);
			if (CinematicTrack == nullptr || !ConstructCinematicData(InMovieScene, CinematicTrack))
			{ 
				return false; 
			}
			bFoundCinematicTrack = true;
		}
		else if (Track->IsA(UMovieSceneAudioTrack::StaticClass()))
		{
			const UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(Track);
			if (AudioTrack == nullptr || !ConstructAudioData(InMovieScene, AudioTrack, AudioTrackMap))
			{ 
				return false; 
			}
		}
	}

	// sort the audio tracks by their sorting index and add to the AudioTracks array
	if (AudioTrackMap.Num() > 0)
	{
		AudioTrackMap.KeySort([](int32 A, int32 B) {
			return A < B; // sort keys in order
		});

		for (auto& Elem : AudioTrackMap)
		{
			if (Elem.Value.IsValid())
			{
				MovieSceneData->AudioData.Add(Elem.Value);

				if (!SupportMultipleAudioTracks())
				{
					break;
				}
			}
		}
	}

	return true;
}

bool FMovieSceneExportData::ConstructCinematicData(const UMovieScene* InMovieScene, const UMovieSceneCinematicShotTrack* InCinematicTrack)
{
	if (InMovieScene == nullptr || !MovieSceneData.IsValid())
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportCinematicData> CinematicData = MakeShared<FMovieSceneExportCinematicData>();
	CinematicData->MovieSceneTrack = InCinematicTrack;
	MovieSceneData->CinematicData = CinematicData;

	// Make a list of shotnames and how many duplicates
	for (UMovieSceneSection* Section : InCinematicTrack->GetAllSections())
	{
		const UMovieSceneCinematicShotSection* CinematicSection = Cast<UMovieSceneCinematicShotSection>(Section);

		if (CinematicSection != nullptr && CinematicSection->GetSequence() != nullptr)
		{
			++DuplicateShotNameCounts.FindOrAdd(CinematicSection->GetShotDisplayName(), 0);
		}
	}

	// Remove entries for shotnames which do not have duplicates
	TArray<FString> ShotNames;
	DuplicateShotNameCounts.GetKeys(ShotNames);
	for (FString ShotName : ShotNames)
	{
		int32& Count = DuplicateShotNameCounts[ShotName];
		if (Count > 1)
		{
			// Reset duplicate count to zero, so we can increment as we export
			Count = 0;
		}
		else
		{
			// No dupes, so remove it
			DuplicateShotNameCounts.Remove(ShotName);
		}
	}

	// Construct sections & create track row index array
	TArray<int32> CinematicTrackRowIndices;

	for (UMovieSceneSection* Section : InCinematicTrack->GetAllSections())
	{
		const UMovieSceneCinematicShotSection* CinematicSection = Cast<UMovieSceneCinematicShotSection>(Section);

		if (CinematicSection != nullptr && CinematicSection->GetSequence() != nullptr)
		{
			if (!ConstructCinematicSectionData(InMovieScene, CinematicData, CinematicSection))
			{
				return false;
			}

			int32 RowIndex = CinematicSection->GetRowIndex();
			if (RowIndex >= 0)
			{
				CinematicTrackRowIndices.AddUnique(RowIndex);
			}
		}
	}

	// Construct tracks and point to sections
	CinematicTrackRowIndices.Sort();

	for (int32 CinematicTrackRowIndex : CinematicTrackRowIndices)
	{	
		if (!ConstructCinematicTrackData(InMovieScene, CinematicData, CinematicTrackRowIndex))
		{
			return false;
		}		
	}

	return true;
}

bool FMovieSceneExportData::ConstructCinematicTrackData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, int32 InRowIndex)
{
	if (InMovieScene == nullptr || !InCinematicData.IsValid() || !MovieSceneData.IsValid() || !MovieSceneData->CinematicData.IsValid())
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportCinematicTrackData> TrackData = MakeShared<FMovieSceneExportCinematicTrackData>();
	TrackData->RowIndex = InRowIndex;
	MovieSceneData->CinematicData->CinematicTracks.Add(TrackData);

	for (TSharedPtr<FMovieSceneExportCinematicSectionData> Section : InCinematicData->CinematicSections)
	{
		if (Section.IsValid() && Section->RowIndex == InRowIndex)
		{
			TrackData->CinematicSections.Add(Section);
		}
	}

	return true;
}

bool FMovieSceneExportData::ConstructAudioData(const UMovieScene* InMovieScene, const UMovieSceneAudioTrack* InAudioTrack, TMap<int32, TSharedPtr<FMovieSceneExportAudioData>>& InAudioTrackMap)
{
	if (InMovieScene == nullptr || !MovieSceneData.IsValid())
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportAudioData> TrackData = MakeShared<FMovieSceneExportAudioData>();
	TrackData->MovieSceneTrack = InAudioTrack;
	InAudioTrackMap.Add(InAudioTrack->GetSortingOrder(), TrackData);

	// Construct sections & create track row index array
	TArray<int32> AudioTrackRowIndices;

	TArray<FString> SectionPathNames;

	for (UMovieSceneSection* Section : InAudioTrack->GetAudioSections())
	{
		const UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Section);
		if (AudioSection == nullptr)
		{
			continue;
		}

		// skip duplicate sections
		if (SectionPathNames.Num() > 0 && SectionPathNames.Contains(AudioSection->GetPathName()))
		{
			continue;
		}

		if (AudioSectionIsSoundWave(AudioSection))
		{
			if (!ConstructAudioSectionData(InMovieScene, TrackData, AudioSection))
			{
				return false;
			}

			int32 RowIndex = AudioSection->GetRowIndex();
			if (RowIndex >= 0)
			{
				AudioTrackRowIndices.AddUnique(RowIndex);
			}
		}
		SectionPathNames.Add(AudioSection->GetPathName());
	}

	// Construct tracks and point to sections
	AudioTrackRowIndices.Sort();

	for (int32 AudioTrackRowIndex : AudioTrackRowIndices)
	{
		if (!ConstructAudioData(InMovieScene, TrackData, AudioTrackRowIndex))
		{
			return false;
		}
	}

	return true;
}

bool FMovieSceneExportData::ConstructAudioData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportAudioData> InAudioData, int32 InRowIndex)
{
	if (InMovieScene == nullptr || !InAudioData.IsValid())
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportAudioTrackData> TrackData = MakeShared<FMovieSceneExportAudioTrackData>();
	TrackData->SampleRate = DefaultAudioSampleRate;
	TrackData->RowIndex = InRowIndex;
	InAudioData->AudioTracks.Add(TrackData);

	for (TSharedPtr<FMovieSceneExportAudioSectionData> Section : InAudioData->AudioSections)
	{
		if (Section.IsValid() && Section->RowIndex == InRowIndex)
		{
			TrackData->AudioSections.Add(Section);
		}
	}

	return true;
}

bool FMovieSceneExportData::ConstructCinematicSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, const UMovieSceneCinematicShotSection* InCinematicSection)
{
	if (InMovieScene == nullptr || !InCinematicData.IsValid() || InCinematicSection == nullptr)
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportCinematicSectionData> SectionData = MakeShared<FMovieSceneExportCinematicSectionData>();
	InCinematicData->CinematicSections.Add(SectionData);

	SectionData->DisplayName = InCinematicSection->GetShotDisplayName();
	if (DuplicateShotNameCounts.Contains(SectionData->DisplayName))
	{
		// If this shotname is a duplicate, append a number to make it unique
		int32 Count = ++DuplicateShotNameCounts[SectionData->DisplayName];
		SectionData->DisplayName.Append(FString::Format(TEXT("({0})"), { Count }));
	}
	SectionData->SourceFilename = SectionData->DisplayName + MovieExtension;
	SectionData->SourceFilePath = TEXT("");

	ConstructSectionData(InMovieScene, SectionData, InCinematicSection, EMovieSceneTranslatorSectionType::Cinematic, SectionData->DisplayName);

	return true;
}

bool FMovieSceneExportData::ConstructAudioSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportAudioData> InAudioData, const UMovieSceneAudioSection* InAudioSection)
{
	if (InMovieScene == nullptr || !InAudioData.IsValid() || InAudioSection == nullptr)
	{
		return false;
	}

	USoundBase* SoundBase = InAudioSection->GetSound();
	if (SoundBase == nullptr || !SoundBase->IsA<USoundWave>())
	{
		return false;
	}

	USoundWave* SoundWave = nullptr;
	SoundWave = Cast<USoundWave>(SoundBase);
	if (SoundWave == nullptr || SoundWave->AssetImportData == nullptr)
	{
		return false;
	}

	TSharedPtr<FMovieSceneExportAudioSectionData> SectionData = MakeShared<FMovieSceneExportAudioSectionData>();
	InAudioData->AudioSections.Add(SectionData);

	TArray<FString> Filenames = SoundWave->AssetImportData->ExtractFilenames();
	if (Filenames.Num() < 1)
	{
		return false;
	}

	int32 SampleRate = SoundWave->GetSampleRateForCurrentPlatform();
	if (SampleRate != 48000 && SampleRate != 44100 && SampleRate != 32000)
	{
		// @todo - warning about invalid sample rate
		SampleRate = 44100;
	}
	
	SectionData->DisplayName = SoundWave->GetName();
	SectionData->SourceFilename = FPaths::GetCleanFilename(Filenames[0]);
	SectionData->SourceFilePath = FPaths::GetPath(Filenames[0]);
	SectionData->Depth = GetDefaultAudioDepth();
	SectionData->SampleRate = SampleRate;
	SectionData->NumChannels = SoundWave->NumChannels;

	ConstructSectionData(InMovieScene, SectionData, InAudioSection, EMovieSceneTranslatorSectionType::Audio, TEXT(""));

	return true;
}

bool FMovieSceneExportData::ConstructSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportSectionData> InSectionData, const UMovieSceneSection* InSection, EMovieSceneTranslatorSectionType InSectionType, const FString& InSectionDisplayName)
{
	if (InMovieScene == nullptr || !MovieSceneData.IsValid() || InSection == nullptr || !InSectionData.IsValid())
	{
		return false;
	}

	InSectionData->MovieSceneSection = InSection;
	InSectionData->RowIndex = InSection->GetRowIndex();

	if (InSection->HasStartFrame())
	{
		FFrameTime InclusiveStartFrame = InSection->GetInclusiveStartFrame();
		FFrameTime ConvertedStartFrame = ConvertFrameTime(InclusiveStartFrame, MovieSceneData->TickResolution, FrameRate);
		InSectionData->StartFrame = ConvertedStartFrame.CeilToFrame();

		if (ExportContext.IsValid() && InSectionType == EMovieSceneTranslatorSectionType::Cinematic && ConvertedStartFrame.GetSubFrame() > 0.0f)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning,
				FText::Format(LOCTEXT("SectionStartNotDivisByDisplayRate", "Section '{0}' starts on tick {1} which is not evenly divisible by the display rate {2}. Enable snapping and adjust the start or edit the section properties to ensure it lands evenly on a whole frame."),
					FText::FromString(InSectionDisplayName),
					FText::AsNumber(InclusiveStartFrame.CeilToFrame().Value),
					FText::FromString(FString::SanitizeFloat(FrameRate.AsDecimal()))));
		}
	}
	else
	{
		InSectionData->StartFrame = FFrameNumber(0);
		if (ExportContext.IsValid() && InSectionType == EMovieSceneTranslatorSectionType::Cinematic)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning, FText::Format(LOCTEXT("SectionHasNoStartFrame", "Section '{0}' has no start frame. Start frame will default to 0."), FText::FromString(InSectionDisplayName)));
		}
	}

	if (InSection->HasEndFrame())
	{
		FFrameTime ExclusiveEndFrame = InSection->GetExclusiveEndFrame();
		FFrameTime ConvertedEndFrame = ConvertFrameTime(ExclusiveEndFrame, MovieSceneData->TickResolution, FrameRate);
		InSectionData->EndFrame = ConvertedEndFrame.CeilToFrame();

		if (ExportContext.IsValid() && InSectionType == EMovieSceneTranslatorSectionType::Cinematic && ConvertedEndFrame.GetSubFrame() > 0.0f)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning,
				FText::Format(LOCTEXT("SectionEndNotDivisByDisplayRate", "Section '{0}' ends on tick {1} which is not evenly divisible by the display rate {2}. Enable snapping and adjust the end or edit the section properties to ensure it lands evenly on a whole frame."),
					FText::FromString(InSectionDisplayName),
					FText::FromString(FString::FromInt(ExclusiveEndFrame.CeilToFrame().Value)), 
					FText::FromString(FString::SanitizeFloat(FrameRate.AsDecimal()))));
		}
	}
	else
	{
		InSectionData->EndFrame = MovieSceneData->PlaybackRangeEndFrame;
		if (ExportContext.IsValid() && InSectionType == EMovieSceneTranslatorSectionType::Cinematic)
		{
			ExportContext->AddMessage(EMessageSeverity::Warning, FText::Format(LOCTEXT("SectionHasNoEndFrame", "Section '{0}' has no end frame. End frame will default to playback range end."), FText::FromString(InSectionDisplayName)));
		}
	}

	// @todo handle intersection with playback range?
	TRange<FFrameNumber> PlaybackRange = InMovieScene->GetPlaybackRange();
	TRange<FFrameNumber> EditRange = InSection->GetRange();
	TRange<FFrameNumber> Intersection = TRange<FFrameNumber>::Intersection(PlaybackRange, EditRange);
	InSectionData->bWithinPlaybackRange = EditRange.Overlaps(PlaybackRange);
	InSectionData->bEnabled = true;

	return true;
}

bool FMovieSceneExportData::FindAudioSections(const FString& InSoundPathName, TArray<TSharedPtr<FMovieSceneExportAudioSectionData>>& OutFoundSections) const
{
	if (!MovieSceneData.IsValid())
	{
		return false;
	}

	for (TSharedPtr<FMovieSceneExportAudioData> AudioData : MovieSceneData->AudioData)
	{
		for (TSharedPtr<FMovieSceneExportAudioSectionData> AudioSectionData : AudioData->AudioSections)
		{
			if (!AudioSectionData.IsValid() || AudioSectionData->MovieSceneSection == nullptr)
			{
				continue;
			}
			const UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(AudioSectionData->MovieSceneSection);
			if (AudioSection == nullptr)
			{
				continue;
			}
			USoundBase* Sound = AudioSection->GetSound();
			if (Sound == nullptr || !Sound->IsA<USoundWave>())
			{
				continue;
			}

			FString SoundPathName = Sound->GetPathName();
			if (SoundPathName == InSoundPathName)
			{
				OutFoundSections.Add(AudioSectionData);
			}
		}
	}

	return true;
}

FString FMovieSceneExportData::GetFilename() const
{
	return SaveFilename;
}

FString FMovieSceneExportData::GetFilenamePath() const
{
	return SaveFilenamePath;
}

FString FMovieSceneExportData::GetMovieExtension() const
{
	return MovieExtension;
}

FFrameRate FMovieSceneExportData::GetFrameRate() const
{
	return FrameRate;
}

uint32 FMovieSceneExportData::GetResX() const
{
	return ResX;
}

uint32 FMovieSceneExportData::GetResY() const
{
	return ResY;
}

uint32 FMovieSceneExportData::GetNearestWholeFrameRate() const
{
	if (GetFrameRateIsNTSC())
	{
		double Rate = FrameRate.AsDecimal();
		return static_cast<int32>(FMath::FloorToDouble(Rate + 0.5));
	}
	return static_cast<uint32>(FrameRate.AsDecimal());
}

bool FMovieSceneExportData::GetFrameRateIsNTSC() const
{
	float FractionalPart = FMath::Frac(FrameRate.AsDecimal());
	return (!FMath::IsNearlyZero(FractionalPart));
}

int32 FMovieSceneExportData::GetHandleFrames() const
{
	return HandleFrames;
}

int32 FMovieSceneExportData::GetDefaultAudioSampleRate() const
{
	return DefaultAudioSampleRate;
}

int32 FMovieSceneExportData::GetDefaultAudioDepth() const
{
	return DefaultAudioDepth;
}

FMovieSceneImportData::FMovieSceneImportData(UMovieScene* InMovieScene, TSharedPtr<FMovieSceneTranslatorContext> InContext)
{
	if (InMovieScene == nullptr)
	{
		return;
	}

	ImportContext = InContext;

	MovieSceneData = ConstructMovieSceneData(InMovieScene);
}

FMovieSceneImportData::FMovieSceneImportData() : MovieSceneData(nullptr)
{
}

FMovieSceneImportData::~FMovieSceneImportData()
{
}

bool FMovieSceneImportData::IsImportDataValid() const
{
	return MovieSceneData.IsValid();
}

TSharedPtr<FMovieSceneImportCinematicSectionData> FMovieSceneImportData::FindCinematicSection(const FString& InSectionPathName)
{
	TSharedPtr<FMovieSceneImportCinematicData> CinematicData = GetCinematicData(false);
	if (!CinematicData.IsValid())
	{
		return nullptr;
	}

	for (TSharedPtr<FMovieSceneImportCinematicSectionData> CinematicSection : CinematicData->CinematicSections)
	{
		UMovieSceneCinematicShotSection* CinematicShotSection = CinematicSection->CinematicSection;
		if (CinematicShotSection != nullptr)
		{
			UMovieSceneSequence* ShotSequence = CinematicShotSection->GetSequence();

			FString ShotSectionPathName = CinematicShotSection->GetPathName();
			if (ShotSequence != nullptr && ShotSectionPathName == InSectionPathName)
			{
				return CinematicSection;
			}
		}
	}
	return nullptr;
}

/** Create cinematic section */
TSharedPtr<FMovieSceneImportCinematicSectionData> FMovieSceneImportData::CreateCinematicSection(FString InName, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame)
{
	if (!MovieSceneData.IsValid() || MovieSceneData->MovieScene == nullptr)
	{ 
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportCinematicData> CinematicData = GetCinematicData(true);
	if (!CinematicData.IsValid() || CinematicData->MovieSceneTrack == nullptr)
	{
		return nullptr;
	}

	UMovieSceneCinematicShotTrack* CinematicTrack = Cast<UMovieSceneCinematicShotTrack>(CinematicData->MovieSceneTrack);
	if (CinematicTrack == nullptr)
	{
		return nullptr;
	}

	UMovieSceneSequence* SequenceToAdd = nullptr;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Collect a full list of assets with the specified class
	TArray<FAssetData> AssetDataArray;
	AssetRegistryModule.Get().GetAssetsByClass(ULevelSequence::StaticClass()->GetClassPathName(), AssetDataArray);

	for (FAssetData AssetData : AssetDataArray)
	{
		if (AssetData.AssetName == *InName)
		{
			SequenceToAdd = Cast<ULevelSequence>(AssetData.GetAsset());
			break;
		}
	}

	// If exact match wasn't found, look for the base filename (in case the clip name comes with an extension)
	if (SequenceToAdd == nullptr)
	{
		for (FAssetData AssetData : AssetDataArray)
		{
			if (AssetData.AssetName == *FPaths::GetBaseFilename(InName))
			{
				SequenceToAdd = Cast<ULevelSequence>(AssetData.GetAsset());
				break;
			}
		}
	}

	if (SequenceToAdd == nullptr)
	{
		UE_LOGF(LogMovieScene, Error, "Can't find valid level sequence asset to map clip to: (%ls)", *InName);
	}

	// both FCP XML and Sequencer have inclusive start frame, exclusive end frame
	FFrameRate TickResolution = MovieSceneData->MovieScene->GetTickResolution();
	FFrameNumber StartFrame = ConvertFrameTime(InStartFrame, InFrameRate, TickResolution).RoundToFrame();
	FFrameNumber StartOffsetFrame = ConvertFrameTime(InStartOffsetFrame, InFrameRate, TickResolution).RoundToFrame();
	FFrameNumber EndFrame = ConvertFrameTime(InEndFrame, InFrameRate, TickResolution).RoundToFrame();
	int32 Duration = (EndFrame - StartFrame).Value;
	
	CinematicTrack->Modify();
	UMovieSceneCinematicShotSection* Section = Cast<UMovieSceneCinematicShotSection>(CinematicTrack->AddSequence(SequenceToAdd, StartFrame, Duration));
	if (Section == nullptr)
	{
		return nullptr;
	}
	Section->Modify();
	Section->SetRowIndex(InRow);
	Section->Parameters.StartFrameOffset = StartOffsetFrame.Value;
	Section->SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));

	TSharedPtr<FMovieSceneImportCinematicSectionData> SectionData = ConstructCinematicSectionData(Section);
	CinematicData->CinematicSections.Add(SectionData);
	for (TSharedPtr<FMovieSceneImportCinematicTrackData> TrackData : CinematicData->CinematicTracks)
	{
		if (InRow == TrackData->RowIndex)
		{
			TrackData->CinematicSections.Add(SectionData);
		}
	}
	return SectionData;
}

bool FMovieSceneImportData::SetCinematicSection(TSharedPtr<FMovieSceneImportCinematicSectionData> InSection, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, TOptional<FFrameNumber> InStartOffsetFrame)
{
	if (!InSection.IsValid() || InSection->CinematicSection == nullptr)
	{
		return false;
	}

	FFrameRate TickResolution = MovieSceneData->MovieScene->GetTickResolution();
	FFrameNumber StartFrame = ConvertFrameTime(InStartFrame, InFrameRate, TickResolution).GetFrame();
	FFrameNumber EndFrame = ConvertFrameTime(InEndFrame, InFrameRate, TickResolution).GetFrame();
		
	InSection->CinematicSection->Modify();
	if (InStartOffsetFrame.IsSet())
	{
		FFrameNumber StartOffsetFrame = ConvertFrameTime(InStartOffsetFrame.GetValue(), InFrameRate, TickResolution).GetFrame();
		InSection->CinematicSection->Parameters.StartFrameOffset = StartOffsetFrame;
	}
	InSection->CinematicSection->SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));
	if (InRow != InSection->CinematicSection->GetRowIndex())
	{
		InSection->CinematicSection->SetRowIndex(InRow);
	}

	return true;
}

TSharedPtr<FMovieSceneImportAudioSectionData> FMovieSceneImportData::FindAudioSection(const FString& InSectionPathName, TSharedPtr<FMovieSceneImportAudioData>& OutAudioData)
{
	if (!MovieSceneData.IsValid())
	{
		OutAudioData = nullptr;
		return nullptr;
	}

	for (TSharedPtr<FMovieSceneImportAudioData> AudioData : MovieSceneData->AudioData)
	{
		if (!AudioData.IsValid())
		{
			continue;
		}

		for (TSharedPtr<FMovieSceneImportAudioSectionData> AudioSectionData : AudioData->AudioSections)
		{
			if (AudioSectionData.IsValid() && AudioSectionData->AudioSection != nullptr)
			{
				FString SectionName = AudioSectionData->AudioSection->GetPathName();
				if (SectionName == InSectionPathName)
				{
					OutAudioData = AudioData;
					return AudioSectionData;
				}
			}
		}
	}

	OutAudioData = nullptr;
	return nullptr;
}

/** Create audio section */
TSharedPtr<FMovieSceneImportAudioSectionData> FMovieSceneImportData::CreateAudioSection(FString InFilenameOrAssetPathName, bool bIsPathName, TSharedPtr<FMovieSceneImportAudioData> InData, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame)
{
	if (!MovieSceneData.IsValid() || MovieSceneData->MovieScene == nullptr || !InData.IsValid() || InData->MovieSceneTrack == nullptr)
	{
		return nullptr;
	}

	UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(InData->MovieSceneTrack);
	if (AudioTrack == nullptr)
	{
		return nullptr;
	}

	USoundWave* SoundToAdd = nullptr;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Collect a full list of assets with the specified class
	TArray<FAssetData> AssetDataArray;
	AssetRegistryModule.Get().GetAssetsByClass(USoundWave::StaticClass()->GetClassPathName(), AssetDataArray);

	for (FAssetData AssetData : AssetDataArray)
	{
		USoundWave* SoundWaveAsset = Cast<USoundWave>(AssetData.GetAsset());

		if (SoundWaveAsset == nullptr)
		{
			continue;
		}

		if (bIsPathName)
		{
			if (InFilenameOrAssetPathName == SoundWaveAsset->GetPathName())
			{
				SoundToAdd = SoundWaveAsset;
				break;
			}
		}
		else
		{
			if (SoundWaveAsset->AssetImportData == nullptr)
			{
				continue;
			}

			TArray<FString> Filenames = SoundWaveAsset->AssetImportData->ExtractFilenames();
			if (Filenames.Num() < 1)
			{
				continue;
			}

			FString Filename = FPaths::GetCleanFilename(Filenames[0]);
			if (Filename == InFilenameOrAssetPathName)
			{
				SoundToAdd = SoundWaveAsset;
				break;
			}
		}
	}
	
	if (SoundToAdd == nullptr)
	{
		UE_LOGF(LogMovieScene, Error, "Can't find valid asset asset to map clip to: (%ls)", *InFilenameOrAssetPathName);
		return nullptr;
	}

	// both FCP XML and Sequencer have inclusive start frame, exclusive end frame
	FFrameRate TickResolution = MovieSceneData->MovieScene->GetTickResolution();
	FFrameNumber StartFrame = ConvertFrameTime(InStartFrame, InFrameRate, TickResolution).RoundToFrame();
	FFrameNumber StartOffsetFrame = ConvertFrameTime(InStartOffsetFrame, InFrameRate, TickResolution).RoundToFrame();
	FFrameNumber EndFrame = ConvertFrameTime(InEndFrame, InFrameRate, TickResolution).RoundToFrame();
	int32 Duration = (EndFrame - StartFrame).Value;

	AudioTrack->Modify();
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(AudioTrack->AddNewSoundOnRow(SoundToAdd, StartFrame, InRow));
	if (AudioSection == nullptr)
	{
		return nullptr;
	}
	AudioSection->Modify();
	AudioSection->SetRowIndex(InRow);
	AudioSection->SetStartOffset(StartOffsetFrame.Value);
	AudioSection->SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));

	TSharedPtr<FMovieSceneImportAudioSectionData> AudioSectionData = ConstructAudioSectionData(AudioSection);

	InData->AudioSections.Add(AudioSectionData);
	for (TSharedPtr<FMovieSceneImportAudioTrackData> TrackData : InData->AudioTracks)
	{
		if (InRow == TrackData->RowIndex)
		{
			TrackData->AudioSections.Add(AudioSectionData);
		}
	}

	return AudioSectionData;
}

bool FMovieSceneImportData::SetAudioSection(TSharedPtr<FMovieSceneImportAudioSectionData> InSection, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame)
{
	if (!MovieSceneData.IsValid() || MovieSceneData->MovieScene == nullptr || !InSection.IsValid() || InSection->AudioSection == nullptr)
	{
		return false;
	}

	FFrameRate TickResolution = MovieSceneData->MovieScene->GetTickResolution();
	FFrameNumber StartFrame = ConvertFrameTime(InStartFrame, InFrameRate, TickResolution).GetFrame();
	FFrameNumber StartOffsetFrame = ConvertFrameTime(InStartOffsetFrame, InFrameRate, TickResolution).GetFrame();
	FFrameNumber EndFrame = ConvertFrameTime(InEndFrame, InFrameRate, TickResolution).GetFrame();

	InSection->AudioSection->Modify();
	InSection->AudioSection->SetStartOffset(StartOffsetFrame.Value);
	InSection->AudioSection->SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));
	if (InRow != InSection->AudioSection->GetRowIndex())
	{
		InSection->AudioSection->SetRowIndex(InRow);
	}

	return true;
}

bool FMovieSceneImportData::MoveAudioSection(TSharedPtr<FMovieSceneImportAudioSectionData> InAudioSectionData, TSharedPtr<FMovieSceneImportAudioData> InFromData, TSharedPtr<FMovieSceneImportAudioData> InToData, int32 InToRowIndex)
{
	if (!MovieSceneData.IsValid() || !InAudioSectionData.IsValid() || !InFromData.IsValid() || !InToData.IsValid())
	{
		return false;
	}

	UMovieSceneAudioTrack* FromTrack = Cast<UMovieSceneAudioTrack>(InFromData->MovieSceneTrack);
	if (FromTrack == nullptr)
	{
		return false;
	}

	UMovieSceneAudioTrack* ToTrack = Cast<UMovieSceneAudioTrack>(InToData->MovieSceneTrack);
	if (ToTrack == nullptr)
	{
		return false;
	}
	UMovieSceneAudioSection* AudioSection = InAudioSectionData->AudioSection;
	if (AudioSection == nullptr)
	{
		return false;
	}

	FromTrack->Modify();
	FromTrack->RemoveSection(*AudioSection);
	ToTrack->Modify();
	ToTrack->AddSection(*AudioSection);

	InFromData->AudioSections.Remove(InAudioSectionData);
	for (TSharedPtr<FMovieSceneImportAudioTrackData> AudioTrackData : InFromData->AudioTracks)
	{
		if (AudioTrackData->AudioSections.Contains(InAudioSectionData))
		{
			AudioTrackData->AudioSections.Remove(InAudioSectionData);
		}
	}

	bool bFoundTrack = false;
	InToData->AudioSections.Add(InAudioSectionData);
	for (TSharedPtr<FMovieSceneImportAudioTrackData> AudioTrackData : InToData->AudioTracks)
	{
		if (AudioTrackData->RowIndex == InToRowIndex)
		{
			AudioTrackData->AudioSections.Add(InAudioSectionData);
			bFoundTrack = true;
			break;
		}
	}

	if (!bFoundTrack)
	{
		TSharedPtr<FMovieSceneImportAudioTrackData> TrackData = MakeShared<FMovieSceneImportAudioTrackData>();
		TrackData->RowIndex = InToRowIndex;
		TrackData->AudioSections.Add(InAudioSectionData);
		InToData->AudioTracks.Add(TrackData);
	}

	return true;
}

TSharedPtr<FMovieSceneImportCinematicData> FMovieSceneImportData::GetCinematicData(bool CreateTrackIfNull) 
{
	if (!MovieSceneData.IsValid())
	{
		return nullptr;
	}
	if (!MovieSceneData->CinematicData.IsValid() && CreateTrackIfNull)
	{
		UMovieSceneCinematicShotTrack* CinematicTrack = MovieSceneData->MovieScene->AddTrack<UMovieSceneCinematicShotTrack>();
		MovieSceneData->CinematicData = ConstructCinematicData(CinematicTrack);
	}
	return MovieSceneData->CinematicData;
}

TSharedPtr<FMovieSceneImportAudioData> FMovieSceneImportData::GetAudioData()
{
	if (!MovieSceneData.IsValid())
	{
		return nullptr;
	}
	for (TSharedPtr<FMovieSceneImportAudioData> AudioData : MovieSceneData->AudioData)
	{
		if (AudioData.IsValid())
		{
			return AudioData;
		}
	}
	return nullptr;
}

TSharedPtr<FMovieSceneImportMovieSceneData> FMovieSceneImportData::ConstructMovieSceneData(UMovieScene* InMovieScene)
{
	if (InMovieScene == nullptr)
	{
		return nullptr;
	}

	MovieSceneData = MakeShared<FMovieSceneImportMovieSceneData>();
	MovieSceneData->MovieScene = InMovieScene;

	// Get cinematic track
	UMovieSceneCinematicShotTrack* CinematicTrack = InMovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (CinematicTrack != nullptr)
	{
		MovieSceneData->CinematicData = ConstructCinematicData(CinematicTrack);
		if (!MovieSceneData->CinematicData.IsValid())
		{
			return nullptr;
		}
	}

	// Get audio tracks
	TMap<int32, TSharedPtr<FMovieSceneImportAudioData>> AudioDataMap;

	const TArray<UMovieSceneTrack*> Tracks = InMovieScene->GetTracks();
	for (UMovieSceneTrack* Track : Tracks)
	{
		if (Track->IsA(UMovieSceneAudioTrack::StaticClass()))
		{

			UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(Track);
			if (AudioTrack == nullptr)
			{
				continue;
			}

			TSharedPtr<FMovieSceneImportAudioData> AudioData = ConstructAudioData(AudioTrack);
			if (!AudioData.IsValid())
			{
				continue;
			}

			AudioDataMap.Add(AudioTrack->GetSortingOrder(), AudioData);
		}
	}


	// sort the audio tracks by their sorting index and add to the AudioTracks array
	if (AudioDataMap.Num() > 0)
	{
		AudioDataMap.KeySort([](int32 A, int32 B) {
			return A < B; // sort keys in order
		});

		for (auto& Elem : AudioDataMap)
		{
			if (Elem.Value.IsValid())
			{
				MovieSceneData->AudioData.Add(Elem.Value);

				if (!SupportMultipleAudioTracks())
				{
					break;
				}
			}
		}
	}

	return MovieSceneData;
}


TSharedPtr<FMovieSceneImportCinematicData> FMovieSceneImportData::ConstructCinematicData(UMovieSceneCinematicShotTrack* InCinematicTrack)
{
	if (!MovieSceneData.IsValid() || InCinematicTrack == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportCinematicData> CinematicData = MakeShared<FMovieSceneImportCinematicData>();
	CinematicData->MovieSceneTrack = InCinematicTrack;

	// Construct sections & create track row index array
	TArray<int32> CinematicTrackRowIndices;

	for (UMovieSceneSection* ShotSection : InCinematicTrack->GetAllSections())
	{
		UMovieSceneCinematicShotSection* CinematicSection = Cast<UMovieSceneCinematicShotSection>(ShotSection);

		if (CinematicSection != nullptr && CinematicSection->GetSequence() != nullptr)
		{
			int32 RowIndex = CinematicSection->GetRowIndex();
			if (RowIndex >= 0)
			{
				CinematicTrackRowIndices.AddUnique(RowIndex);
			}
		}
	}

	// Construct tracks and point to sections
	CinematicTrackRowIndices.Sort();

	for (int32 CinematicTrackRowIndex : CinematicTrackRowIndices)
	{
		TSharedPtr<FMovieSceneImportCinematicTrackData> TrackData = ConstructCinematicTrackData(InCinematicTrack, CinematicTrackRowIndex);
		if (TrackData.IsValid())
		{
			CinematicData->CinematicTracks.Add(TrackData);

			for (TSharedPtr<FMovieSceneImportCinematicSectionData> SectionData : TrackData->CinematicSections)
			{
				if (SectionData.IsValid())
				{
					CinematicData->CinematicSections.Add(SectionData);
				}
			}
		}
	}

	return CinematicData;
}

TSharedPtr<FMovieSceneImportCinematicTrackData> FMovieSceneImportData::ConstructCinematicTrackData(UMovieSceneCinematicShotTrack* InCinematicTrack, int32 InRowIndex)
{
	if (!MovieSceneData.IsValid() || InCinematicTrack == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportCinematicTrackData> TrackData = MakeShared<FMovieSceneImportCinematicTrackData>();
	TrackData->RowIndex = InRowIndex;

	for (UMovieSceneSection* ShotSection : InCinematicTrack->GetAllSections())
	{
		UMovieSceneCinematicShotSection* CinematicSection = Cast<UMovieSceneCinematicShotSection>(ShotSection);

		if (CinematicSection != nullptr && CinematicSection->GetSequence() != nullptr && CinematicSection->GetRowIndex() == InRowIndex)
		{
			TSharedPtr<FMovieSceneImportCinematicSectionData> CinematicSectionData = ConstructCinematicSectionData(CinematicSection);
			if (!CinematicSectionData.IsValid())
			{
				return nullptr;
			}
			TrackData->CinematicSections.Add(CinematicSectionData);
		}
	}

	return TrackData;
}


TSharedPtr<FMovieSceneImportAudioData> FMovieSceneImportData::ConstructAudioData(UMovieSceneAudioTrack* InAudioTrack)
{
	if (!MovieSceneData.IsValid() || InAudioTrack == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportAudioData> AudioData = MakeShared<FMovieSceneImportAudioData>();
	AudioData->MovieSceneTrack = InAudioTrack;
	AudioData->MaxRowIndex = 0;

	// Construct sections & create track row index array
	TArray<int32> AudioTrackRowIndices;

	for (UMovieSceneSection* ShotSection : InAudioTrack->GetAllSections())
	{
		UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(ShotSection);

		if (AudioSection != nullptr)
		{
			int32 RowIndex = AudioSection->GetRowIndex();
			if (RowIndex >= 0)
			{
				AudioTrackRowIndices.AddUnique(RowIndex);

				if (RowIndex > AudioData->MaxRowIndex)
				{
					AudioData->MaxRowIndex = RowIndex;
				}
			}
		}
	}

	// Construct tracks and point to sections
	AudioTrackRowIndices.Sort();

	for (int32 AudioTrackRowIndex : AudioTrackRowIndices)
	{
		TSharedPtr<FMovieSceneImportAudioTrackData> TrackData = ConstructAudioTrackData(InAudioTrack, AudioTrackRowIndex);
		if (TrackData.IsValid())
		{
			AudioData->AudioTracks.Add(TrackData);

			for (TSharedPtr<FMovieSceneImportAudioSectionData> SectionData : TrackData->AudioSections)
			{
				if (SectionData.IsValid())
				{
					AudioData->AudioSections.Add(SectionData);
				}
			}
		}
	}

	return AudioData;
}

TSharedPtr<FMovieSceneImportAudioTrackData> FMovieSceneImportData::ConstructAudioTrackData(UMovieSceneAudioTrack* InAudioTrack, int32 InRowIndex)
{
	if (!MovieSceneData.IsValid() || InAudioTrack == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportAudioTrackData> TrackData = MakeShared<FMovieSceneImportAudioTrackData>();
	TrackData->RowIndex = InRowIndex;

	for (UMovieSceneSection* Section : InAudioTrack->GetAllSections())
	{
		UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Section);

		if (AudioSection != nullptr && AudioSectionIsSoundWave(AudioSection) && AudioSection->GetRowIndex() == InRowIndex)
		{
			TSharedPtr<FMovieSceneImportAudioSectionData> AudioSectionData = ConstructAudioSectionData(AudioSection);
			if (!AudioSectionData.IsValid())
			{
				continue;
			}
			TrackData->AudioSections.Add(AudioSectionData);
		}
	}

	return TrackData;
}

TSharedPtr<FMovieSceneImportCinematicSectionData> FMovieSceneImportData::ConstructCinematicSectionData(UMovieSceneCinematicShotSection* InCinematicSection)
{
	if (!MovieSceneData.IsValid() || MovieSceneData->MovieScene == nullptr || InCinematicSection == nullptr)
	{
		return nullptr;
	}	

	TSharedPtr<FMovieSceneImportCinematicSectionData> SectionData = MakeShared<FMovieSceneImportCinematicSectionData>();
	SectionData->CinematicSection = InCinematicSection;
	
	return SectionData;
}

TSharedPtr<FMovieSceneImportAudioSectionData> FMovieSceneImportData::ConstructAudioSectionData(UMovieSceneAudioSection* InAudioSection)
{
	if (InAudioSection == nullptr)
	{
		return nullptr;
	}

	USoundBase* SoundBase = InAudioSection->GetSound();
	if (SoundBase == nullptr || !SoundBase->IsA<USoundWave>())
	{
		return nullptr;
	}

	USoundWave* SoundWave = nullptr;
	SoundWave = Cast<USoundWave>(SoundBase);
	if (SoundWave == nullptr || SoundWave->AssetImportData == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<FMovieSceneImportAudioSectionData> SectionData = MakeShared<FMovieSceneImportAudioSectionData>();
	SectionData->AudioSection = InAudioSection;

	TArray<FString> Filenames = SoundWave->AssetImportData->ExtractFilenames();
	if (Filenames.Num() < 1)
	{
		return nullptr;
	}

	SectionData->SourceFilename = FPaths::GetCleanFilename(Filenames[0]);
	SectionData->SourceFilePath = FPaths::GetPath(Filenames[0]);

	return SectionData;
}

void FMovieSceneTranslatorContext::Init()
{
	ClearMessages();
}

void FMovieSceneTranslatorContext::AddMessage(EMessageSeverity::Type InMessageSeverity, FText InMessage)
{
	Messages.Add(FTokenizedMessage::Create(InMessageSeverity, InMessage));
}

void FMovieSceneTranslatorContext::ClearMessages()
{
	Messages.Empty();
}

bool FMovieSceneTranslatorContext::ContainsMessageType(EMessageSeverity::Type InMessageSeverity) const
{
	for (const TSharedRef<FTokenizedMessage>& Message : Messages)
	{
		if (Message->GetSeverity() == InMessageSeverity)
		{
			return true;
		}
	}
	return false;
}

const TArray<TSharedRef<FTokenizedMessage>>& FMovieSceneTranslatorContext::GetMessages() const
{
	return Messages;
}

#undef LOCTEXT_NAMESPACE
