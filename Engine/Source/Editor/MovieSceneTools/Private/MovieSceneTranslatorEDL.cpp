// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTranslatorEDL.h"
#include "MovieScene.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/FrameRate.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneCaptureModule.h"
#include "Misc/Timecode.h"

/* MovieSceneCaptureHelpers
 *****************************************************************************/

struct FShotData
{
	enum class ETrackType
	{
		TT_Video,
		TT_A,
		TT_A2,
		TT_AA,
		TT_None
	};

	enum class EEditType
	{
		ET_Cut,
		ET_Dissolve,
		ET_Wipe,
		ET_KeyEdit,
		ET_None
	};

	FShotData(const FString& InElementName, const FString& InElementPath, ETrackType InTrackType, EEditType InEditType, FFrameNumber InSourceInFrame, FFrameNumber InSourceOutFrame, FFrameNumber InEditInFrame, FFrameNumber InEditOutFrame, bool bInWithinPlaybackRange) : 
		  ElementName(InElementName)
		, ElementPath(InElementPath)
		, TrackType(InTrackType)
		, EditType(InEditType)
		, SourceInFrame(InSourceInFrame)
		, SourceOutFrame(InSourceOutFrame)
		, EditInFrame(InEditInFrame)
		, EditOutFrame(InEditOutFrame)
		, bWithinPlaybackRange(bInWithinPlaybackRange) {}

	bool operator<(const FShotData& Other) const { return EditInFrame < Other.EditInFrame; }

	FString ElementName;
	FString ElementPath;
	ETrackType TrackType;
	EEditType EditType;
	FFrameNumber SourceInFrame;
	FFrameNumber SourceOutFrame;
	FFrameNumber EditInFrame;
	FFrameNumber EditOutFrame;
	bool bWithinPlaybackRange;
};

FFrameNumber SMPTEToFrame(FString SMPTE, FFrameRate TickResolution, FFrameRate FrameRate)
{
	TArray<FString> OutArray;
	SMPTE.ParseIntoArray(OutArray, TEXT(":"));
	
	if (OutArray.Num() == 4)
	{
		const int32 Hours   = FCString::Atoi(*OutArray[0]);
		const int32 Minutes = FCString::Atoi(*OutArray[1]);
		const int32 Seconds = FCString::Atoi(*OutArray[2]);
		const int32 Frames  = FCString::Atoi(*OutArray[3]);

		FTimecode Timecode(Hours, Minutes, Seconds, Frames, false);

		return FFrameRate::TransformTime(Timecode.ToFrameNumber(FrameRate), FrameRate, TickResolution).RoundToFrame();
	}
	// The edl is in frames
	else
	{
		return FCString::Atoi(*SMPTE);
	}
}

FString TimeToSMPTE(FFrameNumber InTime, FFrameRate TickResolution, FFrameRate FrameRate)
{
	FFrameNumber PlayRateFrameNumber = FFrameRate::TransformTime(InTime, TickResolution, FrameRate).RoundToFrame();

	return FTimecode::FromFrameNumber(PlayRateFrameNumber, FrameRate, false).ToString();
}

void ParseFromEDL(const FString& InputString, FFrameRate TickResolution, FFrameRate FrameRate, TArray<FShotData>& OutShotData)
{
	TArray<FString> InputLines;
	InputString.ParseIntoArray(InputLines, TEXT("\n"));

	bool bFoundEventLine = false;
	FString EventName;
	FString AuxName;
	FString ReelName;
	FShotData::ETrackType TrackType = FShotData::ETrackType::TT_None;
	FShotData::EEditType EditType = FShotData::EEditType::ET_None;

	FFrameNumber SourceInFrame  = 0;
	FFrameNumber SourceOutFrame = 0;
	FFrameNumber EditInFrame    = 0;
	FFrameNumber EditOutFrame   = 0;

	for (FString InputLine : InputLines)
	{
		TArray<FString> InputChars;
		InputLine.ParseIntoArrayWS(InputChars);

		// First look for :
		// 001 AX V C 00:00:00:00 00:00:12:02 00:00:07:20 00:00:12:03
		if (!bFoundEventLine)
		{
			if (InputChars.Num() >= 8)
			{
				EventName = InputChars[0];
				AuxName = InputChars[1]; // Typically AX but unused in this case
				
				if (InputChars[1] == TEXT("BL"))
				{
					TrackType = FShotData::ETrackType::TT_None;
				}
				else if (InputChars[2] == TEXT("V"))
				{
					TrackType = FShotData::ETrackType::TT_Video;
				}
				else if (InputChars[2] == TEXT("A"))
				{
					TrackType = FShotData::ETrackType::TT_A;
				}
				else if (InputChars[2] == TEXT("A2"))
				{
					TrackType = FShotData::ETrackType::TT_A2;
				}
				else if (InputChars[2] == TEXT("AA"))
				{
					TrackType = FShotData::ETrackType::TT_AA;
				}

				if (InputChars[3] == TEXT("C"))
				{
					EditType = FShotData::EEditType::ET_Cut;
				}
				else if (InputChars[3] == TEXT("D"))
				{
					EditType = FShotData::EEditType::ET_Dissolve;
				}
				else if (InputChars[3] == TEXT("W"))
				{
					EditType = FShotData::EEditType::ET_Wipe;
				}
				else if (InputChars[3] == TEXT("K"))
				{
					EditType = FShotData::EEditType::ET_KeyEdit;
				}

				// If everything checks out
				if (TrackType != FShotData::ETrackType::TT_None &&
					EditType != FShotData::EEditType::ET_None)
				{
					// Look for timecodes at the end
					SourceInFrame  = SMPTEToFrame(InputChars[InputChars.Num() - 4], TickResolution, FrameRate);
					SourceOutFrame = SMPTEToFrame(InputChars[InputChars.Num() - 3], TickResolution, FrameRate);
					EditInFrame    = SMPTEToFrame(InputChars[InputChars.Num() - 2], TickResolution, FrameRate);
					EditOutFrame   = SMPTEToFrame(InputChars[InputChars.Num() - 1], TickResolution, FrameRate);

					bFoundEventLine = true;
					continue; // Go to the next line
				}
			}
		}
		
		// Then look for:
		// * FROM CLIP NAME: shot0010_001.avi
		// * KEY CLIP NAME: shot0010_001.avi
		else
		{
			if (InputChars.Num() == 5 &&
				InputChars[0] == TEXT("*") &&
				( (InputChars[1].ToUpper() == TEXT("FROM") && InputChars[2].ToUpper() == TEXT("CLIP")) ||
				  (InputChars[1].ToUpper() == TEXT("KEY") && InputChars[2].ToUpper() == TEXT("CLIP")) ) &&
				InputChars[3].ToUpper() == TEXT("NAME:"))
			{
				ReelName = InputChars[4];

				if (ReelName.EndsWith(TEXT(".avi")))
				{
					ReelName.LeftChopInline(4, EAllowShrinking::No); // strip .avi
				}

				FString ElementName = MoveTemp(ReelName);
				FString ElementPath = ElementName;

				// If everything checks out add to OutShotData
				OutShotData.Add(FShotData(ElementName, ElementPath, TrackType, EditType, SourceInFrame, SourceOutFrame, EditInFrame, EditOutFrame, true));
				bFoundEventLine = false; // Reset and go to next line to look for element line
				continue;
			}
		}
	}
}

void FormatForEDL(FString& OutputString, const FString& SequenceName, FFrameRate TickResolution, FFrameRate FrameRate, const TArray<FShotData>& InShotData)
{
	OutputString += TEXT("TITLE: ") + SequenceName + TEXT("\n");
	OutputString += TEXT("FCM: NON-DROP FRAME\n\n");

	int32 EventIndex = 0;
	FString EventName, ReelName, EditName, TypeName;
	FString SourceSMPTEIn, SourceSMPTEOut, EditSMPTEIn, EditSMPTEOut;

	// Insert blank if doesn't start at 0.
	if (InShotData[0].EditInFrame != 0)
	{
		EventName = FString::Printf(TEXT("%03d"), ++EventIndex);
		TypeName = TEXT("V");
		EditName = TEXT("C");

		SourceSMPTEIn  = TimeToSMPTE(0, TickResolution, FrameRate);
		SourceSMPTEOut = TimeToSMPTE(InShotData[0].EditInFrame, TickResolution, FrameRate);
		EditSMPTEIn    = TimeToSMPTE(0, TickResolution, FrameRate);
		EditSMPTEOut   = TimeToSMPTE(InShotData[0].EditInFrame, TickResolution, FrameRate);

		OutputString += EventName + TEXT(" ") + TEXT("BL ") + TypeName + TEXT(" ") + EditName + TEXT(" ");
		OutputString += SourceSMPTEIn + TEXT(" ") + SourceSMPTEOut + TEXT(" ") + EditSMPTEIn + TEXT(" ") + EditSMPTEOut + TEXT("\n\n");
	}

	for (int32 ShotIndex = 0; ShotIndex < InShotData.Num(); ++ShotIndex)
	{
		EventName = FString::Printf(TEXT("%03d"), ++EventIndex);

		ReelName = InShotData[ShotIndex].ElementName; 

		if (InShotData[ShotIndex].TrackType == FShotData::ETrackType::TT_Video)
		{
			TypeName = TEXT("V");
		}
		else if (InShotData[ShotIndex].TrackType == FShotData::ETrackType::TT_A)
		{
			TypeName = TEXT("A");
		}
		else if (InShotData[ShotIndex].TrackType == FShotData::ETrackType::TT_A2)
		{
			TypeName = TEXT("A2");
		}
		else if (InShotData[ShotIndex].TrackType == FShotData::ETrackType::TT_AA)
		{
			TypeName = TEXT("AA");
		}

		if (InShotData[ShotIndex].EditType == FShotData::EEditType::ET_Cut)
		{
			EditName = TEXT("C");
		}
		else if (InShotData[ShotIndex].EditType == FShotData::EEditType::ET_Dissolve)
		{
			EditName = TEXT("D");
		}
		else if (InShotData[ShotIndex].EditType == FShotData::EEditType::ET_Wipe)
		{
			EditName = TEXT("W");
		}
		else if (InShotData[ShotIndex].EditType == FShotData::EEditType::ET_KeyEdit)
		{
			EditName = TEXT("K");
		}

		SourceSMPTEIn  = TimeToSMPTE(InShotData[ShotIndex].SourceInFrame, TickResolution, FrameRate);
		SourceSMPTEOut = TimeToSMPTE(InShotData[ShotIndex].SourceOutFrame, TickResolution, FrameRate);
		EditSMPTEIn    = TimeToSMPTE(InShotData[ShotIndex].EditInFrame, TickResolution, FrameRate);
		EditSMPTEOut   = TimeToSMPTE(InShotData[ShotIndex].EditOutFrame, TickResolution, FrameRate);
	
		OutputString += EventName + TEXT(" ") + TEXT("AX ") + TypeName + TEXT(" ") + EditName + TEXT(" ");
		OutputString += SourceSMPTEIn + TEXT(" ") + SourceSMPTEOut + TEXT(" ") + EditSMPTEIn + TEXT(" ") + EditSMPTEOut + TEXT("\n");
		OutputString += TEXT("* FROM CLIP NAME: ") + ReelName + TEXT("\n\n");
	}
}

void FormatForRV(FString& OutputString, const FString& SequenceName, FFrameRate TickResolution, FFrameRate FrameRate, const TArray<FShotData>& InShotData)
{
	// Header
	OutputString += TEXT("GTOa (3)\n\n");
	OutputString += TEXT("rv : RVSession (2)\n");
	OutputString += TEXT("{\n");
	OutputString += TEXT("\tsession\n");
	OutputString += TEXT("\t{\n");
	OutputString += TEXT("\t\tfloat fps = ") + FString::Printf(TEXT("%f"), FrameRate.AsDecimal()) + TEXT("\n");
	OutputString += TEXT("\t\tint realtime = 1\n");
	OutputString += TEXT("\t}\n\n");
	OutputString += TEXT("\twriter\n");
	OutputString += TEXT("\t{\n");
	OutputString += TEXT("\t\tstring name = \"rvSession.py\"\n");
	OutputString += TEXT("\t\tstring version = \"0.3\"\n");
	OutputString += TEXT("\t}\n");
	OutputString += TEXT("}\n\n");

	// Body
	for (int32 EventIndex = 0; EventIndex < InShotData.Num(); ++EventIndex)
	{
		if (!InShotData[EventIndex].bWithinPlaybackRange)
		{
			continue;
		}

		FString SourceName = FString::Printf(TEXT("sourceGroup%06d"), EventIndex);

		FFrameTime SourceInTime = FFrameRate::TransformTime(InShotData[EventIndex].SourceInFrame, TickResolution, FrameRate);
		FFrameTime SourceOutTime = FFrameRate::TransformTime(InShotData[EventIndex].SourceOutFrame, TickResolution, FrameRate);

		OutputString += SourceName + TEXT(" : RVSourceGroup (1)\n");
		OutputString += TEXT("{\n");
		OutputString += TEXT("\tui\n");
		OutputString += TEXT("\t{\n");
		OutputString += TEXT("\t\tstring name = \"") + InShotData[EventIndex].ElementName + TEXT("\"\n");
		OutputString += TEXT("\t}\n");
		OutputString += TEXT("}\n\n");

		OutputString += SourceName + TEXT("_source : RVFileSource (1)\n");
		OutputString += TEXT("{\n");
		OutputString += TEXT("\tcut\n");
		OutputString += TEXT("\t{\n");
		OutputString += TEXT("\t\tint in = ") + FString::FromInt(SourceInTime.GetFrame().Value) + TEXT("\n");
		OutputString += TEXT("\t\tint out = ") + FString::FromInt(SourceOutTime.GetFrame().Value) + TEXT("\n");
		OutputString += TEXT("\t}\n\n");

		OutputString += TEXT("\tgroup\n");
		OutputString += TEXT("\t{\n");
		OutputString += TEXT("\t\tint noMovieAudio = 1\n");
		OutputString += TEXT("\t}\n\n");

		OutputString += TEXT("\tmedia\n");
		OutputString += TEXT("\t{\n");
		OutputString += TEXT("\t\tstring movie = \"") + InShotData[EventIndex].ElementPath + TEXT("\"\n");
		OutputString += TEXT("\t\tstring shot = \"\"\n");
		OutputString += TEXT("\t}\n");
		OutputString += TEXT("}\n\n");
	}
}

void FormatForRVBat(FString& OutputString, const FString& SequenceName, FFrameRate TickResolution, FFrameRate FrameRate, const TArray<FShotData>& InShotData)
{
	OutputString += TEXT("rv -nomb -fullscreen -noBorders -fps ") + FString::Printf(TEXT("%f"), FrameRate.AsDecimal());
	for (int32 EventIndex = 0; EventIndex < InShotData.Num(); ++EventIndex)
	{
		if (InShotData[EventIndex].bWithinPlaybackRange)
		{
			OutputString += " " + InShotData[EventIndex].ElementName;
		}
	}
}


bool MovieSceneTranslatorEDL::ImportEDL(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InFilename)
{
	FString InputString;
	if (!FFileHelper::LoadFileToString(InputString, *InFilename))
	{
		return false;
	}

	FFrameRate TickResolution = InMovieScene->GetTickResolution();

	TArray<FShotData> ShotDataArray;
	ParseFromEDL(InputString, TickResolution, InFrameRate, ShotDataArray);

	UMovieSceneCinematicShotTrack* CinematicShotTrack = InMovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (!CinematicShotTrack)
	{
		CinematicShotTrack = InMovieScene->AddTrack<UMovieSceneCinematicShotTrack>();
	}

	TSet<UMovieSceneSection*> RelevantSections;

	for (FShotData ShotData : ShotDataArray)
	{
		if (ShotData.TrackType == FShotData::ETrackType::TT_Video)
		{
			UMovieSceneCinematicShotSection* ShotSection = nullptr;

			FString ShotName = ShotData.ElementName;
			for (UMovieSceneSection* Section : CinematicShotTrack->GetAllSections())
			{
				UMovieSceneCinematicShotSection* CinematicShotSection = Cast<UMovieSceneCinematicShotSection>(Section);
				if (CinematicShotSection != nullptr && !RelevantSections.Contains(Section))
				{
					UMovieSceneSequence* ShotSequence = CinematicShotSection->GetSequence();
				
					if (ShotSequence != nullptr && ShotSequence->GetName() == ShotName)
					{
						ShotSection = CinematicShotSection;
						break;
					}
				}
			}

			// If the shot doesn't already exist, create it
			if (!ShotSection)
			{
				UMovieSceneSequence* SequenceToAdd = nullptr;

				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

				// Collect a full list of assets with the specified class
				TArray<FAssetData> AssetDataArray;
				AssetRegistryModule.Get().GetAssetsByClass(ULevelSequence::StaticClass()->GetClassPathName(), AssetDataArray);

				for (FAssetData AssetData : AssetDataArray)
				{
					if (AssetData.AssetName == *ShotName)
					{	
						SequenceToAdd = Cast<ULevelSequence>(AssetData.GetAsset());
						break;
					}
				}

				CinematicShotTrack->Modify();
				ShotSection = Cast<UMovieSceneCinematicShotSection>(CinematicShotTrack->AddSequence(SequenceToAdd, ShotData.EditInFrame, (ShotData.EditOutFrame - ShotData.EditInFrame).Value));
			}

			// Conform this shot section
			if (ShotSection)
			{
				RelevantSections.Add(ShotSection);

				ShotSection->Modify();
				ShotSection->Parameters.StartFrameOffset = ShotData.SourceInFrame;
				ShotSection->SetRange(TRange<FFrameNumber>(ShotData.EditInFrame, ShotData.EditOutFrame));
			}
		}
	}

	// Remove irrelevant sections backwards to ensure prior indices remain valid
	const TArray<UMovieSceneSection*>& Sections = CinematicShotTrack->GetAllSections();
	for (int32 Index = Sections.Num()-1; Index >= 0; --Index) 
	{
		if (!RelevantSections.Contains(Sections[Index]))
		{
			CinematicShotTrack->RemoveSectionAt(Index);
		}
	}

	return true;
}

bool MovieSceneTranslatorEDL::ExportEDL(const UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InSaveFilename, const int32 InHandleFrames, FString InMovieExtension)
{
	FString SequenceName = InMovieScene->GetOuter()->GetName();
	FString SaveBasename = FPaths::GetPath(InSaveFilename) / FPaths::GetBaseFilename(InSaveFilename);

	TArray<FString> SaveFilenames;
	if (!SaveBasename.IsEmpty())
	{
		SaveFilenames.Add(SaveBasename + TEXT(".rv"));
		SaveFilenames.Add(SaveBasename + TEXT(".edl"));
		SaveFilenames.Add(SaveBasename + TEXT(".bat"));
	}

	TArray<FShotData> ShotDataArray;

	for (UMovieSceneTrack* Track : InMovieScene->GetTracks())
	{
		// @todo: sequencer-timecode: deal with framerate differences??
		TRange<FFrameNumber> PlaybackRange = InMovieScene->GetPlaybackRange();
		if (Track->IsA(UMovieSceneCinematicShotTrack::StaticClass()))
		{
			UMovieSceneCinematicShotTrack* CinematicShotTrack = Cast<UMovieSceneCinematicShotTrack>(Track);

			for (UMovieSceneSection* ShotSection : CinematicShotTrack->GetAllSections())
			{
				UMovieSceneCinematicShotSection* CinematicShotSection = Cast<UMovieSceneCinematicShotSection>(ShotSection);

				if ( CinematicShotSection->GetSequence() == nullptr || !CinematicShotSection->HasStartFrame() || !CinematicShotSection->HasEndFrame() || !CinematicShotSection->IsActive())
				{
					// If the shot doesn't have a valid sequence skip it.  This is currently the case for filler sections.
					// TODO: Handle this properly in the edl output.
					continue;
				}

				FFrameRate TickResolution = CinematicShotSection->GetSequence()->GetMovieScene()->GetTickResolution();

				FString ShotName = CinematicShotSection->GetShotDisplayName();
				FString ShotPath = CinematicShotSection->GetSequence()->GetMovieScene()->GetOuter()->GetPathName();

				FFrameNumber SourceInFrame = ConvertFrameTime(InHandleFrames, InFrameRate, TickResolution).FrameNumber;
				FFrameNumber SourceOutFrame = ConvertFrameTime(InHandleFrames, InFrameRate, TickResolution).FrameNumber + UE::MovieScene::DiscreteSize(CinematicShotSection->GetRange());

				FFrameNumber EditInFrame    = CinematicShotSection->GetInclusiveStartFrame();
				FFrameNumber EditOutFrame   = CinematicShotSection->GetExclusiveEndFrame();

				ShotName += InMovieExtension;

				//@todo shotpath should really be moviefile path
				ShotPath = ShotName;

				TRange<FFrameNumber> EditRange    = TRange<FFrameNumber>(EditInFrame, EditOutFrame);
				TRange<FFrameNumber> Intersection = TRange<FFrameNumber>::Intersection(PlaybackRange, EditRange);

				bool bWithinPlaybackRange = EditRange.Overlaps(PlaybackRange);

				ShotDataArray.Add(FShotData(ShotName, ShotPath, FShotData::ETrackType::TT_Video, FShotData::EEditType::ET_Cut, SourceInFrame, SourceOutFrame, EditInFrame, EditOutFrame, bWithinPlaybackRange));
			}
		}
		else if (Track->IsA(UMovieSceneAudioTrack::StaticClass()))
		{
			UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(Track);

			//@todo support audio clips
		}
	}

	if (ShotDataArray.Num() == 0)
	{
		return false;
	}

	ShotDataArray.Sort();

	FFrameRate TickResolution = InMovieScene->GetTickResolution();

	for (auto SaveFilename : SaveFilenames)
	{
		FString OutputString;
		const FString SaveFilenameExtension = FPaths::GetExtension(SaveFilename);

		if (SaveFilenameExtension == TEXT("EDL"))
		{
			FormatForEDL(OutputString, SequenceName, TickResolution, InFrameRate, ShotDataArray);
		}
		else if (SaveFilenameExtension == TEXT("RV"))
		{
			FormatForRV(OutputString, SequenceName, TickResolution, InFrameRate, ShotDataArray);
		}
		else if (SaveFilenameExtension == TEXT("BAT"))
		{
			FormatForRVBat(OutputString, SequenceName, TickResolution, InFrameRate, ShotDataArray);
		}

		FFileHelper::SaveStringToFile(OutputString, *SaveFilename);
	}

	return true;
}
