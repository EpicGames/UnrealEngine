// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectBindingTagCache.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "Math/NumericLimits.h"
#include "MovieScene.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequence.h"
#include "Templates/Tuple.h"

float GSequencerTagSaturation = 0.6f;
FAutoConsoleVariableRef CVarSequencerTagSaturation(
	TEXT("Sequencer.TagSaturation"),
	GSequencerTagSaturation,
	TEXT("Specifies how saturated object binding tags should appear in the Sequencer UI.\n"),
	ECVF_Default
);


void FObjectBindingTagCache::ConditionalUpdate(UMovieSceneSequence* RootSequence)
{
	if (!RootSequence)
	{
		if (RootSequenceSignature.IsValid())
		{
			RootSequenceSignature = FGuid();
			ExposedNameReverseLUT.Empty();
			OnUpdatedEvent.Broadcast(this);
		}
		return;
	}

	if (RootSequenceSignature.IsValid() && RootSequence->GetSignature() == RootSequenceSignature)
	{
		return;
	}

	RootSequenceSignature = RootSequence->GetSignature();

	ExposedNameReverseLUT.Empty();

	for (const TTuple<FName, FMovieSceneObjectBindingIDs>& Pair : RootSequence->GetMovieScene()->AllTaggedBindings())
	{
		FName Tag = Pair.Key;

		// Define and cache a color for this type if necessary
		if (!ExposedNameColors.Contains(Tag))
		{
			// Could use SHA1 here which might give a better distribution of color hues but I don't have good sample data to test with.
			// CRC32 is fine for know until we know better
			// BYTE HashBytes[20];
			// FString TagString = Tag.ToString();

			// FSHA1::HashBuffer(*TagString, sizeof(TCHAR)*TagString.Len(), HashBytes);

			// Down sample the 20 byte hash to a 4 byte range
			// uint32 StringHash = 0;
			// for (int32 i = 0; i < 20; i+=4)
			// {
			//  	StringHash = HashCombine(StringHash, *reinterpret_cast<uint32*>(&HashBytes[i]));
			// }

			uint32 StringHash = GetTypeHash(Tag.ToString());
			float Hue = (double(StringHash) / MAX_uint32) * 360.f;
			FLinearColor ColorTintRGB = FLinearColor(Hue, GSequencerTagSaturation, .5f).HSVToLinearRGB();

			ExposedNameColors.Add(Tag, ColorTintRGB);
		}

		for (FMovieSceneObjectBindingID ID : Pair.Value.IDs)
		{
			ExposedNameReverseLUT.Add(ID.ReinterpretAsFixed(), Pair.Key);
		}
	}

	OnUpdatedEvent.Broadcast(this);
}

