// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Layout/SlateRect.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Misc/TextFilterExpressionEvaluator.h"

class SComponentChooserPopup : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnComponentChosen, FName );

	SLATE_BEGIN_ARGS( SComponentChooserPopup )
		: _Actor(NULL)
		{}

		/** An actor with components */
		SLATE_ARGUMENT( AActor*, Actor )

		/** Called when the text is chosen. */
		SLATE_EVENT( FOnComponentChosen, OnComponentChosen )

	SLATE_END_ARGS()

	/** Delegate to call when component is selected */
	FOnComponentChosen OnComponentChosen;

	/** List of tag names selected in the tag containers*/
	TArray< TSharedPtr<FName> > ComponentNames;

private:
	TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew( STableRow< TSharedPtr<FName> >, OwnerTable )
				[
					SNew(STextBlock) .Text( FText::FromName(*InItem.Get()) )
				];
	}

	void OnComponentSelected(TSharedPtr<FName> InItem, ESelectInfo::Type InSelectInfo)
	{
		FSlateApplication::Get().DismissAllMenus();

		if(OnComponentChosen.IsBound())
		{
			OnComponentChosen.Execute(*InItem.Get());
		}
	}

public:
	void Construct( const FArguments& InArgs )
	{
		OnComponentChosen = InArgs._OnComponentChosen;
		AActor* Actor = InArgs._Actor;

		TInlineComponentArray<USceneComponent*> Components(Actor);

		ComponentNames.Empty();
		for(USceneComponent* Component : Components)
		{
			if (Component->HasAnySockets())
			{
				ComponentNames.Add(MakeShareable(new FName(Component->GetFName())));
			}
		}

		// Then make widget
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
			.Padding(5)
			.Content()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 1.0f)
				[
					SNew(STextBlock)
					.Font( FAppStyle::GetFontStyle(TEXT("SocketChooser.TitleFont")) )
					.Text( NSLOCTEXT("ComponentChooser", "ChooseComponentLabel", "Choose Component") )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(512)
				[
					SNew(SBox)
					.WidthOverride(256)
					.Content()
					[
						SNew(SListView< TSharedPtr<FName> >)
						.ListItemsSource( &ComponentNames)
						.OnGenerateRow( this, &SComponentChooserPopup::MakeListViewWidget )
						.OnSelectionChanged( this, &SComponentChooserPopup::OnComponentSelected )
					]
				]
			]
		];
	}
};