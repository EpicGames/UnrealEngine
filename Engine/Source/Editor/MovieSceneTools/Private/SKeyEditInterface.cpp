// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKeyEditInterface.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "UObject/StructOnScope.h"
#include "MovieSceneSection.h"
#include "ISequencer.h"
#include "Editor.h"
#include "MovieSceneKeyStruct.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "FrameNumberDetailsCustomization.h"
#include "MovieSceneSectionDetailsCustomization.h"
#include "MovieSceneEventCustomization.h"
#include "Misc/FrameNumber.h"
#include "Modules/ModuleManager.h"
#include "IDetailCustomization.h"
#include "IStructureDataProvider.h"
#include "MovieSceneToolsModule.h"

SKeyEditInterface::~SKeyEditInterface()
{
	GEditor->UnregisterForUndo(this);
}

void SKeyEditInterface::Construct(const FArguments& InArgs, TSharedRef<ISequencer> InSequencer)
{
	GEditor->RegisterForUndo(this);

	EditDataAttribute = InArgs._EditData;

	WeakSequencer = InSequencer;
	Initialize();
}

void SKeyEditInterface::Initialize()
{
	// Reset the section and widget content
	FKeyEditData NewEditData = EditDataAttribute.Get();
	WeakSection = NewEditData.OwningSection;
	WeakOwningObject = NewEditData.OwningObject;

	ChildSlot
	[
		SNullWidget::NullWidget
	];

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid() || !NewEditData.KeyStruct.IsValid())
	{
		return;
	}

	// Set up the details panel
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bCustomFilterAreaLocation = true;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
		DetailsViewArgs.NotifyHook = this;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = false;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = false;
	}

	StructureDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor")
		.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	// register details customizations for this instance
	StructureDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("MovieSceneObjectBindingID", FOnGetPropertyTypeCustomizationInstance::CreateSP(this, &SKeyEditInterface::CreateBindingIDCustomization));
	StructureDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("MovieSceneEvent", FOnGetPropertyTypeCustomizationInstance::CreateSP(this, &SKeyEditInterface::CreateEventCustomization));
	StructureDetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber", FOnGetPropertyTypeCustomizationInstance::CreateSP(this, &SKeyEditInterface::CreateFrameNumberCustomization));

	FMovieSceneToolsModule::Get().CustomizeKeyStructInstancedPropertyTypes(StructureDetailsView.ToSharedRef(), WeakSection);

	StructureDetailsView->SetStructureData(NewEditData.KeyStruct);
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SKeyEditInterface::OnFinishedChangingProperties, NewEditData.KeyStruct);

	ChildSlot
	[
		StructureDetailsView->GetWidget().ToSharedRef()
	];
}

FVector2D SKeyEditInterface::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D ChildSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	return FVector2D::Max(ChildSize, FVector2D(350.f, 0.f));
}

TSharedRef<IPropertyTypeCustomization> SKeyEditInterface::CreateBindingIDCustomization()
{
	if (const TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
	{
		FMovieSceneSequenceID SequenceID = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedTemplateID() : MovieSceneSequenceID::Root;
		return MakeShared<FMovieSceneObjectBindingIDCustomization>(SequenceID, WeakSequencer);
	}
	return MakeShared<FMovieSceneObjectBindingIDCustomization>(FMovieSceneSequenceID(), WeakSequencer);
}

TSharedRef<IPropertyTypeCustomization> SKeyEditInterface::CreateFrameNumberCustomization()
{
	return WeakSequencer.Pin()->MakeFrameNumberDetailsCustomization();
}

TSharedRef<IPropertyTypeCustomization> SKeyEditInterface::CreateEventCustomization()
{
	return FMovieSceneEventCustomization::MakeInstance(WeakSection.Get());
}

void SKeyEditInterface::OnFinishedChangingProperties(const FPropertyChangedEvent& ChangeEvent, TSharedPtr<FStructOnScope> KeyStruct)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}

void SKeyEditInterface::ModifySectionAndOwner()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->Modify();
	}
	// When the channel lives on a nested object (e.g. a decoration), Modify it too so its
	// signature regenerates and the transaction captures its state.
	UObject* OwningObject = WeakOwningObject.Get();
	if (OwningObject && OwningObject != static_cast<UObject*>(WeakSection.Get()))
	{
		OwningObject->Modify();
	}
}

void SKeyEditInterface::NotifyPreChange( FProperty* PropertyAboutToChange )
{
	ModifySectionAndOwner();
}

void SKeyEditInterface::NotifyPostChange(const FPropertyChangedEvent& ChangeEvent, class FEditPropertyChain* PropertyThatChanged)
{
	TSharedPtr<const class IStructureDataProvider> StructureProvider = StructureDetailsView->GetStructureProvider();
	if (!StructureProvider)
	{
		return;
	}

	TArray<TSharedPtr<FStructOnScope>> KeyStructInstances;
	StructureProvider->GetInstances(KeyStructInstances, FMovieSceneKeyStruct::StaticStruct());

	for (const TSharedPtr<FStructOnScope>& KeyStruct : KeyStructInstances)
	{
		if (KeyStruct.IsValid() && KeyStruct->IsValid())
		{
			if (KeyStruct->GetStruct()->IsChildOf(FMovieSceneKeyStruct::StaticStruct()))
			{
				ModifySectionAndOwner();

				((FMovieSceneKeyStruct*)KeyStruct->GetStructMemory())->PropagateChanges(ChangeEvent);
			}
		}
	}

	TArray<TSharedPtr<FStructOnScope>> GeneratedKeyStructInstances;
	StructureProvider->GetInstances(GeneratedKeyStructInstances, FGeneratedMovieSceneKeyStruct::StaticStruct());

	for (const TSharedPtr<FStructOnScope>& KeyStruct : GeneratedKeyStructInstances)
	{
		if (KeyStruct.IsValid() && KeyStruct->IsValid())
		{
			ModifySectionAndOwner();

			FGeneratedMovieSceneKeyStruct* GeneratedStruct = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(KeyStruct->GetStructMemory());
			if (GeneratedStruct->OnPropertyChangedEvent)
			{
				GeneratedStruct->OnPropertyChangedEvent(ChangeEvent);
			}
		}
	}
}

void SKeyEditInterface::PostUndo(bool bSuccess)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	if (Sequencer.IsValid())
	{
		Initialize();
	}
}

void SKeyEditInterface::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}
