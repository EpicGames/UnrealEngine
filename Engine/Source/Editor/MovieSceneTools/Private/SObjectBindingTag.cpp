// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectBindingTag.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EditorFontGlyphs.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "MovieSceneObjectBindingID.h"
#include "ObjectBindingTagCache.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/TypeHash.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SObjectBindingTag"

void SObjectBindingTags::Construct(const FArguments& InArgs, const UE::MovieScene::FFixedObjectBindingID& InBindingID, FObjectBindingTagCache* BindingCache)
{
	BindingID = InBindingID;
	OnTagDeletedEvent = InArgs._OnTagDeleted;

	BindingCache->OnUpdatedEvent.AddSP(this, &SObjectBindingTags::OnBindingCacheUpdated);
	OnBindingCacheUpdated(BindingCache);
}

void SObjectBindingTags::OnBindingCacheUpdated(const FObjectBindingTagCache* BindingCache)
{
	TSharedPtr<SHorizontalBox> Box;
	float LeftPadding = 0.f;
	for (auto It = BindingCache->IterateTags(BindingID); It; ++It)
	{
		FName TagName = It.Value();

		if (!Box)
		{
			Box = SNew(SHorizontalBox);
		}

		FSimpleDelegate OnDeleteDelegate;
		if (OnTagDeletedEvent.IsBound())
		{
			OnDeleteDelegate = FSimpleDelegate::CreateSP(this, &SObjectBindingTags::OnTagDeleted, TagName);
		}

		Box->AddSlot()
		.Padding(FMargin(LeftPadding, 0.f, 0.f, 0.f))
		.AutoWidth()
		[
			SNew(SObjectBindingTag)
			.OnDeleted(OnDeleteDelegate)
			.ColorTint(BindingCache->GetTagColor(TagName))
			.Text(FText::FromName(TagName))
			.ToolTipText(FText::Format(LOCTEXT("TagToolTip", "This object binding can be looked up, resolved, and overridden in the root sequence by using the tag '{0}'"), FText::FromName(TagName)))
		];

		LeftPadding = 5.f;
	}

	ChildSlot
	[
		Box.IsValid()
		? Box.ToSharedRef()
		: SNullWidget::NullWidget
	];
}

void SObjectBindingTags::OnTagDeleted(FName TagName)
{
	OnTagDeletedEvent.Execute(TagName);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SObjectBindingTag::Construct(const FArguments& InArgs)
{
	OnDeleted   = InArgs._OnDeleted;
	OnClicked   = InArgs._OnClicked;
	OnCreateNew = InArgs._OnCreateNew;

	TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

	if (InArgs._OnCreateNew.IsBound())
	{
		ContentBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(EditableText, SEditableTextBox)
			.Font(FAppStyle::GetFontStyle("TinyText"))
			.OnTextCommitted(this, &SObjectBindingTag::OnNewTextCommitted)
			.HintText(LOCTEXT("AddNew_Hint", "Enter New Name"))
		];
	}
	else
	{
		ContentBox->AddSlot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("TinyText"))
			.Text(InArgs._Text)
		];
	}

	// Create button
	if (OnCreateNew.IsBound())
	{
		ContentBox->AddSlot()
		.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ContentPadding(FMargin(0.f))
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SObjectBindingTag::HandleCreateButtonClicked)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Plus)
			]
		];
	}
	// Delete button
	if (InArgs._OnDeleted.IsBound())
	{
		ContentBox->AddSlot()
		.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ContentPadding(FMargin(0.f))
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SObjectBindingTag::HandleDeleteButtonClicked)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Times)
			]
		];
	}

	if (OnClicked.IsBound())
	{
		// If this is clickable, we need to be a button
		ChildSlot
		[
			SNew(SButton)
			.ToolTipText(InArgs._ToolTipText)
			.ButtonStyle(FAppStyle::Get(), "Sequencer.ExposedNamePill")
			.ButtonColorAndOpacity(InArgs._ColorTint)
			.ContentPadding(FMargin(8.f, 2.f))
			.OnClicked(this, &SObjectBindingTag::HandlePillClicked)
			[
				ContentBox
			]
		];
	}
	else if (OnCreateNew.IsBound())
	{
		ChildSlot
		[
			ContentBox
		];
	}
	else
	{
		// else it's just a border
		ChildSlot
		[
			SNew(SBorder)
			.ToolTipText(InArgs._ToolTipText)
			.BorderImage(FAppStyle::GetBrush("Sequencer.ExposedNamePill_BG"))
			.BorderBackgroundColor(InArgs._ColorTint)
			.Padding(FMargin(8.f, 2.f))
			[
				ContentBox
			]
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SObjectBindingTag::OnNewTextCommitted(const FText& InNewText, ETextCommit::Type CommitType)
{
	if (!InNewText.IsEmpty() && CommitType == ETextCommit::OnEnter)
	{
		FString NewName = InNewText.ToString();
		FText FailureReason, Context = LOCTEXT("TagName", "Object binding tags");
		if (!FName::IsValidXName(NewName, INVALID_NAME_CHARACTERS, &FailureReason, &Context))
		{
			FNotificationInfo NotificationInfo(FailureReason);
			NotificationInfo.ExpireDuration = 4.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			return;
		}

		OnCreateNew.Execute(FName(*NewName));

		EditableText->SetText(FText());
	}
}

FReply SObjectBindingTag::HandleCreateButtonClicked()
{
	OnNewTextCommitted(EditableText->GetText(), ETextCommit::OnEnter);
	return FReply::Handled();
}

FReply SObjectBindingTag::HandleDeleteButtonClicked()
{
	OnDeleted.Execute();
	return FReply::Handled();
}

FReply SObjectBindingTag::HandlePillClicked()
{
	OnClicked.Execute();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE