// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/BindingLifetimeSection.h"
#include "SequencerSectionPainter.h"

#define LOCTEXT_NAMESPACE "BindingLifetimeSection"

bool FBindingLifetimeSection::IsSectionSelected() const
{
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();

	TArray<UMovieSceneTrack*> SelectedTracks;
	SequencerPtr->GetSelectedTracks(SelectedTracks);

	UMovieSceneSection* Section = WeakSection.Get();
	UMovieSceneTrack* Track = Section ? CastChecked<UMovieSceneTrack>(Section->GetOuter()) : nullptr;
	return Track && SelectedTracks.Contains(Track);
}

int32 FBindingLifetimeSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();
	return LayerId;
}

#undef LOCTEXT_NAMESPACE // "BindingLifetimeSection"