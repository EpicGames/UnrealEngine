// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "ISequencer.h"

class FSequencerSectionPainter;

class FBindingLifetimeSection
	: public FSequencerSection
{
public:

	FBindingLifetimeSection(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSectionObject)
		, Sequencer(InSequencer)
	{}

protected:

	bool IsSectionSelected() const; 
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

protected:

	TWeakPtr<ISequencer> Sequencer;
};
