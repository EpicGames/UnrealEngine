// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/BoolPropertySection.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MovieSceneSection.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Sections/MovieSceneBoolSection.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "TimeToPixel.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FSlateBrush;


int32 FBoolPropertySection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	// custom drawing for bool curves
	UMovieSceneBoolSection* BoolSection = Cast<UMovieSceneBoolSection>( WeakSection.Get() );

	TArray<FFrameTime> SectionSwitchTimes;

	const FTimeToPixel& TimeConverter = Painter.GetTimeConverter();

	// Add the start time
	const FFrameNumber StartTime = TimeConverter.PixelToFrame(0.f).FloorToFrame();
	const FFrameNumber EndTime   = TimeConverter.PixelToFrame(Painter.SectionGeometry.GetLocalSize().X).CeilToFrame();
	
	SectionSwitchTimes.Add(StartTime);

	int32 LayerId = Painter.PaintSectionBackground();

	FMovieSceneBoolChannel* BoolChannel = BoolSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0);
	if (!BoolChannel)
	{
		return LayerId;
	}

	for ( FFrameNumber Time : BoolChannel->GetTimes() )
	{
		if ( Time > StartTime && Time < EndTime )
		{
			SectionSwitchTimes.Add( Time );
		}
	}

	SectionSwitchTimes.Add(EndTime);

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled
		? ESlateDrawEffect::None
		: ESlateDrawEffect::DisabledEffect;

	static const int32 Height = 5;
	const float VerticalOffset = Painter.SectionGeometry.GetLocalSize().Y * .5f - Height * .5f;

	const FSlateBrush* BoolOverlayBrush = FAppStyle::GetBrush("Sequencer.Section.StripeOverlay");

	for ( int32 i = 0; i < SectionSwitchTimes.Num() - 1; ++i )
	{
		FFrameTime ThisTime = SectionSwitchTimes[i];

		bool ValueAtTime = false;
		BoolChannel->Evaluate(ThisTime, ValueAtTime);

		const FColor Color = ValueAtTime ? FColor(0, 255, 0, 125) : FColor(255, 0, 0, 125);
		
		FVector2D StartPos(TimeConverter.FrameToPixel(ThisTime), VerticalOffset);
		FVector2D Size(TimeConverter.FrameToPixel(SectionSwitchTimes[i+1]) - StartPos.X, Height);

		FSlateDrawElement::MakeBox(
			Painter.DrawElements,
			LayerId + 1,
			Painter.SectionGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(StartPos)),
			BoolOverlayBrush,
			DrawEffects,
			Color
			);
	}

	return LayerId + 1;
}
