// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/CameraCutSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Editor.h"
#include "MovieScene.h"
#include "SequencerSectionPainter.h"
#include "ScopedTransaction.h"
#include "MovieSceneSequence.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Styling/AppStyle.h"
#include "EngineUtils.h"
#include "Camera/CameraComponent.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"


#define LOCTEXT_NAMESPACE "FCameraCutSection"


/* FCameraCutSection structors
 *****************************************************************************/

FCameraCutSection::FCameraCutSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, UMovieSceneSection& InSection) : FViewportThumbnailSection(InSequencer, InThumbnailPool, InSection)
{
	AdditionalDrawEffect = ESlateDrawEffect::NoGamma;
}

FCameraCutSection::~FCameraCutSection()
{
}


/* ISequencerSection interface
 *****************************************************************************/

void FCameraCutSection::SetSingleTime(double GlobalTime)
{
	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);
	if (CameraCutSection && CameraCutSection->HasStartFrame())
	{
		double ReferenceOffsetSeconds = CameraCutSection->GetInclusiveStartFrame() / CameraCutSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		CameraCutSection->SetThumbnailReferenceOffset(GlobalTime - ReferenceOffsetSeconds);
	}
}

void FCameraCutSection::Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);
	if (CameraCutSection)
	{
		if (GetDefault<UMovieSceneUserThumbnailSettings>()->bDrawSingleThumbnails && CameraCutSection->HasStartFrame())
		{
			double ReferenceOffsetSeconds = CameraCutSection->GetInclusiveStartFrame() / CameraCutSection->GetTypedOuter<UMovieScene>()->GetTickResolution() + CameraCutSection->GetThumbnailReferenceOffset();
			ThumbnailCache.SetSingleReferenceFrame(ReferenceOffsetSeconds);
		}
		else
		{
			ThumbnailCache.SetSingleReferenceFrame(TOptional<double>());
		}
	}

	FViewportThumbnailSection::Tick(AllottedGeometry, ClippedGeometry, InCurrentTime, InDeltaTime);
}

void FCameraCutSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	FViewportThumbnailSection::BuildSectionContextMenu(MenuBuilder, ObjectBinding);

	UWorld* World = GEditor->GetEditorWorldContext().World();

	if (World == nullptr || !Section->HasStartFrame())
	{
		return;
	}

	AActor* CameraActor = GetCameraForFrame(Section->GetInclusiveStartFrame());

	if (CameraActor)
	{
		MenuBuilder.AddMenuSeparator();
		
		// GetCameraForFrame will return the Spawnable Template (for names) but we can't select those.
		const bool bCanSelect = CameraActor->GetWorld() != nullptr;
		const FText CameraNameLabel = FText::FromString(CameraActor->GetActorLabel());
		const FText Tooltip = bCanSelect ?
			FText::Format(LOCTEXT("SelectCameraTooltipFormat", "Select {0}"), CameraNameLabel) :
			FText::Format(LOCTEXT("SelectCameraInvalidTooltipFormat", "Cannot Select {0} (Currently Unspawned)"), CameraNameLabel);

		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("SelectCameraTextFormat", "Select {0}"), CameraNameLabel),
			Tooltip,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &FCameraCutSection::HandleSelectCameraMenuEntryExecute, CameraActor),
				FCanExecuteAction::CreateRaw(this, & FCameraCutSection::CanSelectCameraActor, CameraActor))
		);
	}

	// get list of available cameras
	TArray<AActor*> AllCameras;

	for (FActorIterator ActorIt(World); ActorIt; ++ActorIt)
	{
		AActor* Actor = *ActorIt;

		if ((Actor != CameraActor) && Actor->IsListedInSceneOutliner())
		{
			UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(Actor);
			if (CameraComponent)
			{
				AllCameras.Add(Actor);
			}
		}
	}

	if (AllCameras.Num() == 0)
	{
		return;
	}

	AllCameras.Sort([](const AActor& A, const AActor& B) { return A.GetActorLabel().Compare(B.GetActorLabel()) < 0; });

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ChangeCameraMenuText", "Change Camera"));
	{
		for (auto EachCamera : AllCameras)
		{
			FText ActorLabel = FText::FromString(EachCamera->GetActorLabel());

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("SetCameraMenuEntryTextFormat", "{0}"), ActorLabel),
				FText::Format(LOCTEXT("SetCameraMenuEntryTooltipFormat", "Assign {0} to this camera cut"), FText::FromString(EachCamera->GetPathName())),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateRaw(this, &FCameraCutSection::HandleSetCameraMenuEntryExecute, EachCamera))
			);
		}
	}
	MenuBuilder.EndSection();
}


/* FThumbnailSection interface
 *****************************************************************************/

AActor* FCameraCutSection::GetCameraForFrame(FFrameNumber Time) const
{
	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);
	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();

	if (CameraCutSection && Sequencer.IsValid())
	{
		UCameraComponent* CameraComponent = CameraCutSection->GetFirstCamera(*Sequencer, Sequencer->GetFocusedTemplateID());
		if (CameraComponent)
		{
			return CameraComponent->GetOwner();
		}

		if (AActor* ActorTemplate = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(Sequencer->GetFocusedMovieSceneSequence(), CameraCutSection->GetCameraBindingID().GetGuid(), Sequencer->GetSharedPlaybackState())))
		{
			return ActorTemplate;
		}
	}

	return nullptr;
}

FText FCameraCutSection::GetSectionTitle() const
{
	return HandleThumbnailTextBlockText();
}

float FCameraCutSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return FViewportThumbnailSection::GetSectionHeight(ViewDensity) + 10.f;
}

FMargin FCameraCutSection::GetContentPadding() const
{
	return FMargin(6.f, 10.f);
}

int32 FCameraCutSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	InPainter.LayerId = InPainter.PaintSectionBackground();

	// Draw a red frame around the edges to indicate an error since we can't highlight the error text right now.
	AActor* CameraActor = GetCameraForFrame(Section->GetInclusiveStartFrame());
	if (!CameraActor)
	{
		static const FSlateBrush* ErroredSectionOverlay = FAppStyle::Get().GetBrush("Sequencer.Section.ErroredSectionOverlay");
		const ESlateDrawEffect DrawEffects = InPainter.bParentEnabled
			? ESlateDrawEffect::None
			: ESlateDrawEffect::DisabledEffect;

		FLinearColor ErrorColor = FLinearColor::Red;
		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId,
			InPainter.SectionGeometry.ToPaintGeometry(InPainter.SectionGeometry.GetLocalSize() - FVector2D(1.f, 1.f), FSlateLayoutTransform(FVector2D(1.f, 1.f))),
			ErroredSectionOverlay,
			DrawEffects,
			ErrorColor.CopyWithNewOpacity(0.8f)
		);
		InPainter.LayerId++;
	}

	return FViewportThumbnailSection::OnPaintSection(InPainter);
}

FText FCameraCutSection::HandleThumbnailTextBlockText() const
{
	const AActor* CameraActor = Section->HasStartFrame() ? GetCameraForFrame(Section->GetInclusiveStartFrame()) : nullptr;
	if (CameraActor)
	{
		return FText::FromString(CameraActor->GetActorLabel());
	}

	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);
	if (CameraCutSection)
	{
		if(!CameraCutSection->GetCameraBindingID().IsValid())
		{
			return LOCTEXT("CameraBindingError_NoBinding", "No Object Binding specified.");
		}
		else
		{
			return LOCTEXT("CameraBindingError_MissingBinding", "Object Binding / Bound Object is missing!");
		}
	}

	return FText::GetEmpty();
}


/* FCameraCutSection callbacks
 *****************************************************************************/

bool FCameraCutSection::CanSelectCameraActor(AActor* InCamera) const
{
	return InCamera && InCamera->GetWorld();
}

void FCameraCutSection::HandleSelectCameraMenuEntryExecute(AActor* InCamera)
{
	const bool bInSelected = true;
	const bool bNotify = true;
	const bool bSelectEventIfHidden = true;
	GEditor->SelectActor(InCamera, bInSelected, bNotify, bSelectEventIfHidden);
}

void FCameraCutSection::HandleSetCameraMenuEntryExecute(AActor* InCamera)
{
	auto Sequencer = SequencerPtr.Pin();

	if (Sequencer.IsValid())
	{
		FGuid ObjectGuid = Sequencer->GetHandleToObject(InCamera, true);

		UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);

		CameraCutSection->SetFlags(RF_Transactional);

		const FScopedTransaction Transaction(LOCTEXT("SetCameraCut", "Set Camera Cut"));

		CameraCutSection->Modify();
	
		CameraCutSection->SetCameraGuid(ObjectGuid);
	
		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}

UCameraComponent* FCameraCutSection::GetViewCamera()
{
	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section);
	TSharedPtr<ISequencer>       Sequencer        = SequencerPtr.Pin();

	if (CameraCutSection && Sequencer.IsValid())
	{
		return CameraCutSection->GetFirstCamera(*Sequencer, Sequencer->GetFocusedTemplateID());
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
