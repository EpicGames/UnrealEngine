// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Sections/ThumbnailSection.h"

class AActor;
class FMenuBuilder;
class FSequencerSectionPainter;
class FTrackEditorThumbnailPool;

/**
 * CameraCut section, which paints and ticks the appropriate section.
 */
class FCameraCutSection
	: public FViewportThumbnailSection
{
public:

	/** Create and initialize a new instance. */
	FCameraCutSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, UMovieSceneSection& InSection);

	/** Virtual destructor. */
	virtual ~FCameraCutSection();

public:

	// ISequencerSection interface
	virtual void Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	virtual FMargin GetContentPadding() const override;
	// FThumbnail interface

	virtual void SetSingleTime(double GlobalTime) override;
	virtual FText HandleThumbnailTextBlockText() const override;
	virtual UCameraComponent* GetViewCamera() override;

private:

	/** Get a representative camera for the given time */
	AActor* GetCameraForFrame(FFrameNumber Time) const;

	/** Callback for executing a "Select Camera" menu entry in the context menu. */
	void HandleSelectCameraMenuEntryExecute(AActor* InCamera);

	/** Callback for determining if the specified camera can actually be selected in the World. */
	bool CanSelectCameraActor(AActor* InCamera) const;


	/** Callback for executing a "Set Camera" menu entry in the context menu. */
	void HandleSetCameraMenuEntryExecute(AActor* InCamera);
};
