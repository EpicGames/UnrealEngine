// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/CinematicShotSection.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Rendering/DrawElements.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "MovieSceneTrack.h"
#include "MovieScene.h"
#include "TrackEditors/CinematicShotTrackEditor.h"
#include "SequencerSectionPainter.h"
#include "Misc/AxisDisplayInfo.h"
#include "Styling/AppStyle.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "FCinematicShotSection"


/* FCinematicShotSection structors
 *****************************************************************************/

FCinematicShotSection::FCinematicSectionCache::FCinematicSectionCache(UMovieSceneCinematicShotSection* Section)
	: InnerFrameRate(1, 1)
	, InnerFrameOffset(0)
	, SectionStartFrame(0)
{
	if (Section)
	{
		UMovieSceneSequence* InnerSequence = Section->GetSequence();
		if (InnerSequence&& InnerSequence->GetMovieScene())
		{
			InnerFrameRate = InnerSequence->GetMovieScene()->GetTickResolution();
		}

		InnerFrameOffset = Section->Parameters.StartFrameOffset;
		SectionStartFrame = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : 0;
		TimeScale = Section->Parameters.TimeScale;
	}
}


FCinematicShotSection::FCinematicShotSection(TSharedPtr<ISequencer> InSequencer, UMovieSceneCinematicShotSection& InSection, TSharedPtr<FCinematicShotTrackEditor> InCinematicShotTrackEditor, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool)
	: TSubSectionMixin(InSequencer, InSection, InSequencer, InThumbnailPool, InSection)
	, CinematicShotTrackEditor(InCinematicShotTrackEditor)
	, ThumbnailCacheData(&InSection)
{
	AdditionalDrawEffect = ESlateDrawEffect::NoGamma;
}


FCinematicShotSection::~FCinematicShotSection()
{
}

FText FCinematicShotSection::GetSectionTitle() const
{
	return GetRenameVisibility() == EVisibility::Visible ? FText::GetEmpty() : HandleThumbnailTextBlockText();
}

float FCinematicShotSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return FViewportThumbnailSection::GetSectionHeight(ViewDensity) + 2*13.f;
}

FMargin FCinematicShotSection::GetContentPadding() const
{
	// When the rename widget is visible, use less padding so that the widget is visible over the film border (when thumbnails are not shown)
	return FMargin(8.f, GetRenameVisibility() == EVisibility::Visible ? 10.f : 15.f);
}

void FCinematicShotSection::SetSingleTime(double GlobalTime)
{
	UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();
	double ReferenceOffsetSeconds = SectionObject.HasStartFrame() ? SectionObject.GetInclusiveStartFrame() / SectionObject.GetTypedOuter<UMovieScene>()->GetTickResolution() : 0;
	SectionObject.SetThumbnailReferenceOffset(GlobalTime - ReferenceOffsetSeconds);
}

UCameraComponent* FindCameraCutComponentRecursive(FFrameNumber GlobalTime, FMovieSceneSequenceID InnerSequenceID, const FMovieSceneSequenceHierarchy& Hierarchy, IMovieScenePlayer& Player)
{
	const FMovieSceneSequenceHierarchyNode* Node    = Hierarchy.FindNode(InnerSequenceID);
	const FMovieSceneSubSequenceData*       SubData = Hierarchy.FindSubData(InnerSequenceID);
	if (!ensure(SubData && Node))
	{
		return nullptr;
	}

	UMovieSceneSequence* InnerSequence   = SubData->GetSequence();
	UMovieScene*         InnerMovieScene = InnerSequence ? InnerSequence->GetMovieScene() : nullptr;
	if (!InnerMovieScene)
	{
		return nullptr;
	}

	FFrameNumber InnerTime = (GlobalTime * SubData->RootToSequenceTransform).FloorToFrame();
	if (!SubData->PlayRange.Value.Contains(InnerTime))
	{
		return nullptr;
	}

	int32 LowestRow      = TNumericLimits<int32>::Max();
	int32 HighestOverlap = 0;

	UMovieSceneCameraCutSection* ActiveSection = nullptr;

	if (UMovieSceneCameraCutTrack* CutTrack = Cast<UMovieSceneCameraCutTrack>(InnerMovieScene->GetCameraCutTrack()))
	{
		for (UMovieSceneSection* ItSection : CutTrack->GetAllSections())
		{
			UMovieSceneCameraCutSection* CutSection = Cast<UMovieSceneCameraCutSection>(ItSection);
			if (CutSection && CutSection->GetRange().Contains(InnerTime))
			{
				bool bSectionWins = 
					( CutSection->GetRowIndex() < LowestRow ) ||
					( CutSection->GetRowIndex() == LowestRow && CutSection->GetOverlapPriority() > HighestOverlap );

				if (bSectionWins)
				{
					HighestOverlap = CutSection->GetOverlapPriority();
					LowestRow      = CutSection->GetRowIndex();
					ActiveSection  = CutSection;
				}
			}
		}
	}
	
	if (ActiveSection)
	{
		return ActiveSection->GetFirstCamera(Player, InnerSequenceID);
	}

	for (FMovieSceneSequenceID Child : Node->Children)
	{
		UCameraComponent* CameraComponent = FindCameraCutComponentRecursive(GlobalTime, Child, Hierarchy, Player);
		if (CameraComponent)
		{
			return CameraComponent;
		}
	}

	return nullptr;
}

UCameraComponent* FCinematicShotSection::GetViewCamera()
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}


	const UMovieSceneCinematicShotSection&	SectionObject    = GetSectionObjectAs<UMovieSceneCinematicShotSection>();
	const FMovieSceneSequenceID             ThisSequenceID   = Sequencer->GetFocusedTemplateID();
	const FMovieSceneSequenceID             TargetSequenceID = SectionObject.GetSequenceID();
	const FMovieSceneSequenceHierarchy*     Hierarchy        = Sequencer->GetEvaluationTemplate().GetCompiledDataManager()->FindHierarchy(Sequencer->GetEvaluationTemplate().GetCompiledDataID());

	if (!Hierarchy)
	{
		return nullptr;
	}

	const FMovieSceneSequenceHierarchyNode* ThisSequenceNode = Hierarchy->FindNode(ThisSequenceID);
	
	check(ThisSequenceNode);
	
	// Find the TargetSequenceID by comparing deterministic sequence IDs for all children of the current node
	const FMovieSceneSequenceID* InnerSequenceID = Algo::FindByPredicate(ThisSequenceNode->Children,
		[Hierarchy, TargetSequenceID](FMovieSceneSequenceID InSequenceID)
		{
			const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(InSequenceID);
			return SubData && SubData->DeterministicSequenceID == TargetSequenceID;
		}
	);
	
	if (InnerSequenceID)
	{
		UCameraComponent* CameraComponent = FindCameraCutComponentRecursive(Sequencer->GetGlobalTime().Time.FrameNumber, *InnerSequenceID, *Hierarchy, *Sequencer);
		if (CameraComponent)
		{
			return CameraComponent;
		}
	}

	return nullptr;
}

bool FCinematicShotSection::IsReadOnly() const
{
	// Overridden to false regardless of movie scene section read only state so that we can double click into the sub section
	return false;
}

void FCinematicShotSection::Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Set cached data
	UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();
	FCinematicSectionCache NewCacheData(&SectionObject);
	if (NewCacheData != ThumbnailCacheData)
	{
		ThumbnailCache.ForceRedraw();
	}
	ThumbnailCacheData = NewCacheData;

	// Update single reference frame settings
	if (GetDefault<UMovieSceneUserThumbnailSettings>()->bDrawSingleThumbnails && SectionObject.HasStartFrame())
	{
		double ReferenceTime = SectionObject.GetInclusiveStartFrame() / SectionObject.GetTypedOuter<UMovieScene>()->GetTickResolution() + SectionObject.GetThumbnailReferenceOffset();
		ThumbnailCache.SetSingleReferenceFrame(ReferenceTime);
	}
	else
	{
		ThumbnailCache.SetSingleReferenceFrame(TOptional<double>());
	}

	FViewportThumbnailSection::Tick(AllottedGeometry, ClippedGeometry, InCurrentTime, InDeltaTime);
}

int32 FCinematicShotSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	static const FSlateBrush* FilmBorder = FAppStyle::GetBrush("Sequencer.Section.FilmBorder");

	InPainter.LayerId = InPainter.PaintSectionBackground();

	FVector2D LocalHeaderSize = InPainter.HeaderGeometry.GetLocalSize();
	const UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();

	// Paint fancy-looking film border.
	FSlateDrawElement::MakeBox(
		InPainter.DrawElements,
		InPainter.LayerId++,
		InPainter.SectionGeometry.ToPaintGeometry(FVector2D(LocalHeaderSize.X-2.f, 7.f), FSlateLayoutTransform(FVector2D(1.f, 4.f))),
		FilmBorder,
		InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
	);

	FSlateDrawElement::MakeBox(
		InPainter.DrawElements,
		InPainter.LayerId++,
		InPainter.SectionGeometry.ToPaintGeometry(FVector2D(LocalHeaderSize.X-2.f, 7.f), FSlateLayoutTransform(FVector2D(1.f, LocalHeaderSize.Y - 11.f))),
		FilmBorder,
		InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
	);

	// Paint the thumbnails.
	FViewportThumbnailSection::OnPaintSection(InPainter);

	// Paint the sub-sequence information/looping boundaries/etc.

	FSubSectionPainterParams SubSectionPainterParams(GetContentPadding());
	SubSectionPainterParams.bShowTrackNum = false;

	FSubSectionPainterUtil::PaintSection(
			GetSequencer(), SectionObject, InPainter, SubSectionPainterParams);

	return InPainter.LayerId;
}

void FCinematicShotSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	FViewportThumbnailSection::BuildSectionContextMenu(MenuBuilder, ObjectBinding);

	AddShotMenuSection(MenuBuilder, ObjectBinding);
}

void FCinematicShotSection::BuildSectionSidebarMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	AddShotMenuSection(MenuBuilder, ObjectBinding);
}

void FCinematicShotSection::AddShotMenuSection(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ShotMenuText", "Shot"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("TakesMenu", "Takes"),
			LOCTEXT("TakesMenuTooltip", "Shot takes"),
			FNewMenuDelegate::CreateLambda([this, &SectionObject](FMenuBuilder& InMenuBuilder) { CinematicShotTrackEditor.Pin()->AddTakesMenu(&SectionObject, InMenuBuilder); }));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("NewTake", "New Take"),
			FText::Format(LOCTEXT("NewTakeTooltip", "Create a new take for {0}"), FText::FromString(SectionObject.GetShotDisplayName())),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(CinematicShotTrackEditor.Pin().ToSharedRef(), &FCinematicShotTrackEditor::CreateNewTake, Cast<UMovieSceneSubSection>(&SectionObject)))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("InsertNewShot", "Insert Shot"),
			LOCTEXT("InsertNewShotTooltip", "Insert a new shot at the current time"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(CinematicShotTrackEditor.Pin().ToSharedRef(), &FCinematicShotTrackEditor::InsertSection, Cast<UMovieSceneTrack>(SectionObject.GetOuter())))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DuplicateShot", "Duplicate Shot"),
			FText::Format(LOCTEXT("DuplicateShotTooltip", "Duplicate {0} to create a new shot"), FText::FromString(SectionObject.GetShotDisplayName())),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(CinematicShotTrackEditor.Pin().ToSharedRef(), &FCinematicShotTrackEditor::DuplicateSection, Cast<UMovieSceneSubSection>(&SectionObject)))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("EditMetaData", "Edit Meta Data"),
			LOCTEXT("EditMetaDataTooltip", "Edit meta data"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(CinematicShotTrackEditor.Pin().ToSharedRef(), &FCinematicShotTrackEditor::EditMetaData, Cast<UMovieSceneSubSection>(&SectionObject)))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenderShot", "Render Shot"),
			FText::Format(LOCTEXT("RenderShotTooltip", "Render shot movie"), FText::FromString(SectionObject.GetShotDisplayName())),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, &SectionObject]()
				{
					TArray<UMovieSceneCinematicShotSection*> ShotSections;
					TArray<UMovieSceneSection*> Sections;
					GetSequencer()->GetSelectedSections(Sections);
					for (UMovieSceneSection* Section : Sections)
					{
						if (UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section))
						{
							ShotSections.Add(ShotSection);
						}
					}

					if (!ShotSections.Contains(&SectionObject))
					{
						ShotSections.Add(&SectionObject);
					}

					CinematicShotTrackEditor.Pin()->RenderShots(ShotSections);
				}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameShot", "Rename Shot"),
			FText::Format(LOCTEXT("RenameShotTooltip", "Rename {0}"), FText::FromString(SectionObject.GetShotDisplayName())),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FCinematicShotSection::EnterRename))
		);
	}
	MenuBuilder.EndSection();

	auto MakeUIAction = [this](EMovieSceneTransformChannel ChannelsToToggle, const TSharedPtr<ISequencer>& Sequencer)
	{

		UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
		if(!SubSection)
		{
			return FUIAction();
		}
		return FUIAction(
			FExecuteAction::CreateLambda([SubSection, ChannelsToToggle, Sequencer]
				{
					FScopedTransaction Transaction(LOCTEXT("SetActiveChannelsTransaction", "Set Active Channels"));
					SubSection->Modify();
					EMovieSceneTransformChannel Channels = SubSection->GetMask().GetChannels();

					if (EnumHasAllFlags(Channels, ChannelsToToggle) || (Channels & ChannelsToToggle) == EMovieSceneTransformChannel::None)
					{
						SubSection->SetMask(SubSection->GetMask().GetChannels() ^ ChannelsToToggle);
					}
					else
					{
						SubSection->SetMask(SubSection->GetMask().GetChannels() | ChannelsToToggle);
					}
				
					Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([SubSection, ChannelsToToggle]
			{
				EMovieSceneTransformChannel Channels = SubSection->GetMask().GetChannels();
				if (EnumHasAllFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Checked;
				}
				else if (EnumHasAnyFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Undetermined;
				}
				return ECheckBoxState::Unchecked;
			})
		);
	};

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("OriginChannelsText", "Active Channels"));
	MenuBuilder.AddSubMenu(
		LOCTEXT("AllTranslation", "Translation"), LOCTEXT("AllTranslation_ToolTip", "Causes this section to affect the translation of the transform"),
		FNewMenuDelegate::CreateLambda([Sequencer, MakeUIAction](FMenuBuilder& SubMenuBuilder){

			const EAxisList::Type XAxis = EAxisList::Forward;
			const EAxisList::Type YAxis = EAxisList::Left;
			const EAxisList::Type ZAxis = EAxisList::Up;

			const int32 NumMenuItems = 3;
			TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
				[&SubMenuBuilder, MakeUIAction, Sequencer, XAxis]()
				{
					SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(XAxis),
						FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationX, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);

				},
				[&SubMenuBuilder, MakeUIAction, Sequencer, YAxis]()
				{
					SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(YAxis),
						FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationY, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
				},
				[&SubMenuBuilder, MakeUIAction, Sequencer, ZAxis]()
				{
					SubMenuBuilder.AddMenuEntry(
					AxisDisplayInfo::GetAxisDisplayName(ZAxis),
					FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationZ, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
				}
			};

			const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
			for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
			{
				const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
				MenuConstructors[SwizzledComponentIndex]();
			}
		}),
		MakeUIAction(EMovieSceneTransformChannel::Translation, Sequencer),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddSubMenu(
		LOCTEXT("AllRotation", "Rotation"), LOCTEXT("AllRotation_ToolTip", "Causes this section to affect the rotation of the transform"),
		FNewMenuDelegate::CreateLambda([Sequencer, MakeUIAction](FMenuBuilder& SubMenuBuilder){
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("RotationX", "Roll"), LOCTEXT("RotationX_ToolTip", "Causes this section to affect the roll channel the transform's rotation"),
				FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationX, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("RotationY", "Pitch"), LOCTEXT("RotationY_ToolTip", "Causes this section to affect the pitch channel the transform's rotation"),
				FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationY, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
			SubMenuBuilder.AddMenuEntry(
				LOCTEXT("RotationZ", "Yaw"), LOCTEXT("RotationZ_ToolTip", "Causes this section to affect the yaw channel the transform's rotation"),
				FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationZ, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
		}),
		MakeUIAction(EMovieSceneTransformChannel::Rotation, Sequencer),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);
	MenuBuilder.EndSection();
}

/* FCinematicShotSection callbacks
 *****************************************************************************/

FText FCinematicShotSection::HandleThumbnailTextBlockText() const
{
	const UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();
	return FText::FromString(SectionObject.GetShotDisplayName());
}


void FCinematicShotSection::HandleThumbnailTextBlockTextCommitted(const FText& NewShotName, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter && !HandleThumbnailTextBlockText().EqualTo(NewShotName))
	{
		UMovieSceneCinematicShotSection& SectionObject = GetSectionObjectAs<UMovieSceneCinematicShotSection>();

		SectionObject.Modify();

		const FScopedTransaction Transaction(LOCTEXT("SetShotName", "Set Shot Name"));
	
		SectionObject.SetShotDisplayName(NewShotName.ToString());
	}
}

#undef LOCTEXT_NAMESPACE
