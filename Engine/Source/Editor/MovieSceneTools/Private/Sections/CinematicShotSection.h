// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Sections/ThumbnailSection.h"
#include "TrackEditors/SubTrackEditorBase.h"

class FCinematicShotTrackEditor;
class FMenuBuilder;
class FSequencerSectionPainter;
class FTrackEditorThumbnailPool;
class UMovieSceneCinematicShotSection;

/**
 * CinematicShot section, which paints and ticks the appropriate section.
 */
class FCinematicShotSection
	: public TSubSectionMixin<FViewportThumbnailSection>
{
public:

	/** Create and initialize a new instance. */
	FCinematicShotSection(TSharedPtr<ISequencer> InSequencer, UMovieSceneCinematicShotSection& InSection, TSharedPtr<FCinematicShotTrackEditor> InCinematicShotTrackEditor, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool);

	/** Virtual destructor. */
	virtual ~FCinematicShotSection();

public:

	// ISequencerSection interface

	virtual void Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	virtual void BuildSectionSidebarMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual FMargin GetContentPadding() const override;
	virtual bool IsReadOnly() const override;

	// FThumbnail interface
	virtual void SetSingleTime(double GlobalTime) override;
	virtual bool CanRename() const override { return true; }
	virtual FText HandleThumbnailTextBlockText() const override;
	virtual void HandleThumbnailTextBlockTextCommitted(const FText& NewThumbnailName, ETextCommit::Type CommitType) override;
	virtual UCameraComponent* GetViewCamera() override;

private:

	void AddShotMenuSection(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding);

	/** The cinematic shot track editor that contains this section */
	TWeakPtr<FCinematicShotTrackEditor> CinematicShotTrackEditor;

	struct FCinematicSectionCache
	{
		FCinematicSectionCache(UMovieSceneCinematicShotSection* Section = nullptr);

		bool operator!=(const FCinematicSectionCache& RHS) const
		{
			return InnerFrameRate != RHS.InnerFrameRate || InnerFrameOffset != RHS.InnerFrameOffset || SectionStartFrame != RHS.SectionStartFrame || TimeScale != RHS.TimeScale;
		}

		FFrameRate   InnerFrameRate;
		FFrameNumber InnerFrameOffset;
		FFrameNumber SectionStartFrame;
		FMovieSceneTimeWarpVariant TimeScale;
	};

	/** Cached section thumbnail data */
	FCinematicSectionCache ThumbnailCacheData;
};
