// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/ColorPropertySection.h"

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseArray.h"
#include "ISequencer.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Vector2D.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneSectionHelpers.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Sections/MovieSceneColorSection.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "TimeToPixel.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;
struct FKeyHandle;

FColorPropertySection::FColorPropertySection(UMovieSceneSection& InSectionObject, const FGuid& InObjectBindingID, TWeakPtr<ISequencer> InSequencer)
	: FSequencerSection(InSectionObject)
	, ObjectBindingID(InObjectBindingID)
	, WeakSequencer(InSequencer)
{
	UMovieScenePropertyTrack* PropertyTrack = InSectionObject.GetTypedOuter<UMovieScenePropertyTrack>();
	if (PropertyTrack)
	{
		PropertyBindings = FTrackInstancePropertyBindings(PropertyTrack->GetPropertyName(), PropertyTrack->GetPropertyPath().ToString());
	}
}

FReply FColorPropertySection::OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles )
{
	UMovieSceneColorSection* ColorSection = Cast<UMovieSceneColorSection>( WeakSection.Get() );
	if (!ColorSection)
	{
		return FReply::Handled();
	}

	FMovieSceneChannelProxy& Proxy = ColorSection->GetChannelProxy();
	FMovieSceneFloatChannel* RChannel = Proxy.GetChannel<FMovieSceneFloatChannel>(0);
	FMovieSceneFloatChannel* GChannel = Proxy.GetChannel<FMovieSceneFloatChannel>(1);
	FMovieSceneFloatChannel* BChannel = Proxy.GetChannel<FMovieSceneFloatChannel>(2);
	FMovieSceneFloatChannel* AChannel = Proxy.GetChannel<FMovieSceneFloatChannel>(3);

	FMovieSceneKeyColorPicker KeyColorPicker(ColorSection, RChannel, GChannel, BChannel, AChannel, KeyHandles, WeakSequencer);

	return FReply::Handled();
}

int32 FColorPropertySection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	int32 LayerId = Painter.PaintSectionBackground();

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const UMovieSceneColorSection* ColorSection = Cast<const UMovieSceneColorSection>( WeakSection.Get() );

	const FTimeToPixel& TimeConverter = Painter.GetTimeConverter();

	const float StartTime       = TimeConverter.PixelToSeconds(0.f);
	const float EndTime         = TimeConverter.PixelToSeconds(Painter.SectionGeometry.GetLocalSize().X);
	const float SectionDuration = EndTime - StartTime;

	FVector2D GradientSize = FVector2D( Painter.SectionGeometry.Size.X - 2.f, (Painter.SectionGeometry.Size.Y / 4) - 3.0f );
	if ( GradientSize.X >= 1.f )
	{
		FPaintGeometry PaintGeometry = Painter.SectionGeometry.ToPaintGeometry( GradientSize, FSlateLayoutTransform(FVector2D( 1.f, 1.f )) );

		// If we are showing a background pattern and the colors is transparent, draw a checker pattern
		FSlateDrawElement::MakeBox(
			Painter.DrawElements,
			LayerId,
			PaintGeometry,
			FAppStyle::GetBrush( "Checker" ),
			DrawEffects);

		FLinearColor DefaultColor = GetPropertyValueAsLinearColor();
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = ColorSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

		TArray<FMovieSceneFloatChannel*> ColorChannels;
		ColorChannels.Add(FloatChannels[0]);
		ColorChannels.Add(FloatChannels[1]);
		ColorChannels.Add(FloatChannels[2]);
		ColorChannels.Add(FloatChannels[3]);

		TArray< TTuple<float, FLinearColor> > ColorKeys;
		MovieSceneSectionHelpers::ConsolidateColorCurves( ColorKeys, DefaultColor, ColorChannels, TimeConverter );

		TArray<FSlateGradientStop> GradientStops;

		for (const TTuple<float, FLinearColor>& ColorStop : ColorKeys)
		{
			const float Time = ColorStop.Get<0>();

			// HACK: The color is converted to SRgb and then reinterpreted as linear here because gradients are converted to FColor
			// without the SRgb conversion before being passed to the renderer for some reason.
			const FLinearColor Color = ColorStop.Get<1>().ToFColor( true ).ReinterpretAsLinear();

			float TimeFraction = (Time - StartTime) / SectionDuration;
			GradientStops.Add( FSlateGradientStop( FVector2D( TimeFraction * Painter.SectionGeometry.Size.X, 0 ), Color ) );
		}

		if ( GradientStops.Num() > 0 )
		{
			FSlateDrawElement::MakeGradient(
				Painter.DrawElements,
				Painter.LayerId + 1,
				PaintGeometry,
				GradientStops,
				Orient_Vertical,
				DrawEffects
				);
		}
	}

	return LayerId + 1;
}


FLinearColor FColorPropertySection::GetPropertyValueAsLinearColor() const
{
	UMovieSceneSection* Section = WeakSection.Get();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	FLinearColor LinearColor = FLinearColor::Black;

	if (Section && Sequencer.IsValid())
	{
		// Find the first object bound to this object binding ID, and apply each channel's external value to the color if possible
		for (TWeakObjectPtr<> WeakObject : Sequencer->FindObjectsInCurrentSequence(ObjectBindingID))
		{
			if (UObject* Object = WeakObject.Get())
			{
				// Access the editor data for the float channels which define how to extract the property value from the object
				TArrayView<const TMovieSceneExternalValue<float>> ExternalValues = Section->GetChannelProxy().GetAllExtendedEditorData<FMovieSceneFloatChannel>();

				FTrackInstancePropertyBindings* BindingsPtr = PropertyBindings.IsSet() ? &PropertyBindings.GetValue() : nullptr;

				if (ExternalValues[0].OnGetExternalValue)
				{
					LinearColor.R = ExternalValues[0].OnGetExternalValue(*Object, BindingsPtr).Get(0.f);
				}
				if (ExternalValues[1].OnGetExternalValue)
				{
					LinearColor.G = ExternalValues[1].OnGetExternalValue(*Object, BindingsPtr).Get(0.f);
				}
				if (ExternalValues[2].OnGetExternalValue)
				{
					LinearColor.B = ExternalValues[2].OnGetExternalValue(*Object, BindingsPtr).Get(0.f);
				}
				if (ExternalValues[3].OnGetExternalValue)
				{
					LinearColor.A = ExternalValues[3].OnGetExternalValue(*Object, BindingsPtr).Get(0.f);
				}

				break;
			}
		}
	}

	return LinearColor;
}
