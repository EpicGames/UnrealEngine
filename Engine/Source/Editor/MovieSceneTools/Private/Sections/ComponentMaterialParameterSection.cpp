// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/ComponentMaterialParameterSection.h"

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "ISequencer.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "MovieSceneSection.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/Views/SChannelView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneComponentMaterialParameterSection.h"
#include "Sections/MovieSceneSectionHelpers.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "TimeToPixel.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FName;
struct FKeyHandle;

#define LOCTEXT_NAMESPACE "ParameterSection"

namespace UE::Sequencer
{
	class SMaterialColorStripView : public SChannelView
	{
		SLATE_BEGIN_ARGS(SMaterialColorStripView) {}
		SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const FViewModelPtr& InViewModel, TSharedPtr<STrackAreaView> InTrackAreaView)
		{
			WeakModel = InViewModel;
			WeakTrackAreaView = InTrackAreaView;

			SChannelView::Construct(SChannelView::FArguments(), InViewModel, InTrackAreaView);
		}

		int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			FViewModelPtr OwningModel = WeakModel.Pin();
			if (!OwningModel)
			{
				return LayerId;
			}

			bool bIsExpanded = false;
			if (TSharedPtr<FLinkedOutlinerExtension> LinkedOutlinerExtension = OwningModel.ImplicitCast())
			{
				TSharedPtr<IOutlinerExtension> OutlinerItem = LinkedOutlinerExtension ? LinkedOutlinerExtension->GetLinkedOutlinerItem() : nullptr;
				if (OutlinerItem)
				{
					bIsExpanded = OutlinerItem->IsExpanded();
				}
			}

			const FMovieSceneFloatChannel* ColorChannels[4] = { };
			for (TViewModelPtr<FChannelModel> ChildChannel : OwningModel->GetChildrenOfType<FChannelModel>())
			{
				FMovieSceneChannelHandle ChannelHandle = ChildChannel->GetKeyArea()->GetChannel();

				FText DisplayText = ChannelHandle.GetMetaData()->DisplayText;

				if (ChannelHandle.GetMetaData()->DisplayText.EqualTo(FCommonChannelData::ChannelR))
				{
					ColorChannels[0] = ChannelHandle.Cast<FMovieSceneFloatChannel>().Get();
				}
				if (ChannelHandle.GetMetaData()->DisplayText.EqualTo(FCommonChannelData::ChannelG))
				{
					ColorChannels[1] = ChannelHandle.Cast<FMovieSceneFloatChannel>().Get();
				}
				if (ChannelHandle.GetMetaData()->DisplayText.EqualTo(FCommonChannelData::ChannelB))
				{
					ColorChannels[2] = ChannelHandle.Cast<FMovieSceneFloatChannel>().Get();
				}
				if (ChannelHandle.GetMetaData()->DisplayText.EqualTo(FCommonChannelData::ChannelA))
				{
					ColorChannels[3] = ChannelHandle.Cast<FMovieSceneFloatChannel>().Get();
				}
			}

			if (ColorChannels[0] && ColorChannels[1] && ColorChannels[2] && ColorChannels[3])
			{
				const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

				FTimeToPixel RelativeTimeToPixel = *WeakTrackAreaView->GetTimeToPixel();

				TSharedPtr<ITrackLaneExtension> TrackLaneExtension = WeakModel.ImplicitPin();
				if (TrackLaneExtension)
				{
					FTrackLaneVirtualAlignment VirtualAlignment = TrackLaneExtension->ArrangeVirtualTrackLaneView();
					if (VirtualAlignment.Range.GetLowerBound().IsClosed())
					{
						RelativeTimeToPixel = RelativeTimeToPixel.RelativeTo(VirtualAlignment.Range.GetLowerBoundValue());
					}
				}

				const float StartTime = RelativeTimeToPixel.PixelToSeconds(0.f);
				const float EndTime = RelativeTimeToPixel.PixelToSeconds(AllottedGeometry.GetLocalSize().X);
				const float SectionDuration = EndTime - StartTime;

				FVector2D GradientSize = AllottedGeometry.GetLocalSize() - FVector2D(2.f, 0.f);
				if (GradientSize.X >= 1.f)
				{
					FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry(GradientSize, FSlateLayoutTransform(FVector2D(1.f, 1.f)));

					// If we are showing a background pattern and the colors is transparent, draw a checker pattern
					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						PaintGeometry,
						FAppStyle::GetBrush("Checker"),
						DrawEffects);

					FLinearColor DefaultColor = FLinearColor::Black;
					DefaultColor.R = ColorChannels[0]->GetDefault().IsSet() ? ColorChannels[0]->GetDefault().GetValue() : 0.f;
					DefaultColor.G = ColorChannels[1]->GetDefault().IsSet() ? ColorChannels[1]->GetDefault().GetValue() : 0.f;
					DefaultColor.B = ColorChannels[2]->GetDefault().IsSet() ? ColorChannels[2]->GetDefault().GetValue() : 0.f;
					DefaultColor.A = ColorChannels[3]->GetDefault().IsSet() ? ColorChannels[3]->GetDefault().GetValue() : 0.f;

					TArray< TTuple<float, FLinearColor> > ColorKeys;
					MovieSceneSectionHelpers::ConsolidateColorCurves(ColorKeys, DefaultColor, ColorChannels, RelativeTimeToPixel);

					TArray<FSlateGradientStop> GradientStops;

					for (const TTuple<float, FLinearColor>& ColorStop : ColorKeys)
					{
						const float Time = ColorStop.Get<0>();

						// HACK: The color is converted to SRgb and then reinterpreted as linear here because gradients are converted to FColor
						// without the SRgb conversion before being passed to the renderer for some reason.
						const FLinearColor Color = ColorStop.Get<1>().ToFColor(true).ReinterpretAsLinear();

						float TimeFraction = (Time - StartTime) / SectionDuration;
						GradientStops.Add(FSlateGradientStop(FVector2D(TimeFraction * GradientSize.X, 0), Color));
					}

					if (GradientStops.Num() > 0)
					{
						FSlateDrawElement::MakeGradient(
							OutDrawElements,
							LayerId + 1,
							PaintGeometry,
							GradientStops,
							Orient_Vertical,
							DrawEffects
						);
					}
				}
			}

			LayerId += 2;
			return SChannelView::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		}

		FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D();
		}

		FWeakViewModelPtr WeakModel;
		TSharedPtr<STrackAreaView> WeakTrackAreaView;
	};
}

FReply FComponentMaterialParameterSection::OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles)
{
	UMovieSceneComponentMaterialParameterSection* ParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(WeakSection.Get());
	if (!ParameterSection)
	{
		return FReply::Handled();
	}

	for (FColorMaterialParameterInfoAndCurves& NameAndCurve : ParameterSection->ColorParameterInfosAndCurves)
	{
		FMovieSceneKeyColorPicker KeyColorPicker(ParameterSection, &NameAndCurve.RedCurve, &NameAndCurve.GreenCurve, &NameAndCurve.BlueCurve, &NameAndCurve.AlphaCurve, KeyHandles, WeakSequencer);
	}

	return FReply::Handled();
}

TSharedPtr<UE::Sequencer::FCategoryModel> FComponentMaterialParameterSection::ConstructCategoryModel(FName InCategoryName, const FText& InDisplayText, TArrayView<const FChannelData> Channels) const
{
	using namespace UE::Sequencer;

	// Only construct the color category if it has all the channels
	uint8 ColorChannelMask = 0;

	for (const FChannelData& Channel : Channels)
	{
		if (Channel.MetaData.DisplayText.EqualTo(FCommonChannelData::ChannelR))
		{
			ColorChannelMask |= 1 << 0;
		}
		else if (Channel.MetaData.DisplayText.EqualTo(FCommonChannelData::ChannelG))
		{
			ColorChannelMask |= 1 << 1;
		}
		else if (Channel.MetaData.DisplayText.EqualTo(FCommonChannelData::ChannelB))
		{
			ColorChannelMask |= 1 << 2;
		}
		else if (Channel.MetaData.DisplayText.EqualTo(FCommonChannelData::ChannelA))
		{
			ColorChannelMask |= 1 << 3;
		}
	}

	if (ColorChannelMask == 0b1111)
	{
		class FMaterialColorCategory : public FCategoryModel
		{
		public:
			FMaterialColorCategory(FName InCategoryName, const FText& InDisplayText)
				: FCategoryModel(InCategoryName)
			{}

			TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override
			{
				return SNew(SMaterialColorStripView, SharedThis(this), InParams.OwningTrackLane->GetTrackAreaView());
			}
		};

		return MakeShared<FMaterialColorCategory>(InCategoryName, InDisplayText);
	}

	return nullptr;
}

bool FComponentMaterialParameterSection::RequestDeleteCategory(const TArray<FName>& CategoryNamePath)
{
	const FScopedTransaction Transaction(LOCTEXT("DeleteParameter", "Delete parameter"));
	UMovieSceneComponentMaterialParameterSection* ParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(WeakSection.Get());
	if (ParameterSection && ParameterSection->Modify() && !CategoryNamePath.IsEmpty())
	{
		const FName& ChannelName = CategoryNamePath.Last();

		const bool bColorParameterDeleted = ParameterSection->RemoveColorParameter(ChannelName);
		const bool bScalarParameterDeleted = ParameterSection->RemoveScalarParameter(ChannelName);
		return bColorParameterDeleted || bScalarParameterDeleted;
	}
	return false;
}


bool FComponentMaterialParameterSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath)
{
	// Only handle paths with a single name, in all other cases the user is deleting a component of a vector parameter.
	if (KeyAreaNamePath.Num() == 1)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteParameter", "Delete parameter"));
		UMovieSceneComponentMaterialParameterSection* ParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(WeakSection.Get());
		if (ParameterSection && ParameterSection->TryModify())
		{
			const bool bColorParameterDeleted = ParameterSection->RemoveColorParameter(KeyAreaNamePath[0]);
			const bool bScalarParameterDeleted = ParameterSection->RemoveScalarParameter(KeyAreaNamePath[0]);
			return bColorParameterDeleted || bScalarParameterDeleted;
		}
	}
	return false;
}
#undef LOCTEXT_NAMESPACE
