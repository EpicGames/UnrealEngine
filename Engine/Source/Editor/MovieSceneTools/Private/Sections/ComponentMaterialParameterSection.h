// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "ISequencerSection.h"
#include "Input/Reply.h"

class FName;
class FSequencerSectionPainter;
class ISequencer;
class UMovieSceneSection;
struct FKeyHandle;

/**
 * A movie scene section for material parameters.
 */
class FComponentMaterialParameterSection
	: public FSequencerSection
{
public:

	FComponentMaterialParameterSection(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSectionObject)
		, WeakSequencer(InSequencer)
	{ }

public:

	//~ ISequencerSection interface
	virtual FReply OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles) override;
	virtual bool RequestDeleteCategory(const TArray<FName>& CategoryNamePath) override;
	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;

	virtual TSharedPtr<UE::Sequencer::FCategoryModel> ConstructCategoryModel(FName InCategoryName, const FText& InDisplayText, TArrayView<const FChannelData> Channels) const;

private:

	/** Weak pointer to the sequencer this section is for */
	TWeakPtr<ISequencer> WeakSequencer;
};
