// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/EventSection.h"

#include "EditorFontGlyphs.h"
#include "K2Node_CustomEvent.h"  // IWYU pragma: keep
#include "K2Node_FunctionEntry.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MovieSceneEventUtils.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceEditor.h"
#include "ScopedTransaction.h"
#include "Sections/EventSectionHelper.h"
#include "Sections/MovieSceneEventRepeaterSection.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/MovieSceneEventTriggerSection.h"
#include "SequencerSectionPainter.h"
#include "TimeToPixel.h"

#define LOCTEXT_NAMESPACE "EventSection"

bool FEventSectionBase::IsSectionSelected() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin())
	{
		return FEventSectionHelper::IsSectionSelected(SequencerPtr.ToSharedRef(), WeakSection.Get());
	}
	return false;
}

void FEventSectionBase::PaintEventName(FSequencerSectionPainter& Painter, int32 LayerId, const FString& InEventString, float PixelPos, bool bIsEventValid) const
{
	FEventSectionHelper::PaintEventName(Painter, LayerId, InEventString, PixelPos, bIsEventValid);
}

int32 FEventSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();
	UMovieSceneEventSection* EventSection = Cast<UMovieSceneEventSection>( WeakSection.Get() );
	if (!EventSection || !IsSectionSelected())
	{
		return LayerId;
	}

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	for (int32 KeyIndex = 0; KeyIndex < EventSection->GetEventData().GetKeyTimes().Num(); ++KeyIndex)
	{
		FFrameNumber EventTime = EventSection->GetEventData().GetKeyTimes()[KeyIndex];
		FEventPayload EventData = EventSection->GetEventData().GetKeyValues()[KeyIndex];

		if (EventSection->GetRange().Contains(EventTime))
		{
			FString EventString = EventData.EventName.ToString();
			if (!EventString.IsEmpty())
			{
				const float PixelPos = TimeToPixelConverter.FrameToPixel(EventTime);
				PaintEventName(Painter, LayerId, EventString, PixelPos);
			}
		}
	}

	return LayerId + 3;
}

int32 FEventTriggerSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();

	UMovieSceneEventTriggerSection* EventTriggerSection = Cast<UMovieSceneEventTriggerSection>(WeakSection.Get());
	if (!EventTriggerSection || !IsSectionSelected())
	{
		return LayerId;
	}

	UMovieSceneSequence* Sequence = EventTriggerSection->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	UBlueprint* SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	// If we do not have a sequence director BP we can't possibly be bound to anything
	if (!SequenceDirectorBP)
	{
		return LayerId;
	}

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	TArrayView<const FFrameNumber> Times  = EventTriggerSection->EventChannel.GetData().GetTimes();
	TArrayView<FMovieSceneEvent>   Events = EventTriggerSection->EventChannel.GetData().GetValues();

	TRange<FFrameNumber> EventSectionRange = EventTriggerSection->GetRange();
	for (int32 KeyIndex = 0; KeyIndex < Times.Num(); ++KeyIndex)
	{
		FFrameNumber EventTime = Times[KeyIndex];
		if (EventSectionRange.Contains(EventTime))
		{
			UK2Node* EndpointNode = FMovieSceneEventUtils::FindEndpoint(&Events[KeyIndex], EventTriggerSection, SequenceDirectorBP);

			FString EventString = EndpointNode ? EndpointNode->GetNodeTitle(ENodeTitleType::MenuTitle).ToString() : FString();
			bool bIsEventValid = true;

			const float PixelPos = TimeToPixelConverter.FrameToPixel(EventTime);
			PaintEventName(Painter, LayerId, EventString, PixelPos, bIsEventValid);
		}
	}

	return LayerId + 3;
}

FReply FEventTriggerSection::OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles)
{
	UMovieSceneEventTriggerSection* EventTriggerSection = Cast<UMovieSceneEventTriggerSection>( WeakSection.Get() );
	if (!EventTriggerSection)
	{
		return FReply::Handled();
	}

	UMovieSceneSequence* Sequence = EventTriggerSection->GetTypedOuter<UMovieSceneSequence>();
	check(Sequence);

	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	if (!SequenceEditor)
	{
		return FReply::Handled();
	}

	UBlueprint* SequenceDirectorBP = SequenceEditor->GetOrCreateDirectorBlueprint(Sequence);
	if (!SequenceDirectorBP)
	{
		return FReply::Handled();
	}

	TMovieSceneChannelData<FMovieSceneEvent> ChannelData = EventTriggerSection->EventChannel.GetData();
	for (FKeyHandle KeyHandle : KeyHandles)
	{
		const int32 EventIndex = ChannelData.GetIndex(KeyHandle);
		if (EventIndex == INDEX_NONE)
		{
			continue;
		}

		FMovieSceneEvent* EventEntryPoint = &ChannelData.GetValues()[EventIndex];
		UK2Node* Endpoint = FMovieSceneEventUtils::FindEndpoint(EventEntryPoint, EventTriggerSection, SequenceDirectorBP);

		if (!Endpoint)
		{
			FScopedTransaction Transaction(LOCTEXT("CreateEventEndpoint", "Create Event Endpoint"));
			Endpoint = FMovieSceneEventUtils::BindNewUserFacingEvent(EventEntryPoint, EventTriggerSection, SequenceDirectorBP);
		}

		if (Endpoint)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Endpoint, false);
			return FReply::Handled();
		}
	}

	return FReply::Handled();
}

int32 FEventRepeaterSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();

	UMovieSceneEventRepeaterSection* EventRepeaterSection = Cast<UMovieSceneEventRepeaterSection>(WeakSection.Get());
	if (!EventRepeaterSection)
	{
		return LayerId;
	}

	UMovieSceneSequence* Sequence = EventRepeaterSection->GetTypedOuter<UMovieSceneSequence>();
	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	UBlueprint* SequenceDirectorBP = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

	// If we do not have a sequence director BP we can't possibly be bound to anything
	if (!SequenceDirectorBP)
	{
		return LayerId;
	}

	UK2Node* EndpointNode = FMovieSceneEventUtils::FindEndpoint(&EventRepeaterSection->Event, EventRepeaterSection, SequenceDirectorBP);

	float TextOffsetX = EventRepeaterSection->GetRange().GetLowerBound().IsClosed() ? FMath::Max(0.f, Painter.GetTimeConverter().FrameToPixel(EventRepeaterSection->GetRange().GetLowerBoundValue())) : 0.f;

	FString EventString = EndpointNode ? EndpointNode->GetNodeTitle(ENodeTitleType::MenuTitle).ToString() : FString();
	bool bIsEventValid = true;
	PaintEventName(Painter, LayerId, EventString, TextOffsetX, bIsEventValid);

	return LayerId + 1;
}

FReply FEventRepeaterSection::OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (!SequencerPtr)
	{
		return FReply::Handled();
	}

	UMovieSceneEventRepeaterSection* EventRepeaterSection = Cast<UMovieSceneEventRepeaterSection>(WeakSection.Get());
	if (!EventRepeaterSection)
	{
		return FReply::Handled();
	}

	UMovieSceneSequence* Sequence = EventRepeaterSection->GetTypedOuter<UMovieSceneSequence>();
	check(Sequence);

	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(Sequence);
	if (!SequenceEditor)
	{
		return FReply::Handled();
	}

	UBlueprint* SequenceDirectorBP = SequenceEditor->GetOrCreateDirectorBlueprint(Sequence);
	if (!SequenceDirectorBP)
	{
		return FReply::Handled();
	}

	UK2Node* Endpoint = FMovieSceneEventUtils::FindEndpoint(&EventRepeaterSection->Event, EventRepeaterSection, SequenceDirectorBP);

	if (!Endpoint)
	{
		FScopedTransaction Transaction(LOCTEXT("BindRepeaterEvent", "Create Event Endpoint"));
		Endpoint = FMovieSceneEventUtils::BindNewUserFacingEvent(&EventRepeaterSection->Event, EventRepeaterSection, SequenceDirectorBP);
	}

	if (Endpoint)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Endpoint, false);
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE // "EventSection"
