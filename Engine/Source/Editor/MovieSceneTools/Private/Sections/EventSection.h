// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "ISequencer.h"

class FSequencerSectionPainter;

class FEventSectionBase
	: public FSequencerSection
{
public:

	FEventSectionBase(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSectionObject)
		, Sequencer(InSequencer)
	{}

protected:

	void PaintEventName(FSequencerSectionPainter& Painter, int32 LayerId, const FString& EventString, float PixelPosition, bool bIsEventValid = true) const;

	bool IsSectionSelected() const;

protected:

	TWeakPtr<ISequencer> Sequencer;
};

/**
* An implementation of event sections.
*/
class FEventSection
	: public FEventSectionBase
{
public:
	FEventSection(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FEventSectionBase(InSectionObject, InSequencer)
	{}

public:

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
};

class FEventTriggerSection
	: public FEventSectionBase
{
public:

	FEventTriggerSection(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FEventSectionBase(InSectionObject, InSequencer)
	{}

public:

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual FReply OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles) override;
};

class FEventRepeaterSection
	: public FEventSectionBase
{
public:

	FEventRepeaterSection(UMovieSceneSection& InSectionObject, TWeakPtr<ISequencer> InSequencer)
		: FEventSectionBase(InSectionObject, InSequencer)
	{}


public:

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual FReply OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent) override;
};

