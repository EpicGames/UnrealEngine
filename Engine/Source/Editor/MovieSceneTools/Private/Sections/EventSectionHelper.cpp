// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/EventSectionHelper.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencer.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"

void FEventSectionHelper::PaintEventName(
		FSequencerSectionPainter& Painter,
		int32 LayerId,
		const FString& InEventString,
		float PixelPos,
		bool bIsEventValid)
{
	static const int32   FontSize      = 10;
	static const float   BoxOffsetPx   = 10.f;
	static const TCHAR*  WarningString = TEXT("\xf071");

	const FSlateFontInfo FontAwesomeFont = FAppStyle::Get().GetFontStyle("FontAwesome.10");
	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const FLinearColor   DrawColor       = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Setup the warning size. Static since it won't ever change
	static FVector2D WarningSize    = FontMeasureService->Measure(WarningString, FontAwesomeFont);
	const  FMargin   WarningPadding = (bIsEventValid || InEventString.Len() == 0) ? FMargin(0.f) : FMargin(0.f, 0.f, 4.f, 0.f);
	const  FMargin   BoxPadding     = FMargin(4.0f, 2.0f);

	const FVector2D  TextSize       = FontMeasureService->Measure(InEventString, SmallLayoutFont);
	const FVector2D  IconSize       = bIsEventValid ? FVector2D::ZeroVector : WarningSize;
	const FVector2D  PaddedIconSize = IconSize + WarningPadding.GetDesiredSize();
	const FVector2D  BoxSize        = FVector2D(TextSize.X + PaddedIconSize.X, FMath::Max(TextSize.Y, PaddedIconSize.Y )) + BoxPadding.GetDesiredSize();

	// Flip the text position if getting near the end of the view range
	bool  bDrawLeft    = (Painter.SectionGeometry.Size.X - PixelPos) < (BoxSize.X + 22.f) - BoxOffsetPx;
	float BoxPositionX = bDrawLeft ? PixelPos - BoxSize.X - BoxOffsetPx : PixelPos + BoxOffsetPx;
	if (BoxPositionX < 0.f)
	{
		BoxPositionX = 0.f;
	}

	FVector2D BoxOffset  = FVector2D(BoxPositionX,                    Painter.SectionGeometry.Size.Y*.5f - BoxSize.Y*.5f);
	FVector2D IconOffset = FVector2D(BoxPadding.Left,                 BoxSize.Y*.5f - IconSize.Y*.5f);
	FVector2D TextOffset = FVector2D(IconOffset.X + PaddedIconSize.X, BoxSize.Y*.5f - TextSize.Y*.5f);

	// Draw the background box
	FSlateDrawElement::MakeBox(
		Painter.DrawElements,
		LayerId + 1,
		Painter.SectionGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxOffset)),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.5f)
	);

	if (!bIsEventValid)
	{
		// Draw a warning icon for unbound repeaters
		FSlateDrawElement::MakeText(
			Painter.DrawElements,
			LayerId + 2,
			Painter.SectionGeometry.ToPaintGeometry(IconSize, FSlateLayoutTransform(BoxOffset + IconOffset)),
			WarningString,
			FontAwesomeFont,
			Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FAppStyle::GetWidgetStyle<FTextBlockStyle>("Log.Warning").ColorAndOpacity.GetSpecifiedColor()
		);
	}

	FSlateDrawElement::MakeText(
		Painter.DrawElements,
		LayerId + 2,
		Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(BoxOffset + TextOffset)),
		InEventString,
		SmallLayoutFont,
		Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		DrawColor
	);
}

bool FEventSectionHelper::IsSectionSelected(TSharedRef<ISequencer> Sequencer, UMovieSceneSection* InSection)
{
	TArray<UMovieSceneTrack*> SelectedTracks;
	Sequencer->GetSelectedTracks(SelectedTracks);

	UMovieSceneTrack* Track = InSection ? CastChecked<UMovieSceneTrack>(InSection->GetOuter()) : nullptr;
	return Track && SelectedTracks.Contains(Track);
}

