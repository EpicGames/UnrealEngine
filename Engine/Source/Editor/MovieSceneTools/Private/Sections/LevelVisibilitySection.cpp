// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/LevelVisibilitySection.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DragAndDrop/LevelDragDropOp.h"
#include "Engine/LevelStreaming.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/PackageName.h"
#include "SDropTarget.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "SequencerSectionPainter.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UMovieSceneSection;
struct FGeometry;

namespace LevelVisibilitySection
{
	constexpr int32 MaxNumLevelsToShow = 3;
}

FLevelVisibilitySection::FLevelVisibilitySection( UMovieSceneLevelVisibilitySection& InSectionObject )
	: SectionObject( InSectionObject )
{
	VisibleText = NSLOCTEXT( "LevelVisibilitySection", "VisibleHeader", "Visible" );
	HiddenText = NSLOCTEXT( "LevelVisibilitySection", "HiddenHeader", "Hidden" );
}

UMovieSceneSection* FLevelVisibilitySection::GetSectionObject()
{
	return &SectionObject;
}

TSharedRef<SWidget> FLevelVisibilitySection::GenerateSectionWidget()
{
	return
		SNew( SDropTarget )
		.OnAllowDrop( this, &FLevelVisibilitySection::OnAllowDrop )
		.OnDropped( this, &FLevelVisibilitySection::OnDrop )
		.Content()
		[
			SNew( SBorder )
			.BorderBackgroundColor( this, &FLevelVisibilitySection::GetBackgroundColor )
			.BorderImage( FCoreStyle::Get().GetBrush( "WhiteBrush" ) )
			[
				SNew( STextBlock )
				.Text( this, &FLevelVisibilitySection::GetVisibilityText )
				.ToolTipText( this, &FLevelVisibilitySection::GetVisibilityToolTip )
			]
		];
}


int32 FLevelVisibilitySection::OnPaintSection( FSequencerSectionPainter& InPainter ) const
{
	return InPainter.PaintSectionBackground();
}


FSlateColor FLevelVisibilitySection::GetBackgroundColor() const
{
	return SectionObject.GetVisibility() == ELevelVisibility::Visible
		? FSlateColor( FLinearColor::Green.Desaturate( .5f ) )
		: FSlateColor( FLinearColor::Red.Desaturate( .5f ) );
}


FText FLevelVisibilitySection::GetVisibilityText() const
{
	TArray<FString> LevelNameStrings;
	LevelNameStrings.Reserve(LevelVisibilitySection::MaxNumLevelsToShow);
	
	int32 Count = 0;
	for ( ; Count < LevelVisibilitySection::MaxNumLevelsToShow && Count < SectionObject.GetLevelNames().Num(); Count++ )
	{
		LevelNameStrings.Add( SectionObject.GetLevelNames()[Count].ToString() );
	}

	int32 NumRemaining = SectionObject.GetLevelNames().Num() - Count;
	FString LevelsText = FString::Join( LevelNameStrings, TEXT( ", " ) );

	if (SectionObject.GetLevelNames().Num() > LevelVisibilitySection::MaxNumLevelsToShow )
	{
		LevelsText.Append( FString::Format(TEXT(" (+{0} more)"), { FString::FormatAsNumber(NumRemaining) } ) );
	}

	if (LevelsText.IsEmpty())
	{
		FText VisibilityText = SectionObject.GetVisibility() == ELevelVisibility::Visible ? VisibleText : HiddenText;

		return VisibilityText;
	}

	return FText::Format( NSLOCTEXT( "LevelVisibilitySection", "SectionTextFormat", "{0}" ), FText::FromString( LevelsText ) );
}

FText FLevelVisibilitySection::GetVisibilityToolTip() const
{
	TArray<FString> LevelNameStrings;
	for ( const FName& LevelName : SectionObject.GetLevelNames() )
	{
		LevelNameStrings.Add( LevelName.ToString() );
	}

	FText VisibilityText = SectionObject.GetVisibility() == ELevelVisibility::Visible ? VisibleText : HiddenText;
	return FText::Format( NSLOCTEXT( "LevelVisibilitySection", "ToolTipFormat", "{0}\r\n{1}" ), VisibilityText, FText::FromString( FString::Join( LevelNameStrings, TEXT( "\r\n" ) ) ) );
}


bool FLevelVisibilitySection::OnAllowDrop( TSharedPtr<FDragDropOperation> DragDropOperation )
{
	return DragDropOperation->IsOfType<FLevelDragDropOp>() && StaticCastSharedPtr<FLevelDragDropOp>( DragDropOperation )->StreamingLevelsToDrop.Num() > 0;
}


FReply FLevelVisibilitySection::OnDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent )
{
	TSharedPtr<FLevelDragDropOp> LevelDragDropOperation = InDragDropEvent.GetOperationAs<FLevelDragDropOp>();
	if ( LevelDragDropOperation )
	{
		if ( LevelDragDropOperation->StreamingLevelsToDrop.Num() > 0 )
		{
			FScopedTransaction Transaction(NSLOCTEXT("LevelVisibilitySection", "TransactionText", "Add Level(s) to Level Visibility Section"));
			SectionObject.Modify();

			TArray<FName> LevelNames = SectionObject.GetLevelNames();
			for ( TWeakObjectPtr<ULevelStreaming> Level : LevelDragDropOperation->StreamingLevelsToDrop )
			{
				if ( Level.IsValid() )
				{
					FName ShortLevelName = FPackageName::GetShortFName( Level->GetWorldAssetPackageFName() );
					LevelNames.AddUnique( ShortLevelName );
				}
			}

			SectionObject.SetLevelNames(LevelNames);

			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

