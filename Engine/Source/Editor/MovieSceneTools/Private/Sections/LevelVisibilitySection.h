// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ISequencerSection.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

class FDragDropEvent;
class FDragDropOperation;
class FSequencerSectionPainter;
class SWidget;
class UMovieSceneLevelVisibilitySection;
class UMovieSceneSection;
struct FGeometry;

/**
 * A seqeuencer section for displaying and interacting with level visibility movie scene sections. 
 */
class FLevelVisibilitySection
	: public ISequencerSection
	, public TSharedFromThis<FLevelVisibilitySection>
{
public:

	FLevelVisibilitySection( UMovieSceneLevelVisibilitySection& InSectionObject );

public:

	// ISequencerSectionInterface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual TSharedRef<SWidget> GenerateSectionWidget() override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override;

private:

	FSlateColor GetBackgroundColor() const;
	FText GetVisibilityText() const;
	FText GetVisibilityToolTip() const;

	bool OnAllowDrop( TSharedPtr<FDragDropOperation> DragDropOperation );
	FReply OnDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent );

private:

	FText VisibleText;
	FText HiddenText;

	UMovieSceneLevelVisibilitySection& SectionObject;

	FText DisplayName;
};
