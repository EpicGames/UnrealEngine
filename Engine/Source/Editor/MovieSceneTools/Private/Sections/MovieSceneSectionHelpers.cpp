// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSectionHelpers.h"

#include "Sections/MovieSceneColorSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"

#include "ISequencer.h"
#include "Misc/FrameNumber.h"
#include "Widgets/Colors/SColorPicker.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"
#include "TimeToPixel.h"

void MovieSceneSectionHelpers::ConsolidateColorCurves( TArray< TTuple<float, FLinearColor> >& OutColorKeys, const FLinearColor& DefaultColor, TArrayView<const FMovieSceneFloatChannel* const> ColorChannels, const FTimeToPixel& TimeConverter )
{
	// @todo Sequencer Optimize - This could all get cached, instead of recalculating everything every OnPaint

	if (!ensure(ColorChannels.Num() == 4))
	{
		return;
	}

	// Gather all the channels with keys
	TArray<TArrayView<const FFrameNumber>, TInlineAllocator<4>> ChannelTimes;
	for (int32 Index = 0; Index < 4; ++Index)
	{
		if (ColorChannels[Index]->GetTimes().Num())
		{
			ChannelTimes.Add(ColorChannels[Index]->GetTimes());
		}
	}

	// Keep adding color stops for similar times until we have nothing left
	while ( ChannelTimes.Num() )
	{
		// Find the earliest time from the remaining channels
		FFrameNumber Time = TNumericLimits<int32>::Max();
		for (const TArrayView<const FFrameNumber>& Channel : ChannelTimes)
		{
			Time = FMath::Min(Time, Channel[0]);
		}

		// Slice the channels until we no longer match the next time
		for (TArrayView<const FFrameNumber>& Channel : ChannelTimes)
		{
			int32 SliceIndex = 0;
			while (SliceIndex < Channel.Num() && Time == Channel[SliceIndex])
			{
				++SliceIndex;
			}

			if (SliceIndex > 0)
			{
				int32 NewNum = Channel.Num() - SliceIndex;
				Channel = NewNum > 0 ? Channel.Slice(SliceIndex, NewNum) : TArrayView<const FFrameNumber>();
			}
		}

		// Remove empty channels with no keys left
		for (int32 Index = ChannelTimes.Num()-1; Index >= 0; --Index)
		{
			if (ChannelTimes[Index].Num() == 0)
			{
				ChannelTimes.RemoveAt(Index, EAllowShrinking::No);
			}
		}

		FLinearColor ColorAtTime = DefaultColor;
		ColorChannels[0]->Evaluate(Time, ColorAtTime.R);
		ColorChannels[1]->Evaluate(Time, ColorAtTime.G);
		ColorChannels[2]->Evaluate(Time, ColorAtTime.B);
		ColorChannels[3]->Evaluate(Time, ColorAtTime.A);

		OutColorKeys.Add(MakeTuple(float(Time / TimeConverter.GetTickResolution()), ColorAtTime));
	}

	// Enforce at least one key for the default value
	if (OutColorKeys.Num() == 0)
	{
		OutColorKeys.Add(MakeTuple(0.f, DefaultColor));
	}
}

FFrameNumber FMovieSceneKeyColorPicker::KeyTime = FFrameNumber();
FLinearColor FMovieSceneKeyColorPicker::InitialColor = FLinearColor();
bool FMovieSceneKeyColorPicker::bColorPickerWasCancelled = false;

FMovieSceneKeyColorPicker::FMovieSceneKeyColorPicker(UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, const TArray<FKeyHandle>& KeyHandles, TWeakPtr<ISequencer> InSequencer)
{
	// One key was clicked on, find which from the channels
	int32 KeyIndex = INDEX_NONE;

	for (FKeyHandle KeyHandle : KeyHandles)
	{
		if (RChannel->GetData().GetIndex(KeyHandle) != INDEX_NONE)
		{
			KeyIndex = RChannel->GetData().GetIndex(KeyHandle);
			RChannel->GetKeyTime(KeyHandle, KeyTime);
			break;
		}
		if (GChannel->GetData().GetIndex(KeyHandle) != INDEX_NONE)
		{
			KeyIndex = GChannel->GetData().GetIndex(KeyHandle);
			GChannel->GetKeyTime(KeyHandle, KeyTime);
			break;
		}	
		if (BChannel->GetData().GetIndex(KeyHandle) != INDEX_NONE)
		{
			KeyIndex = BChannel->GetData().GetIndex(KeyHandle);
			BChannel->GetKeyTime(KeyHandle, KeyTime);
			break;
		}	
		if (AChannel->GetData().GetIndex(KeyHandle) != INDEX_NONE)
		{
			KeyIndex = AChannel->GetData().GetIndex(KeyHandle);
			AChannel->GetKeyTime(KeyHandle, KeyTime);
			break;
		}	
	}

	if (KeyIndex == INDEX_NONE)
	{
		return;
	}

	float R, G, B, A = 0.f;
	RChannel->Evaluate(KeyTime, R);
	GChannel->Evaluate(KeyTime, G);
	BChannel->Evaluate(KeyTime, B);
	AChannel->Evaluate(KeyTime, A);

	InitialColor = FLinearColor(R, G, B, A);
	bColorPickerWasCancelled = false;

	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.InitialColor = InitialColor;
	PickerArgs.ParentWidget = FSlateApplication::Get().GetActiveTopLevelWindow();
	PickerArgs.OptionalOwningDetailsView = PickerArgs.ParentWidget;

	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateRaw(this, &FMovieSceneKeyColorPicker::OnColorPickerPicked, Section, RChannel, GChannel, BChannel, AChannel, InSequencer);
	PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateRaw(this, &FMovieSceneKeyColorPicker::OnColorPickerClosed, Section, RChannel, GChannel, BChannel, AChannel, InSequencer);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateRaw(this, &FMovieSceneKeyColorPicker::OnColorPickerCancelled, Section, RChannel, GChannel, BChannel, AChannel, InSequencer);

	OpenColorPicker(PickerArgs);
}

void UpdateOrAddKey(FMovieSceneFloatChannel* Channel, FFrameNumber KeyTime, float Value)
{
	int32 KeyHandleIndex = Channel->GetData().FindKey(KeyTime);
	if (KeyHandleIndex == INDEX_NONE)
	{
		Channel->GetData().UpdateOrAddKey(KeyTime, FMovieSceneFloatValue(Value));
	}
	else
	{
		Channel->GetData().GetValues()[KeyHandleIndex].Value = Value;
	}
}

void FMovieSceneKeyColorPicker::OnColorPickerPicked(FLinearColor NewColor, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> WeakSequencer)
{
	Section->Modify();

	UpdateOrAddKey(RChannel, KeyTime, NewColor.R);
	UpdateOrAddKey(GChannel, KeyTime, NewColor.G);
	UpdateOrAddKey(BChannel, KeyTime, NewColor.B);
	UpdateOrAddKey(AChannel, KeyTime, NewColor.A);

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FMovieSceneKeyColorPicker::OnColorPickerClosed(const TSharedRef<SWindow>& Window, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> WeakSequencer)
{
	// Under Unreal UX terms, closing the Color Picker (via the UI) is the same as confirming it since we've been live updating
	// the color. The track already has the latest color change so we undo the change before calling Modify so that Undo sets us
	// to the original color. This is also called in the event of pressing cancel so we need to detect if it was canceled or not.
	if (!bColorPickerWasCancelled)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "SetKeyColor", "Set Key Color"));

		float R, G, B, A = 0.f;
		RChannel->Evaluate(KeyTime, R);
		GChannel->Evaluate(KeyTime, G);
		BChannel->Evaluate(KeyTime, B);
		AChannel->Evaluate(KeyTime, A);

		UpdateOrAddKey(RChannel, KeyTime, InitialColor.R);
		UpdateOrAddKey(GChannel, KeyTime, InitialColor.G);
		UpdateOrAddKey(BChannel, KeyTime, InitialColor.B);
		UpdateOrAddKey(AChannel, KeyTime, InitialColor.A);

		Section->Modify();

		UpdateOrAddKey(RChannel, KeyTime, R);
		UpdateOrAddKey(GChannel, KeyTime, G);
		UpdateOrAddKey(BChannel, KeyTime, B);
		UpdateOrAddKey(AChannel, KeyTime, A);

		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer)
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

void FMovieSceneKeyColorPicker::OnColorPickerCancelled(FLinearColor NewColor, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> WeakSequencer)
{
	bColorPickerWasCancelled = true;
	
	Section->Modify();

	// Restore the original color. No transaction will be created when the OnColorPickerClosed callback is called.
	UpdateOrAddKey(RChannel, KeyTime, InitialColor.R);
	UpdateOrAddKey(GChannel, KeyTime, InitialColor.G);
	UpdateOrAddKey(BChannel, KeyTime, InitialColor.B);
	UpdateOrAddKey(AChannel, KeyTime, InitialColor.A);

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}
