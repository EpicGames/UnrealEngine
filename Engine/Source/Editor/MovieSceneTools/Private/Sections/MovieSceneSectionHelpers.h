// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Containers/ArrayView.h"

#define UE_API MOVIESCENETOOLS_API

struct FKeyHandle;
struct FTimeToPixel;
struct FMovieSceneFloatChannel;
class ISequencer;
class UMovieSceneSection;

class MovieSceneSectionHelpers
{
public:

	/** Consolidate color curves for all track sections. */
	static UE_API void ConsolidateColorCurves(TArray< TTuple<float, FLinearColor> >& OutColorKeys, const FLinearColor& DefaultColor, TArrayView<const FMovieSceneFloatChannel* const> ColorChannels, const FTimeToPixel& TimeConverter);
};

class FMovieSceneKeyColorPicker
{
public:
	UE_API FMovieSceneKeyColorPicker(UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, const TArray<FKeyHandle>& KeyHandles, TWeakPtr<ISequencer> InSequencer);

private:
		
	UE_API void OnColorPickerPicked(FLinearColor NewColor, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> InSequencer);
	UE_API void OnColorPickerClosed(const TSharedRef<SWindow>& Window, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> InSequencer);
	UE_API void OnColorPickerCancelled(FLinearColor NewColor, UMovieSceneSection* Section, FMovieSceneFloatChannel* RChannel, FMovieSceneFloatChannel* GChannel, FMovieSceneFloatChannel* BChannel, FMovieSceneFloatChannel* AChannel, TWeakPtr<ISequencer> InSequencer);

private:

	static UE_API FFrameNumber KeyTime;
	static UE_API FLinearColor InitialColor;
	static UE_API bool bColorPickerWasCancelled;
};

#undef UE_API
