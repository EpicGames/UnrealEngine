// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/ThumbnailSection.h"
#include "AnimatedRange.h"
#include "Rendering/DrawElements.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "Modules/ModuleManager.h"
#include "Application/ThrottleManager.h"
#include "Widgets/Layout/SBox.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "LevelEditorViewport.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "IVREditorModule.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "TrackEditorThumbnail/TrackThumbnailUtils.h"

#define LOCTEXT_NAMESPACE "FThumbnailSection"


namespace ThumbnailSectionConstants
{
	const uint32 ThumbnailHeight = 90;
	const uint32 TrackWidth = 90;
	const float SectionGripSize = 4.0f;
}


/* FThumbnailSection structors
 *****************************************************************************/

FThumbnailSection::FThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, IViewportThumbnailClient* InViewportThumbanilClient, UMovieSceneSection& InSection)
	: Section(&InSection)
	, SequencerPtr(InSequencer)
	, ThumbnailCache(InThumbnailPool, InViewportThumbanilClient)
	, AdditionalDrawEffect(ESlateDrawEffect::None)
	, TimeSpace(ETimeSpace::Global)
{
	WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
	RedrawThumbnailDelegateHandle = GetMutableDefault<UMovieSceneUserThumbnailSettings>()->OnForceRedraw().AddRaw(this, &FThumbnailSection::RedrawThumbnails);
}


FThumbnailSection::FThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, ICustomThumbnailClient* InCustomThumbnailClient, UMovieSceneSection& InSection)
	: Section(&InSection)
	, SequencerPtr(InSequencer)
	, ThumbnailCache(InThumbnailPool, InCustomThumbnailClient)
	, AdditionalDrawEffect(ESlateDrawEffect::None)
	, TimeSpace(ETimeSpace::Global)
{
	WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
	RedrawThumbnailDelegateHandle = GetMutableDefault<UMovieSceneUserThumbnailSettings>()->OnForceRedraw().AddRaw(this, &FThumbnailSection::RedrawThumbnails);
}


FThumbnailSection::~FThumbnailSection()
{
	GetMutableDefault<UMovieSceneUserThumbnailSettings>()->OnForceRedraw().Remove(RedrawThumbnailDelegateHandle);
}

void FThumbnailSection::RedrawThumbnails()
{
	ThumbnailCache.ForceRedraw();
}


/* ISequencerSection interface
 *****************************************************************************/

TSharedRef<SWidget> FThumbnailSection::GenerateSectionWidget()
{
	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding_Lambda([&]() { return GetContentPadding(); })
		[
			SAssignNew(NameWidget, SInlineEditableTextBlock)
				.ToolTipText(CanRename() ? LOCTEXT("RenameThumbnail", "Click or hit F2 to rename") : FText::GetEmpty())
				.Text(this, &FThumbnailSection::HandleThumbnailTextBlockText)
				.ShadowOffset(FVector2D(1,1))
				.OnTextCommitted(this, &FThumbnailSection::HandleThumbnailTextBlockTextCommitted)
				.IsReadOnly(!CanRename())
				.Visibility(EVisibility::Hidden) // Starts hidden and is made visible when renaming is invoked
				.OnEnterEditingMode(this, &FThumbnailSection::OnEnterEditingMode)
				.OnExitEditingMode(this, &FThumbnailSection::OnExitEditingMode)
		];
}

EVisibility FThumbnailSection::GetRenameVisibility() const
{
	if (NameWidget.IsValid() && NameWidget->IsInEditMode())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

void FThumbnailSection::OnEnterEditingMode()
{
	if (NameWidget.IsValid())
	{
		NameWidget->SetVisibility(EVisibility::Visible);
	}
}

void FThumbnailSection::OnExitEditingMode()
{
	if (NameWidget.IsValid())
	{
		NameWidget->SetVisibility(EVisibility::Hidden);
	}
}

void FThumbnailSection::EnterRename()
{
	if (NameWidget.IsValid())
	{
		NameWidget->EnterEditingMode();
	}
}

void FThumbnailSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ViewMenuText", "View"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ThumbnailsMenu", "Thumbnails"),
			FText(),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
			{
				BuildThumbnailsMenu(InMenuBuilder);
			})
		);
	}
	MenuBuilder.EndSection();
}

void FThumbnailSection::BuildThumbnailsMenu(FMenuBuilder& InMenuBuilder)
{
	const TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FText CurrentTime = FText::FromString(Sequencer->GetNumericTypeInterface()->ToString(Sequencer->GetLocalTime().Time.GetFrame().Value));

	InMenuBuilder.BeginSection(TEXT("Thumbnails"), LOCTEXT("ThumbnailsMenuSection", "Thumbnails"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("RefreshText", "Refresh"),
			LOCTEXT("RefreshTooltip", "Refresh this section's thumbnails"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &FThumbnailSection::RedrawThumbnails))
		);
		InMenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("SetSingleTime", "Set Thumbnail Time To {0}"), CurrentTime),
			LOCTEXT("SetSingleTimeTooltip", "Defines the time at which this section should draw its single thumbnail to the current cursor position"),
			FSlateIcon(),
			FUIAction(
			FExecuteAction::CreateLambda([this]
				{
					const TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
					if (!Sequencer.IsValid())
					{
						return;
					}

					SetSingleTime(Sequencer->GetLocalTime().AsSeconds());
					GetMutableDefault<UMovieSceneUserThumbnailSettings>()->bDrawSingleThumbnails = true;
					GetMutableDefault<UMovieSceneUserThumbnailSettings>()->SaveConfig();
				})
			)
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("RefreshAllText", "Refresh All"),
			LOCTEXT("RefreshAllTooltip", "Refresh all sections' thumbnails"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]
				{
					GetDefault<UMovieSceneUserThumbnailSettings>()->BroadcastRedrawThumbnails();
				}))
		);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bAllowSearch = false;
		Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		TSharedRef<IDetailsView> DetailView = PropertyModule.CreateDetailView(Args);
		DetailView->SetObject(GetMutableDefault<UMovieSceneUserThumbnailSettings>());
		InMenuBuilder.AddWidget(DetailView, FText(), true);
	}
	InMenuBuilder.EndSection();
}

void FThumbnailSection::BuildSectionSidebarMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	BuildThumbnailsMenu(MenuBuilder);
}

float FThumbnailSection::GetSectionGripSize() const
{
	return ThumbnailSectionConstants::SectionGripSize;
}


float FThumbnailSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	auto* Settings = GetDefault<UMovieSceneUserThumbnailSettings>();
	if (Settings->bDrawThumbnails)
	{
		return GetDefault<UMovieSceneUserThumbnailSettings>()->ThumbnailSize.Y;
	}
	else
	{
		return FAppStyle::GetFontStyle("NormalFont").Size + 8.f;
	}
}


UMovieSceneSection* FThumbnailSection::GetSectionObject() 
{
	return Section;
}


FText FThumbnailSection::GetSectionTitle() const
{
	return FText::GetEmpty();
}


int32 FThumbnailSection::OnPaintSection( FSequencerSectionPainter& InPainter ) const
{
	if (!GetDefault<UMovieSceneUserThumbnailSettings>()->bDrawThumbnails)
	{
		return InPainter.LayerId;
	}

	static const float SectionThumbnailPadding = 4.f;

	ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	int32 LayerId = InPainter.LayerId;

	const FGeometry& HeaderGeometry = InPainter.HeaderGeometry;

	// @todo Sequencer: Need a way to visualize the key here

	const TRange<double> VisibleRange = GetVisibleRange();
	const TRange<double> GenerationRange = GetTotalRange();

	const float TimePerPx = GenerationRange.Size<double>() / HeaderGeometry.GetLocalSize().X;

	const FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
	const double SectionEaseInDuration = TickResolution.AsSeconds(Section->Easing.GetEaseInDuration()) / TimePerPx;
	const double SectionEaseOutDuration = TickResolution.AsSeconds(Section->Easing.GetEaseOutDuration()) / TimePerPx;

	const FSlateRect ThumbnailClipRect = HeaderGeometry.GetLayoutBoundingRect()
		.InsetBy(FMargin(SectionThumbnailPadding, 0.f))
		.InsetBy(FMargin(SectionEaseInDuration, 0.f, SectionEaseOutDuration, 0.f))
		.IntersectionWith(InPainter.SectionClippingRect);

	for (const TSharedPtr<FTrackEditorThumbnail>& Thumbnail : ThumbnailCache.GetThumbnails())
	{
		const float Fade = Thumbnail->bHasFinishedDrawing ? Thumbnail->GetFadeInCurve() : 1.f;
		if (Fade >= 1.f)
		{
			continue;
		}

		FIntPoint ThumbnailRTSize   = Thumbnail->GetSize();
		FIntPoint ThumbnailCropSize = Thumbnail->GetDesiredSize();
		
		const float ThumbnailScale       = float(ThumbnailCropSize.Y) / ThumbnailRTSize.Y;
		const float HorizontalCropOffset = (ThumbnailRTSize.X*ThumbnailScale - ThumbnailCropSize.X) * 0.5f;

		// Calculate the paint geometry for this thumbnail
		TOptional<double> SingleReferenceFrame = ThumbnailCache.GetSingleReferenceFrame();

		// Single thumbnails are always drawn at the start of the section, clamped to the visible range
		// Thumbnail sequences draw relative to their actual position in the sequence/section
		const float PositionX = SingleReferenceFrame.IsSet()
			? FMath::Max(float(VisibleRange.GetLowerBoundValue() - GenerationRange.GetLowerBoundValue()) / TimePerPx, 0.f) + SectionThumbnailPadding
			: (Thumbnail->GetTimeRange().GetLowerBoundValue() - GenerationRange.GetLowerBoundValue()) / TimePerPx;

		const float PositionY = (HeaderGeometry.GetLocalSize().Y - ThumbnailCropSize.Y)*.5f;

		FPaintGeometry PaintGeometry = HeaderGeometry.ToPaintGeometry(
			ThumbnailRTSize,
			FSlateLayoutTransform(ThumbnailScale, FVector2D(PositionX-HorizontalCropOffset, PositionY))
		);

PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO: XR Creative Framework relevant?
		if (IVREditorModule::Get().IsVREditorModeActive())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// In VR editor every widget is in the world and gamma corrected by the scene renderer.  Thumbnails will have already been gamma
			// corrected and so they need to be reversed
			DrawEffects |= ESlateDrawEffect::ReverseGamma;
		}
		else
		{
			DrawEffects |= ESlateDrawEffect::NoGamma;
		}

		if (Thumbnail->bIgnoreAlpha)
		{
			DrawEffects |= ESlateDrawEffect::IgnoreTextureAlpha;
		}

		FGeometry ClipGeometry = HeaderGeometry.MakeChild(
			ThumbnailCropSize,
			FSlateLayoutTransform(
				FVector2D(PositionX, PositionY)
			)
		);

		FSlateRect ThisThumbnailClipRect = ThumbnailClipRect.IntersectionWith(ClipGeometry.GetLayoutBoundingRect());

		FSlateClippingZone ClippingZone(ThisThumbnailClipRect);
		InPainter.DrawElements.PushClip(ClippingZone);

		FSlateDrawElement::MakeViewport(
			InPainter.DrawElements,
			LayerId,
			PaintGeometry,
			Thumbnail,
			DrawEffects | AdditionalDrawEffect,
			FLinearColor(1.f, 1.f, 1.f, 1.f - Fade)
		);

		InPainter.DrawElements.PopClip();
	}

	return LayerId + 2;
}

TRange<double> FThumbnailSection::GetVisibleRange() const
{
	const FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
	TRange<double> GlobalVisibleRange = SequencerPtr.Pin()->GetViewRange();
	TRange<double> SectionRange = Section->GetRange() / TickResolution;

	if (TimeSpace == ETimeSpace::Global)
	{
		return GlobalVisibleRange;
	}

	TRange<double> Intersection = TRange<double>::Intersection(GlobalVisibleRange, SectionRange);
	return TRange<double>(
		Intersection.GetLowerBoundValue() - SectionRange.GetLowerBoundValue(),
		Intersection.GetUpperBoundValue() - SectionRange.GetLowerBoundValue()
	);
}

TRange<double> FThumbnailSection::GetTotalRange() const
{
	TRange<FFrameNumber> SectionRange   = Section->GetRange();
	FFrameRate           TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

	if (TimeSpace == ETimeSpace::Global)
	{
		return SectionRange / TickResolution;
	}
	else
	{
		const bool bHasDiscreteSize = SectionRange.GetLowerBound().IsClosed() && SectionRange.GetUpperBound().IsClosed();
		TRangeBound<double> UpperBound = bHasDiscreteSize
			? TRangeBound<double>::Exclusive(FFrameNumber(UE::MovieScene::DiscreteSize(SectionRange)) / TickResolution)
			: TRangeBound<double>::Open();

		return TRange<double>(0, UpperBound);
	}
}

void FThumbnailSection::Tick(const FGeometry& AllottedGeometry, const FGeometry& ParentGeometry, const double InCurrentTime, const float InDeltaTime)
{
	using namespace UE::Sequencer;

	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer && FSlateThrottleManager::Get().IsAllowingExpensiveTasks() && GetDefault<UMovieSceneUserThumbnailSettings>()->bDrawThumbnails)
	{
		const UMovieSceneUserThumbnailSettings* Settings = GetDefault<UMovieSceneUserThumbnailSettings>();

		FViewDensityInfo ViewDensity = Sequencer->GetViewModel()->GetViewDensity();
		const float Height = GetSectionHeight(ViewDensity);

		FIntPoint AllocatedSize = AllottedGeometry.GetLocalSize().IntPoint();
		AllocatedSize.X = FMath::Max(AllocatedSize.X, 1);
		AllocatedSize.Y = FMath::RoundToInt(Height);

		ThumbnailCache.Update(GetTotalRange(), GetVisibleRange(), AllocatedSize, Settings->ThumbnailSize, Settings->Quality, InCurrentTime);
	}
}


FViewportThumbnailSection::FViewportThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, UMovieSceneSection& InSection)
	: FThumbnailSection(InSequencer, InThumbnailPool, this, InSection)
{
}


void FViewportThumbnailSection::PreDraw(FTrackEditorThumbnail& Thumbnail)
{
	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer.IsValid())
	{
		UE::MoveSceneTools::PreDrawThumbnailSetupSequencer(*Sequencer, Thumbnail.GetEvalPosition());
	}
}


void FViewportThumbnailSection::PostDraw(FTrackEditorThumbnail& Thumbnail)
{
	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer.IsValid())
	{
		Thumbnail.SetupFade(Sequencer->GetSequencerWidget());
		UE::MoveSceneTools::PostDrawThumbnailCleanupSequencer(*Sequencer);
	}
}

#undef LOCTEXT_NAMESPACE
