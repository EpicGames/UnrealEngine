// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/TransformPropertySection.h"

#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "ScopedTransaction.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

#define LOCTEXT_NAMESPACE "FTransformSection"

void FTransformSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding)
{
	auto MakeUIAction = [this, InObjectBinding](EMovieSceneTransformChannel ChannelsToToggle)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, InObjectBinding, ChannelsToToggle]
				{
					UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(WeakSection.Get());
					TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

					FScopedTransaction Transaction(LOCTEXT("SetActiveChannelsTransaction", "Set Active Channels"));
					TransformSection->Modify();
					EMovieSceneTransformChannel Channels = TransformSection->GetMask().GetChannels();

					if (EnumHasAllFlags(Channels, ChannelsToToggle) || (Channels & ChannelsToToggle) == EMovieSceneTransformChannel::None)
					{
						TransformSection->SetMask(TransformSection->GetMask().GetChannels() ^ ChannelsToToggle);
					}
					else
					{
						TransformSection->SetMask(TransformSection->GetMask().GetChannels() | ChannelsToToggle);
					}

					// Restore pre-animated state for the bound objects so that inactive channels will return to their default values.
					for (TWeakObjectPtr<> WeakObject : SequencerPtr->FindBoundObjects(InObjectBinding, SequencerPtr->GetFocusedTemplateID()))
					{
						if (UObject* Object = WeakObject.Get())
						{
							SequencerPtr->RestorePreAnimatedState();
						}
					}

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this, ChannelsToToggle]
			{
				const UMovieScene3DTransformSection* const TransformSection = CastChecked<UMovieScene3DTransformSection>(WeakSection.Get());
				if (!IsValid(TransformSection))
				{
					return ECheckBoxState::Unchecked;
				}

				const EMovieSceneTransformChannel Channels = TransformSection->GetMask().GetChannels();
				if (EnumHasAllFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Checked;
				}
				else if (EnumHasAnyFlags(Channels, ChannelsToToggle))
				{
					return ECheckBoxState::Undetermined;
				}
				return ECheckBoxState::Unchecked;
			})
		);
	};

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TransformChannelsText", "Active Channels"));
	{
		const EAxisList::Type XAxis = EAxisList::Forward;
		const EAxisList::Type YAxis = EAxisList::Left;
		const EAxisList::Type ZAxis = EAxisList::Up;

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllTranslation", "Translation"), LOCTEXT("AllTranslation_ToolTip", "Causes this section to affect the translation of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				const int32 NumMenuItems = 3;
				TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
					[&SubMenuBuilder, MakeUIAction, XAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(XAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationX), NAME_None, EUserInterfaceActionType::ToggleButton);

					},
					[&SubMenuBuilder, MakeUIAction, YAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(YAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationY), NAME_None, EUserInterfaceActionType::ToggleButton);
					},
					[&SubMenuBuilder, MakeUIAction, ZAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(ZAxis),
						FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationZ), NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				};

				const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
				for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
				{
					const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
					MenuConstructors[SwizzledComponentIndex]();
				}
			}),
			MakeUIAction(EMovieSceneTransformChannel::Translation),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllRotation", "Rotation"), LOCTEXT("AllRotation_ToolTip", "Causes this section to affect the rotation of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationX", "Roll"), LOCTEXT("RotationX_ToolTip", "Causes this section to affect the roll channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationX), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationY", "Pitch"), LOCTEXT("RotationY_ToolTip", "Causes this section to affect the pitch channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationY), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationZ", "Yaw"), LOCTEXT("RotationZ_ToolTip", "Causes this section to affect the yaw channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationZ), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieSceneTransformChannel::Rotation),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllScale", "Scale"), LOCTEXT("AllScale_ToolTip", "Causes this section to affect the scale of the transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){
				const int32 NumMenuItems = 3;
				TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
					[&SubMenuBuilder, MakeUIAction, XAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(XAxis),
							FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleX), NAME_None, EUserInterfaceActionType::ToggleButton);

					},
					[&SubMenuBuilder, MakeUIAction, YAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(YAxis),
							FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleY), NAME_None, EUserInterfaceActionType::ToggleButton);
					},
					[&SubMenuBuilder, MakeUIAction, ZAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(ZAxis),
						FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleZ), NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				};

				const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
				for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
				{
					const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
					MenuConstructors[SwizzledComponentIndex]();
				}
			}),
			MakeUIAction(EMovieSceneTransformChannel::Scale),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Weight", "Weight"), LOCTEXT("Weight_ToolTip", "Causes this section to be applied with a user-specified weight curve"),
			FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::Weight), NAME_None, EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

namespace UE::Private
{
	static const FName ConstraintsSectionName(TEXT("Constraints"));
}

bool FTransformSection::RequestDeleteCategory(const TArray<FName>& CategoryNamePaths)
{
	if (CategoryNamePaths.IsEmpty())
	{
		return false;
	}
	
	UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	if (CategoryNamePaths[0] == UE::Private::ConstraintsSectionName)
	{
		constexpr bool bDoNotCompensate = false;
		if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
		{
			const FScopedTransaction Transaction(LOCTEXT("DeleteConstraintChannels", "Delete Constraint Channels"));
			if (TransformSection->TryModify())
			{
				if (TransformSection->RemoveAllConstraints(World, bDoNotCompensate))
				{
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
				return true;
			}
		}
		return false;
	}
	
	const FScopedTransaction Transaction( LOCTEXT( "DeleteTransformCategory", "Delete transform category" ) );

	if (TransformSection->TryModify())
	{
		FName CategoryName = CategoryNamePaths[CategoryNamePaths.Num()-1];
		
		EMovieSceneTransformChannel Channel = TransformSection->GetMask().GetChannels();
		EMovieSceneTransformChannel ChannelToRemove = TransformSection->GetMaskByName(CategoryName).GetChannels();

		Channel &= ~ChannelToRemove;

		TransformSection->SetMask(Channel);
			
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return false;
}

bool FTransformSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePaths)
{
	if (KeyAreaNamePaths.IsEmpty())
	{
		return false;
	}

	UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	
	if (KeyAreaNamePaths.Num() == 2 && KeyAreaNamePaths[0] == UE::Private::ConstraintsSectionName)
	{
		UTickableConstraint* Constraint = TransformSection->FindConstraint(KeyAreaNamePaths[1]);
		if (!Constraint)
		{
			return false;
		}

		if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
		{
			const FScopedTransaction Transaction(LOCTEXT("DeleteConstraintChannel", "Delete Constraint Channel"));
			if (TransformSection->TryModify())
			{
				FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
				if (Controller.RemoveConstraint(Constraint))
				{
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
				return true;
			}
		}
		
		return false;
	}
	
	const FScopedTransaction Transaction( LOCTEXT( "DeleteTransformChannel", "Delete transform channel" ) );

	if (TransformSection->TryModify())
	{
		// Only delete the last key area path which is the channel. ie. TranslationX as opposed to Translation
		FName KeyAreaName = KeyAreaNamePaths[KeyAreaNamePaths.Num()-1];

		EMovieSceneTransformChannel Channel = TransformSection->GetMask().GetChannels();
		EMovieSceneTransformChannel ChannelToRemove = TransformSection->GetMaskByName(KeyAreaName).GetChannels();

		Channel &= ~ChannelToRemove;

		TransformSection->SetMask(Channel);
					
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE