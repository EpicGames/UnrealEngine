// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ISequencerSection.h"
#include "Templates/SharedPointer.h"

class FMenuBuilder;
class FName;
class ISequencer;
class UMovieSceneSection;
struct FGuid;

/**
 * An implementation of transform property sections.
 */
class FTransformSection : public FSequencerSection
{
public:

	/**
	* Creates a new transform property section.
	*
	* @param InSection The section object which is being displayed and edited.
	* @param InSequencer The sequencer which is controlling this property section.
	*/
	FTransformSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSection), WeakSequencer(InSequencer)
	{
	}

public:

	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding) override;

	//~ ISequencerSection interface

	virtual bool RequestDeleteCategory(const TArray<FName>& CategoryNamePath) override;
	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;

protected:

	/** The sequencer which is controlling this section. */
	TWeakPtr<ISequencer> WeakSequencer;
};
