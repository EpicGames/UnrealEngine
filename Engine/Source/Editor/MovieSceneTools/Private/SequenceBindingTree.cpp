// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceBindingTree.h"

#include "CoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#define LOCTEXT_NAMESPACE "MovieSceneObjectBindingIDPicker"

/** Stack of sequence IDs from parent to child */
struct FSequenceBindingTree::FSequenceIDStack
{
	/** Get the current accumulated sequence ID */
	FMovieSceneSequenceID GetCurrent() const
	{
		FMovieSceneSequenceID ID = MovieSceneSequenceID::Root;
		for (int32 Index = IDs.Num() - 1; Index >= 0; --Index)
		{
			ID = ID.AccumulateParentID(IDs[Index]);
		}
		return ID;
	}

	/** Push a sequence ID onto the stack */
	void Push(FMovieSceneSequenceID InSequenceID) { IDs.Add(InSequenceID); }

	/** Pop the last sequence ID off the stack */
	void Pop() { IDs.RemoveAt(IDs.Num() - 1, EAllowShrinking::No); }
	
private:
	TArray<FMovieSceneSequenceID> IDs;
};

bool FSequenceBindingTree::ConditionalRebuild(UMovieSceneSequence* InSequence, FObjectKey InActiveSequence, FMovieSceneSequenceID InActiveSequenceID)
{
	struct FInternal
	{
		static bool IsOutOfDate(UMovieSceneSequence* ThisSequence, TMap<FObjectKey, FGuid>* InCachedSequenceSignatures)
		{
			FGuid* ExistingSignature = InCachedSequenceSignatures->Find(ThisSequence);
			if (!ExistingSignature || *ExistingSignature != ThisSequence->GetSignature())
			{
				return true;
			}

			UMovieScene* ThisMovieScene = ThisSequence->GetMovieScene();
			for (const UMovieSceneTrack* Track : ThisMovieScene->GetTracks())
			{
				const UMovieSceneSubTrack* SubTrack = Cast<const UMovieSceneSubTrack>(Track);
				if (SubTrack)
				{
					for (UMovieSceneSection* Section : SubTrack->GetAllSections())
					{
						UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
						UMovieSceneSequence* SubSequence = SubSection ? SubSection->GetSequence() : nullptr;
						if (SubSequence && Section->IsActive())
						{
							if (IsOutOfDate(SubSequence, InCachedSequenceSignatures))
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		}
	};
	

	if (FInternal::IsOutOfDate(InSequence, &CachedSequenceSignatures))
	{
		ForceRebuild(InSequence, InActiveSequence, InActiveSequenceID);
		return true;
	}
	return false;
}

void FSequenceBindingTree::ForceRebuild(UMovieSceneSequence* InSequence, FObjectKey InActiveSequence, FMovieSceneSequenceID InActiveSequenceID)
{
	using namespace UE::MovieScene;

	CachedSequenceSignatures.Empty();
	bIsEmpty = true;

	// Reset state
	ActiveSequenceID = InActiveSequenceID;
	ActiveSequence = InActiveSequence;
	Hierarchy.Reset();

	ActiveSequenceNode = nullptr;

	// Create a node for the root sequence
	FFixedObjectBindingID RootSequenceID = FFixedObjectBindingID(FGuid(), MovieSceneSequenceID::Root);
	TSharedRef<FSequenceBindingNode> RootSequenceNode = MakeShared<FSequenceBindingNode>(FText(), RootSequenceID, FSlateIcon());
	Hierarchy.Add(RootSequenceID, RootSequenceNode);

	TopLevelNode = RootSequenceNode;

	if (InSequence)
	{
		RootSequenceNode->DisplayString = FText::FromString(InSequence->GetName());
		RootSequenceNode->Icon = FSlateIconFinder::FindIconForClass(InSequence->GetClass());

		// Build the tree
		FSequenceIDStack SequenceIDStack;
		Build(InSequence, SequenceIDStack);

		// Sort the tree
		Sort(RootSequenceNode);

		// We don't show cross-references to the same sequence since this would result in erroneous mixtures of absolute and local bindings
		if (ActiveSequenceNode.IsValid() && ActiveSequenceNode != RootSequenceNode)
		{
			// Remove it from its parent, and put it at the root for quick access
			TSharedPtr<FSequenceBindingNode> ActiveParent = Hierarchy.FindChecked(ActiveSequenceNode->ParentID);
			ActiveParent->Children.Remove(ActiveSequenceNode.ToSharedRef());

			// Make a new top level node (with an invalid ID)
			FFixedObjectBindingID TopLevelID = FFixedObjectBindingID(FGuid(), MovieSceneSequenceID::Invalid);
			TopLevelNode = MakeShared<FSequenceBindingNode>(FText(), TopLevelID, FSlateIcon());

			// Override the display string and icon
			ActiveSequenceNode->DisplayString = LOCTEXT("ThisSequenceText", "This Sequence");
			ActiveSequenceNode->Icon = FSlateIcon();

			TopLevelNode->Children.Add(ActiveSequenceNode.ToSharedRef());
			TopLevelNode->Children.Add(RootSequenceNode);
		}
	}
}

void FSequenceBindingTree::Sort(TSharedRef<FSequenceBindingNode> Node)
{
	Node->Children.Sort(
		[](TSharedRef<FSequenceBindingNode> A, TSharedRef<FSequenceBindingNode> B)
		{
			// Sort shots first
			if (A->BindingID.Guid.IsValid() != B->BindingID.Guid.IsValid())
			{
				return !A->BindingID.Guid.IsValid();
			}
			return A->DisplayString.CompareToCaseIgnored(B->DisplayString) < 0;
		}
	);

	for (TSharedRef<FSequenceBindingNode> Child : Node->Children)
	{
		Sort(Child);
	}
}

void FSequenceBindingTree::Build(UMovieSceneSequence* InSequence, FSequenceIDStack& SequenceIDStack)
{
	using namespace UE::MovieScene;

	check(InSequence);

	UMovieScene* MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}
	
	if (ActiveSequence == InSequence)
	{
		// Don't allow cross-references to the same sequence (ie, re-entrant references)
		if (SequenceIDStack.GetCurrent() != ActiveSequenceID)
		{
			return;
		}

		// Keep track of the active sequence node
		ActiveSequenceNode = Hierarchy.FindChecked(FFixedObjectBindingID(FGuid(), SequenceIDStack.GetCurrent()));
	}

	// Iterate all sub sections
	for (const UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		const UMovieSceneSubTrack* SubTrack = Cast<const UMovieSceneSubTrack>(Track);
		if (SubTrack)
		{
			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				UMovieSceneSequence* SubSequence = SubSection ? SubSection->GetSequence() : nullptr;
				if (SubSequence && Section->IsActive())
				{
					// Hold onto the current parent ID before adding our ID onto the stack
					FMovieSceneSequenceID ParentID = SequenceIDStack.GetCurrent();
					SequenceIDStack.Push(SubSection->GetSequenceID());
					
					FFixedObjectBindingID CurrentID = FFixedObjectBindingID(FGuid(), SequenceIDStack.GetCurrent());

					UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section);
					FText DisplayString = ShotSection ? FText::FromString(ShotSection->GetShotDisplayName()) : FText::FromName(SubSequence->GetFName());
					FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), ShotSection ? "Sequencer.Tracks.CinematicShot" : "Sequencer.Tracks.Sub");
					
					TSharedRef<FSequenceBindingNode> NewNode = MakeShared<FSequenceBindingNode>(DisplayString, CurrentID, Icon);
					ensure(!Hierarchy.Contains(CurrentID));
					Hierarchy.Add(CurrentID, NewNode);

					EnsureParent(FGuid(), MovieScene, ParentID)->AddChild(NewNode);

					Build(SubSequence, SequenceIDStack);

					SequenceIDStack.Pop();
				}
			}
		}
	}

	FMovieSceneSequenceID CurrentSequenceID = SequenceIDStack.GetCurrent();

	CachedSequenceSignatures.Add(InSequence, InSequence->GetSignature());

	// Add all spawnables first (since possessables can be children of spawnables)
	int32 SpawnableCount = MovieScene->GetSpawnableCount();
	for (int32 Index = 0; Index < SpawnableCount; ++Index)
	{
		const FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(Index);
		
		FFixedObjectBindingID ID = FFixedObjectBindingID(Spawnable.GetGuid(), CurrentSequenceID);

		FSlateIcon Icon;
		if (const UObject* ObjectTemplate = Spawnable.GetObjectTemplate())
		{
			Icon = FSlateIconFinder::FindIconForClass(ObjectTemplate->GetClass());
		}
		else
		{
			Icon = FSlateIconFinder::FindIcon("Sequencer.InvalidSpawnableIcon");
		}

		TSharedRef<FSequenceBindingNode> NewNode = MakeShared<FSequenceBindingNode>(MovieScene->GetObjectDisplayName(Spawnable.GetGuid()), ID, Icon);
		NewNode->bIsSpawnable = true;

		EnsureParent(FGuid(), MovieScene, CurrentSequenceID)->AddChild(NewNode);
		ensure(!Hierarchy.Contains(ID));
		Hierarchy.Add(ID, NewNode);

		bIsEmpty = false;
	}

	// Add all possessables
	const int32 PossessableCount = MovieScene->GetPossessableCount();
	for (int32 Index = 0; Index < PossessableCount; ++Index)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
		if (InSequence->CanRebindPossessable(Possessable))
		{
			FFixedObjectBindingID ID = FFixedObjectBindingID(Possessable.GetGuid(), CurrentSequenceID);

			FSlateIcon Icon = FSlateIconFinder::FindIconForClass(Possessable.GetPossessedObjectClass());
			TSharedRef<FSequenceBindingNode> NewNode = MakeShared<FSequenceBindingNode>(MovieScene->GetObjectDisplayName(Possessable.GetGuid()), ID, Icon);

			EnsureParent(Possessable.GetParent(), MovieScene, CurrentSequenceID)->AddChild(NewNode);
			ensure(!Hierarchy.Contains(ID));
			Hierarchy.Add(ID, NewNode);

			bIsEmpty = false;
		}
	}
}

TSharedRef<FSequenceBindingNode> FSequenceBindingTree::EnsureParent(const FGuid& InParentGuid, UMovieScene* InMovieScene, FMovieSceneSequenceID SequenceID)
{
	using namespace UE::MovieScene;

	FFixedObjectBindingID ParentPtr = FFixedObjectBindingID(InParentGuid, SequenceID);

	// If the node already exists
	TSharedPtr<FSequenceBindingNode> Parent = Hierarchy.FindRef(ParentPtr);
	if (Parent.IsValid())
	{
		return Parent.ToSharedRef();
	}

	// Non-object binding nodes should have already been added externally to EnsureParent
	check(InParentGuid.IsValid());

	// Need to add it
	FGuid AddToGuid;
	if (FMovieScenePossessable* GrandParentPossessable = InMovieScene->FindPossessable(InParentGuid))
	{
		AddToGuid = GrandParentPossessable->GetGuid();
	}

	// Deduce the icon for the node
	FSlateIcon Icon;
	bool bIsSpawnable = false;
	{
		const FMovieScenePossessable* Possessable = InMovieScene->FindPossessable(InParentGuid);
		const FMovieSceneSpawnable* Spawnable = Possessable ? nullptr : InMovieScene->FindSpawnable(InParentGuid);
		if (Possessable)
		{
			Icon = FSlateIconFinder::FindIconForClass(Possessable->GetPossessedObjectClass());
		}
		else if (Spawnable)
		{
			if (Spawnable->GetObjectTemplate())
			{
				Icon = FSlateIconFinder::FindIconForClass(Spawnable->GetObjectTemplate()->GetClass());
			}
			else
			{
				Icon = FSlateIconFinder::FindIcon("Sequencer.InvalidSpawnableIcon");
			}
		}

		bIsSpawnable = Spawnable != nullptr;
	}

	TSharedRef<FSequenceBindingNode> NewNode = MakeShared<FSequenceBindingNode>(InMovieScene->GetObjectDisplayName(InParentGuid), ParentPtr, Icon);
	NewNode->bIsSpawnable = bIsSpawnable;
	
	ensure(!Hierarchy.Contains(ParentPtr));
	Hierarchy.Add(ParentPtr, NewNode);
	bIsEmpty = false;

	EnsureParent(AddToGuid, InMovieScene, SequenceID)->AddChild(NewNode);

	return NewNode;
}

#undef LOCTEXT_NAMESPACE
