// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditorThumbnail/TrackEditorThumbnail.h"
#include "Modules/ModuleManager.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "ShowFlags.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "Slate/SceneViewport.h"
#include "GlobalShader.h"
#include "ScreenRendering.h"
#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "PipelineStateCache.h"
#include "CommonRenderResources.h"
#include "LegacyScreenPercentageDriver.h"
#include "CanvasTypes.h"
#include "EngineModule.h"
#include "SceneViewExtension.h"
#include "TrackEditorThumbnail/TrackThumbnailUtils.h"

namespace TrackEditorThumbnailConstants
{
	const double ThumbnailFadeInDuration = 0.25f;
}


class FThumbnailViewportClient
	: public FLevelEditorViewportClient
{
public:

	FThumbnailViewportClient() : FLevelEditorViewportClient(nullptr) {}

	float CurrentWorldTime, DeltaWorldTime;

	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override
	{
		FSceneView* View = FLevelEditorViewportClient::CalcSceneView(ViewFamily, StereoViewIndex);

		// Artificially set the world times so that graphics settings apply correctly (we don't tick the world when rendering thumbnails)
		ViewFamily->Time = FGameTime::CreateDilated(ViewFamily->Time.GetRealTimeSeconds(), ViewFamily->Time.GetDeltaRealTimeSeconds(), CurrentWorldTime, DeltaWorldTime);

		View->FinalPostProcessSettings.bOverride_AutoExposureSpeedDown = View->FinalPostProcessSettings.bOverride_AutoExposureSpeedUp = true;
		View->FinalPostProcessSettings.AutoExposureSpeedDown = View->FinalPostProcessSettings.AutoExposureSpeedUp = 0.02f;
		return View;
	}

};


/* FTrackEditorThumbnail structors
 *****************************************************************************/

FTrackEditorThumbnail::FTrackEditorThumbnail(const FOnThumbnailDraw& InOnDraw, const FIntPoint& InDesiredSize, TRange<double> InTimeRange, double InPosition)
	: OnDraw(InOnDraw)
	, DesiredSize(InDesiredSize)
	, ThumbnailTexture(nullptr)
	, ThumbnailRenderTarget(nullptr)
	, TimeRange(InTimeRange)
	, Position(InPosition)
	, FadeInCurve(0.0f, TrackEditorThumbnailConstants::ThumbnailFadeInDuration)
{
	SortOrder = 0;
	bIgnoreAlpha = false;
}


FTrackEditorThumbnail::~FTrackEditorThumbnail()
{
	DestroyTexture();
}


void FTrackEditorThumbnail::AssignFrom(TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> InTextureData)
{
	if (!ThumbnailTexture)
	{
		EPixelFormat PixelFormat = InTextureData->GetBytesPerPixel() == 4 ? PF_B8G8R8A8 : PF_FloatRGBA;
		ThumbnailTexture = new FSlateTexture2DRHIRef(InTextureData->GetWidth(), InTextureData->GetHeight(), PixelFormat, NULL, TexCreate_None);
	}

	FSlateTexture2DRHIRef* InThumbnailTexture = ThumbnailTexture;
	ENQUEUE_RENDER_COMMAND(AssignTexture)(
		[InThumbnailTexture, InTextureData](FRHICommandList& RHICmdList){
			InThumbnailTexture->SetTextureData(InTextureData);
			if (InThumbnailTexture->IsInitialized())
			{
				InThumbnailTexture->UpdateRHI(RHICmdList);
			}
			else
			{
				InThumbnailTexture->InitResource(RHICmdList);
			}
		}
	);
}

void FTrackEditorThumbnail::DestroyTexture()
{
	if (ThumbnailRenderTarget || ThumbnailTexture)
	{
		// UE-114425: Defer the destroy until the next tick to work around the RHI getting destroyed before the render command completes.
		FSlateTexture2DRHIRef*               InThumbnailTexture      = ThumbnailTexture;
		FSlateTextureRenderTarget2DResource* InThumbnailRenderTarget = ThumbnailRenderTarget;

		ThumbnailTexture      = nullptr;
		ThumbnailRenderTarget = nullptr;

		GEditor->GetTimerManager()->SetTimerForNextTick([this, InThumbnailRenderTarget, InThumbnailTexture]()
		{
			ENQUEUE_RENDER_COMMAND(DestroyTexture)(
				[InThumbnailRenderTarget, InThumbnailTexture](FRHICommandList& RHICmdList)
				{
					if (InThumbnailTexture)
					{
						InThumbnailTexture->ReleaseResource();
						delete InThumbnailTexture;
					}

					if (InThumbnailRenderTarget)
					{
						InThumbnailRenderTarget->ReleaseResource();
						delete InThumbnailRenderTarget;
					}
				}
			);
		});
	}
}

void FTrackEditorThumbnail::ResizeRenderTarget(const FIntPoint& InSize)
{
	// Delay texture creation until we actually draw the thumbnail
	if (InSize.X <= 0 || InSize.Y <= 0)
	{
		return;
	}

	if (ThumbnailTexture && ThumbnailRenderTarget && ThumbnailTexture->GetWidth() == InSize.X && ThumbnailTexture->GetHeight() == InSize.Y)
	{
		return;
	}

	FSlateTexture2DRHIRef*               InThumbnailTexture      = ThumbnailTexture;
	FSlateTextureRenderTarget2DResource* InThumbnailRenderTarget = ThumbnailRenderTarget;
	
	// Note this used to call DestroyTexture() but now that is a latent destroy, this can no longer call DestroyTexture() since the reallocation would happen first and then the latent destroy 
	ENQUEUE_RENDER_COMMAND(DestroyTexture)(
		[InThumbnailRenderTarget, InThumbnailTexture](FRHICommandList& RHICmdList)
		{
			if (InThumbnailTexture)
			{
				InThumbnailTexture->ReleaseResource();
				delete InThumbnailTexture;
			}

			if (InThumbnailRenderTarget)
			{
				InThumbnailRenderTarget->ReleaseResource();
				delete InThumbnailRenderTarget;
			}
		}
	);

	ThumbnailTexture      = new FSlateTexture2DRHIRef(InSize.X, InSize.Y, PF_B8G8R8A8, NULL, TexCreate_None);
	ThumbnailRenderTarget = new FSlateTextureRenderTarget2DResource(FLinearColor::Black, InSize.X, InSize.Y, PF_B8G8R8A8, SF_Point, TA_Wrap, TA_Wrap, 0.0f);

	InThumbnailTexture      = ThumbnailTexture;
	InThumbnailRenderTarget = ThumbnailRenderTarget;

	ENQUEUE_RENDER_COMMAND(AssignRenderTarget)(
		[InThumbnailRenderTarget, InThumbnailTexture](FRHICommandList& RHICmdList)
		{
			if (InThumbnailTexture && InThumbnailRenderTarget)
			{
				InThumbnailTexture->InitResource(RHICmdList);
				InThumbnailRenderTarget->InitResource(RHICmdList);
				InThumbnailTexture->SetRHIRef(InThumbnailRenderTarget->GetTextureRHI(), InThumbnailRenderTarget->GetSizeX(), InThumbnailRenderTarget->GetSizeY());
			}
		}
	);
}

void FTrackEditorThumbnail::DrawThumbnail()
{
	OnDraw.ExecuteIfBound(*this);
}


void FTrackEditorThumbnail::SetupFade(const TSharedRef<SWidget>& InWidget)
{
	FadeInCurve.PlayReverse(InWidget);
	FadeInCurve.Pause();
}


void FTrackEditorThumbnail::PlayFade()
{
	FadeInCurve.Resume();
}


float FTrackEditorThumbnail::GetFadeInCurve() const 
{
	return FadeInCurve.GetLerp();
}


/* ISlateViewport interface
 *****************************************************************************/

FIntPoint FTrackEditorThumbnail::GetSize() const
{
	if (ThumbnailTexture)
	{
		return FIntPoint(ThumbnailTexture->GetWidth(), ThumbnailTexture->GetHeight());
	}
	return FIntPoint(0,0);
}


FSlateShaderResource* FTrackEditorThumbnail::GetViewportRenderTargetTexture() const
{
	return ThumbnailTexture;
}


bool FTrackEditorThumbnail::RequiresVsync() const 
{
	return false;
}


FTrackEditorThumbnailCache::FTrackEditorThumbnailCache(const TSharedPtr<FTrackEditorThumbnailPool>& InThumbnailPool, IViewportThumbnailClient* InViewportThumbnailClient)
	: ViewportThumbnailClient(InViewportThumbnailClient)
	, CustomThumbnailClient(nullptr)
	, ThumbnailPool(InThumbnailPool)
{
	check(ViewportThumbnailClient);

	LastComputationTime = 0;
	bForceRedraw = false;
	bNeedsNewThumbnails = false;
}


FTrackEditorThumbnailCache::FTrackEditorThumbnailCache(const TSharedPtr<FTrackEditorThumbnailPool>& InThumbnailPool, ICustomThumbnailClient* InCustomThumbnailClient)
	: ViewportThumbnailClient(nullptr)
	, CustomThumbnailClient(InCustomThumbnailClient)
	, ThumbnailPool(InThumbnailPool)
{
	check(CustomThumbnailClient);

	LastComputationTime = 0;
	bForceRedraw = false;
	bNeedsNewThumbnails = false;
}


FTrackEditorThumbnailCache::~FTrackEditorThumbnailCache()
{
	TSharedPtr<FTrackEditorThumbnailPool> PinnedPool = ThumbnailPool.Pin();
	if (PinnedPool.IsValid())
	{
		PinnedPool->RemoveThumbnailsNeedingRedraw(Thumbnails);
	}
}


void FTrackEditorThumbnailCache::SetSingleReferenceFrame(TOptional<double> InReferenceFrame)
{
	CurrentCache.SingleReferenceFrame = InReferenceFrame;
}

void FTrackEditorThumbnailCache::Update(const TRange<double>& NewRange, const TRange<double>& VisibleRange, const FIntPoint& AllottedSize, const FIntPoint& InDesiredSize, EThumbnailQuality Quality, double InCurrentTime)
{
	PreviousCache.TimeRange = CurrentCache.TimeRange;
	PreviousCache.VisibleRange = CurrentCache.VisibleRange;
	PreviousCache.AllottedSize = CurrentCache.AllottedSize;
	PreviousCache.DesiredSize = CurrentCache.DesiredSize;
	PreviousCache.Quality = CurrentCache.Quality;

	CurrentCache.TimeRange = NewRange;
	CurrentCache.VisibleRange = VisibleRange;
	CurrentCache.AllottedSize = AllottedSize;
	CurrentCache.DesiredSize = InDesiredSize;
	CurrentCache.Quality = Quality;

	Revalidate(InCurrentTime);

	// Only update the single reference frame value once we've updated, since that can get set at any time, but Update() may be throttled
	PreviousCache.SingleReferenceFrame = CurrentCache.SingleReferenceFrame;
}


FIntPoint FTrackEditorThumbnailCache::CalculateTextureSize(const FMinimalViewInfo& ViewInfo) const
{
	float DesiredRatio = ViewInfo.AspectRatio;

	if (CurrentCache.DesiredSize.X <= 0 || CurrentCache.DesiredSize.Y <= 0)
	{
		return FIntPoint(0,0);
	}

	float SizeRatio = float(CurrentCache.DesiredSize.X) / CurrentCache.DesiredSize.Y;

	float X = CurrentCache.DesiredSize.X;
	float Y = CurrentCache.DesiredSize.Y;

	if (SizeRatio > DesiredRatio)
	{
		// Take width
		Y = CurrentCache.DesiredSize.X / DesiredRatio;
	}
	else if (SizeRatio < DesiredRatio)
	{
		// Take height
		X = CurrentCache.DesiredSize.Y * DesiredRatio;
	}

	float Scale;
	switch (CurrentCache.Quality)
	{
		case EThumbnailQuality::Draft: 	Scale = 0.5f; 	break;
		case EThumbnailQuality::Best: 	Scale = 2.f; 	break;
		default: 						Scale = 1.f; 	break;
}

	return FIntPoint(
		FMath::RoundToInt(X * Scale),
		FMath::RoundToInt(Y * Scale)
		);
}

bool FTrackEditorThumbnailCache::ShouldRegenerateEverything() const
{
	if (bForceRedraw)
	{
		return true;
	}

	const float PreviousScale = PreviousCache.TimeRange.Size<float>() / PreviousCache.AllottedSize.X;
	const float CurrentScale = CurrentCache.TimeRange.Size<float>() / CurrentCache.AllottedSize.X;
	const float Threshold = PreviousScale * 0.01f;

	return PreviousCache.DesiredSize != CurrentCache.DesiredSize || !FMath::IsNearlyEqual(PreviousScale, CurrentScale, Threshold);
}

void FTrackEditorThumbnailCache::DrawThumbnail(FTrackEditorThumbnail& TrackEditorThumbnail)
{
	if (CustomThumbnailClient)
	{
		CustomThumbnailClient->Draw(TrackEditorThumbnail);
	}
	else if (ViewportThumbnailClient)
	{
		ViewportThumbnailClient->PreDraw(TrackEditorThumbnail);
		DrawViewportThumbnail(TrackEditorThumbnail);
		ViewportThumbnailClient->PostDraw(TrackEditorThumbnail);
	}

	// The texture will be rendered by a shader so transition it to SRVMask.
	FSlateTextureRenderTarget2DResource* pSlateResource = TrackEditorThumbnail.GetRenderTarget();
	if (pSlateResource != nullptr)
	{
		FThreadSafeBool* bHasFinishedDrawingPtr = &TrackEditorThumbnail.bHasFinishedDrawing;
		ENQUEUE_RENDER_COMMAND(SetFinishedDrawing)(
			[bHasFinishedDrawingPtr, pSlateResource](FRHICommandList& RHICmdList)
			{
				FRHITexture* InTexture = pSlateResource->GetTextureRHI();
				RHICmdList.Transition(FRHITransitionInfo(InTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
				*bHasFinishedDrawingPtr = true;
			}
		);
	}
}
void FTrackEditorThumbnailCache::DrawViewportThumbnail(FTrackEditorThumbnail& TrackEditorThumbnail) const
{
	check(ViewportThumbnailClient);

	UCameraComponent* PreviewCameraComponent = ViewportThumbnailClient->GetViewCamera();
	if (!PreviewCameraComponent)
	{
		return;
	}

	FMinimalViewInfo ViewInfo;
	PreviewCameraComponent->GetCameraView(FApp::GetDeltaTime(), ViewInfo);
	const FIntPoint RTSize = CalculateTextureSize(ViewInfo);
	if (RTSize.X <= 0 || RTSize.Y <= 0)
	{
		return;
	}

	TrackEditorThumbnail.bIgnoreAlpha = true;
	TrackEditorThumbnail.ResizeRenderTarget(RTSize);
	UWorld* World = PreviewCameraComponent->GetWorld();
	UE::MoveSceneTools::DrawViewportThumbnail(
		*TrackEditorThumbnail.GetRenderTarget(), RTSize, *World->Scene, ViewInfo, CurrentCache.Quality, &PreviewCameraComponent->PostProcessSettings
		);
}


void FTrackEditorThumbnailCache::Revalidate(double InCurrentTime)
{
	if (CurrentCache == PreviousCache && !bForceRedraw && !bNeedsNewThumbnails)
	{
		return;
	}

	if (FMath::IsNearlyZero(CurrentCache.TimeRange.Size<float>()) || CurrentCache.TimeRange.IsEmpty())
	{
		// Can't generate thumbnails for this
		ThumbnailPool.Pin()->RemoveThumbnailsNeedingRedraw(Thumbnails);
		ThumbnailsNeedingRedraw.Reset();
		Thumbnails.Reset();
		bNeedsNewThumbnails = false;
		return;
	}

	bNeedsNewThumbnails = true;

	if (ShouldRegenerateEverything())
	{
		ThumbnailPool.Pin()->RemoveThumbnailsNeedingRedraw(Thumbnails);
		Thumbnails.Reset();
	}
	
	if (InCurrentTime - LastComputationTime > 0.25f)
	{
		ComputeNewThumbnails();
		LastComputationTime = InCurrentTime;
	}
}

void FTrackEditorThumbnailCache::ComputeNewThumbnails()
{
	ThumbnailsNeedingRedraw.Reset();

	if (CurrentCache.SingleReferenceFrame.IsSet())
	{
		if (!Thumbnails.Num() || bForceRedraw || CurrentCache.SingleReferenceFrame != PreviousCache.SingleReferenceFrame)
		{
			UpdateSingleThumbnail();
		}
	}
	else
	{
		UpdateFilledThumbnails();
	}

	if (ThumbnailsNeedingRedraw.Num())
	{
		ThumbnailPool.Pin()->AddThumbnailsNeedingRedraw(ThumbnailsNeedingRedraw);
	}
	if (Thumbnails.Num())
	{
		Setup();
	}

	bForceRedraw = false;
	bNeedsNewThumbnails = false;
}


void FTrackEditorThumbnailCache::UpdateSingleThumbnail()
{
	Thumbnails.Reset();

	const double TimePerPx    = CurrentCache.TimeRange.Size<double>() / CurrentCache.AllottedSize.X;
	const double HalfRange    = CurrentCache.DesiredSize.X*TimePerPx*.5;
	const double EvalPosition = CurrentCache.SingleReferenceFrame.GetValue();

	TSharedPtr<FTrackEditorThumbnail> NewThumbnail = MakeShareable(new FTrackEditorThumbnail(
		FOnThumbnailDraw::CreateRaw(this, &FTrackEditorThumbnailCache::DrawThumbnail),
		CurrentCache.DesiredSize,
		TRange<double>(EvalPosition - HalfRange, EvalPosition + HalfRange),
		EvalPosition
	));

	Thumbnails.Add(NewThumbnail);
	ThumbnailsNeedingRedraw.Add(NewThumbnail);
}


void FTrackEditorThumbnailCache::UpdateFilledThumbnails()
{
	// Remove any thumbnails from the front of the array that aren't in the actual time range of this section (we keep stuff around outside of the visible range)
	{
		int32 Index = 0;
		for (; Index < Thumbnails.Num(); ++Index)
		{
			if (Thumbnails[Index]->GetTimeRange().Overlaps(CurrentCache.TimeRange))
			{
				break;
			}
		}
		if (Index)
		{
			TArray<TSharedPtr<FTrackEditorThumbnail>> Remove;
			Remove.Append(&Thumbnails[0], Index);
			ThumbnailPool.Pin()->RemoveThumbnailsNeedingRedraw(Remove);

			Thumbnails.RemoveAt(0, Index, EAllowShrinking::No);
		}
	}

	// Remove any thumbnails from the back of the array that aren't in the *actual* time range of this section (we keep stuff around outside of the visible range)
	{
		int32 NumToRemove = 0;
		for (int32 Index = Thumbnails.Num() - 1; Index >= 0; --Index)
		{
			if (!Thumbnails[Index]->GetTimeRange().Overlaps(CurrentCache.TimeRange))
			{
				++NumToRemove;
			}
			else
			{
				break;
			}
		}
		
		if (NumToRemove)
		{
			TArray<TSharedPtr<FTrackEditorThumbnail>> Remove;
			Remove.Append(&Thumbnails[Thumbnails.Num() - NumToRemove], NumToRemove);
			ThumbnailPool.Pin()->RemoveThumbnailsNeedingRedraw(Remove);

			Thumbnails.RemoveAt(Thumbnails.Num() - NumToRemove, NumToRemove, EAllowShrinking::No);
		}
	}

	TRange<double> MaxRange(CurrentCache.VisibleRange.GetLowerBoundValue() - CurrentCache.VisibleRange.Size<double>(), CurrentCache.VisibleRange.GetUpperBoundValue() + CurrentCache.VisibleRange.Size<double>());
	TRange<double> Boundary = TRange<double>::Intersection(CurrentCache.TimeRange, MaxRange);

	if (!Boundary.IsEmpty())
	{
		GenerateFront(Boundary);
		GenerateBack(Boundary);
	}

	if (Thumbnails.Num())
	{
		for (const TSharedPtr<FTrackEditorThumbnail>& Thumbnail : Thumbnails)
		{
			Thumbnail->SortOrder = Thumbnail->GetTimeRange().Overlaps(CurrentCache.VisibleRange) ? 1 : 10;
		}
	}
}


void FTrackEditorThumbnailCache::GenerateFront(const TRange<double>& Boundary)
{
	if (!Thumbnails.Num())
	{
		return;
	}

	const double TimePerPx = CurrentCache.TimeRange.Size<double>() / CurrentCache.AllottedSize.X;
	double EndTime = Thumbnails[0]->GetTimeRange().GetLowerBoundValue();

	while (EndTime > Boundary.GetLowerBoundValue())
	{
		FIntPoint TextureSize = CurrentCache.DesiredSize;

		TRange<double> TimeRange(EndTime - TextureSize.X * TimePerPx, EndTime);

		// Evaluate the thumbnail along the length of its duration, based on its position in the sequence
		const double FrameLength = TimeRange.Size<double>();
		double TotalLerp = (TimeRange.GetLowerBoundValue() - CurrentCache.TimeRange.GetLowerBoundValue()) / (CurrentCache.TimeRange.Size<double>() - FrameLength);
		double EvalPosition = CurrentCache.TimeRange.GetLowerBoundValue() + FMath::Clamp(TotalLerp, 0.0, .99)*CurrentCache.TimeRange.Size<double>();

		TSharedPtr<FTrackEditorThumbnail> NewThumbnail = MakeShareable(new FTrackEditorThumbnail(
			FOnThumbnailDraw::CreateRaw(this, &FTrackEditorThumbnailCache::DrawThumbnail),
			TextureSize,
			TimeRange,
			EvalPosition
		));

		Thumbnails.Insert(NewThumbnail, 0);
		ThumbnailsNeedingRedraw.Add(NewThumbnail);

		EndTime = TimeRange.GetLowerBoundValue();
	}
}


void FTrackEditorThumbnailCache::GenerateBack(const TRange<double>& Boundary)
{
	const double TimePerPx = CurrentCache.TimeRange.Size<double>() / CurrentCache.AllottedSize.X;
	double StartTime = Thumbnails.Num() ? Thumbnails.Last()->GetTimeRange().GetUpperBoundValue() : Boundary.GetLowerBoundValue();

	while (StartTime < Boundary.GetUpperBoundValue())
	{
		FIntPoint TextureSize = CurrentCache.DesiredSize;

		{
			// Move the thumbnail to the center of the space if we're the only thumbnail, and we don't fit on
			double Overflow = TextureSize.X*TimePerPx - CurrentCache.TimeRange.Size<double>();
			if (Thumbnails.Num() == 0 && Overflow > 0)
			{
				StartTime -= Overflow*.5f;
			}
		}

		TRange<double> TimeRange(StartTime, StartTime + TextureSize.X * TimePerPx);

		const double FrameLength = TimeRange.Size<double>();
		double TotalLerp = (TimeRange.GetLowerBoundValue() - CurrentCache.TimeRange.GetLowerBoundValue()) / (CurrentCache.TimeRange.Size<double>() - FrameLength);
		double EvalPosition = CurrentCache.TimeRange.GetLowerBoundValue() + FMath::Clamp(TotalLerp, 0.0, .99)*CurrentCache.TimeRange.Size<double>();

		TSharedPtr<FTrackEditorThumbnail> NewThumbnail = MakeShareable(new FTrackEditorThumbnail(
			FOnThumbnailDraw::CreateRaw(this, &FTrackEditorThumbnailCache::DrawThumbnail),
			TextureSize,
			TimeRange,
			EvalPosition
		));

		NewThumbnail->SortOrder = TimeRange.Overlaps(CurrentCache.VisibleRange) ? 1 : 10;

		Thumbnails.Add(NewThumbnail);
		ThumbnailsNeedingRedraw.Add(NewThumbnail);

		StartTime = TimeRange.GetUpperBoundValue();
	}
}


void FTrackEditorThumbnailCache::Setup()
{
	if (CustomThumbnailClient)
	{
		CustomThumbnailClient->Setup();
	}
}

