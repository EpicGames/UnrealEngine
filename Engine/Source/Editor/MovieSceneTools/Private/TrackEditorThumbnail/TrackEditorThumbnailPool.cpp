// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "TrackEditorThumbnail/TrackEditorThumbnail.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencer.h"
#include "Editor.h"


/* FShotSequencerSection structors
 *****************************************************************************/

FTrackEditorThumbnailPool::FTrackEditorThumbnailPool(TSharedPtr<ISequencer> InSequencer)
	: Sequencer(InSequencer)
	, ThumbnailsNeedingDraw()
	, bNeedsSort(false)
{
	TimeOfLastDraw = TimeOfLastUpdate = 0;
}


/* FShotSequencerSection interface
 *****************************************************************************/

void FTrackEditorThumbnailPool::AddThumbnailsNeedingRedraw(const TArray<TSharedPtr<FTrackEditorThumbnail>>& InThumbnails)
{
	ThumbnailsNeedingDraw.Append(InThumbnails);
	bNeedsSort = true;
}


bool FTrackEditorThumbnailPool::DrawThumbnails()
{
	int32 ThumbnailsDrawn = 0;

	// apply sorting if necessary
	if (bNeedsSort)
	{
		auto SortFunc = [](const TSharedPtr<FTrackEditorThumbnail>& A, const TSharedPtr<FTrackEditorThumbnail>& B){
			if (A->SortOrder == B->SortOrder)
			{
				return A->GetTimeRange().GetLowerBoundValue() < B->GetTimeRange().GetLowerBoundValue();
			}
			return A->SortOrder < B->SortOrder;
		};

		ThumbnailsNeedingDraw.Sort(SortFunc);
		bNeedsSort = false;
	}

	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// only allow drawing if we're not waiting on other thumbnails
	bool bAllowDraw = !ThumbnailsBeingDrawn.Num();

	for (int32 ThumbnailIndex = ThumbnailsBeingDrawn.Num() - 1; ThumbnailIndex >= 0; --ThumbnailIndex)
	{
		if (ThumbnailsBeingDrawn[ThumbnailIndex]->bHasFinishedDrawing)
		{
			ThumbnailsBeingDrawn[ThumbnailIndex]->PlayFade();
			ThumbnailsBeingDrawn.RemoveAt(ThumbnailIndex, EAllowShrinking::No);
			TimeOfLastDraw = CurrentTime;
		}
		else
		{
			bAllowDraw = false;
		}
	}

	if (bAllowDraw)
	{
		if (!ThumbnailsNeedingDraw.Num() && !ThumbnailsBeingDrawn.Num())
		{
			TimeOfLastDraw = CurrentTime;
		}
		else
		{
			const float MinThumbnailsPerS = 2.f;
			const float MaxThumbnailsPerS = 120.f;

			const float MinFramerate = 10.f;
			const float MaxFramerate = 90.f;

			const float AverageDeltaTime = FSlateApplication::Get().GetAverageDeltaTime();
			const float Lerp = FMath::Max(((1.f / AverageDeltaTime) - MinFramerate), 0.f) / (MaxFramerate - MinFramerate);
			const float NumToDrawPerS = FMath::Lerp(MinThumbnailsPerS, MaxThumbnailsPerS, Lerp);

			const float TimeThreshold = 1.f / NumToDrawPerS;
			const float TimeElapsed = CurrentTime - TimeOfLastDraw;

			if (!FMath::IsNearlyEqual(float(CurrentTime - TimeOfLastUpdate), AverageDeltaTime, AverageDeltaTime*2.f))
			{
				// don't generate thumbnails if we haven't had an update within a reasonable time - assume some blocking task
				TimeOfLastDraw = CurrentTime;
			}
			else
			{
				const int32 NumToDraw = FMath::TruncToInt(TimeElapsed / TimeThreshold);

				for (; ThumbnailsDrawn < FMath::Min(NumToDraw, ThumbnailsNeedingDraw.Num()); ++ThumbnailsDrawn)
				{
					const TSharedPtr<FTrackEditorThumbnail>& Thumbnail = ThumbnailsNeedingDraw[ThumbnailsDrawn];

					bool bIsEnabled = Sequencer.Pin()->IsPerspectiveViewportCameraCutEnabled();
					Sequencer.Pin()->SetPerspectiveViewportCameraCutEnabled(false);

					Thumbnail->DrawThumbnail();
					ThumbnailsBeingDrawn.Add(Thumbnail);

					Sequencer.Pin()->SetPerspectiveViewportCameraCutEnabled(bIsEnabled);
				}
			}
		}
	}

	if (ThumbnailsDrawn > 0)
	{
		ThumbnailsNeedingDraw.RemoveAt(0, ThumbnailsDrawn, EAllowShrinking::No);

		// If done drawing thumbnails, refresh actor list
		if (ThumbnailsNeedingDraw.Num() == 0)
		{
			GEditor->BroadcastLevelActorListChanged();
		}
	}

	TimeOfLastUpdate = CurrentTime;

	return ThumbnailsDrawn > 0;
}


void FTrackEditorThumbnailPool::RemoveThumbnailsNeedingRedraw(const TArray< TSharedPtr<FTrackEditorThumbnail> >& InThumbnails)
{
	for (int32 i = 0; i < InThumbnails.Num(); ++i)
	{
		ThumbnailsNeedingDraw.Remove(InThumbnails[i]);
		ThumbnailsBeingDrawn.Remove(InThumbnails[i]);
	}
}
