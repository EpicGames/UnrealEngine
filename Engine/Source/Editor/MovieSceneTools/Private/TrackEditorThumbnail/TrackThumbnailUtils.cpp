// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditorThumbnail/TrackThumbnailUtils.h"

#include "Camera/CameraTypes.h"
#include "CanvasTypes.h"
#include "EngineModule.h"
#include "LegacyScreenPercentageDriver.h"
#include "MovieSceneToolsUserSettings.h"
#include "SceneViewExtension.h"
#include "RendererInterface.h"

static TAutoConsoleVariable<int32> CVarSequencerUsePostProcessThumbnails(
	TEXT("Sequencer.UsePostProcessThumbnails"),
	0, TEXT("Enable post process in thumbnails."),
	ECVF_Default);

namespace UE::MoveSceneTools
{
	void PreDrawThumbnailSetupSequencer(ISequencer& Sequencer, FFrameTime CaptureFrame)
	{
		Sequencer.EnterSilentMode();
		Sequencer.SetPlaybackStatus(EMovieScenePlayerStatus::Jumping);
		Sequencer.SetLocalTimeDirectly(CaptureFrame);
		Sequencer.ForceEvaluate();
	}

	void PostDrawThumbnailCleanupSequencer(ISequencer& Sequencer)
	{
		Sequencer.ExitSilentMode();
	}

	void DrawViewportThumbnail(
		FRenderTarget& ThumbnailRenderTarget,
		const FIntPoint& RenderTargetSize,
		FSceneInterface& Scene,
		const FMinimalViewInfo& ViewInfo,
		EThumbnailQuality Quality,
		const FPostProcessSettings* OverridePostProcessSettings
		)
	{
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues(&ThumbnailRenderTarget, &Scene, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime::GetTimeSinceAppStart())
		.SetResolveScene(true));
		
		// When GRayTracingMode is disabled(default: disabled) and MegaLights is enabled(default: enabled), we'll get a red warning text on the thumbnail.
		// Even if the project was set up to use ray tracing, we don't need thumbnails to use raytracing anyways (performance), so just disable. 
		ViewFamily.EngineShowFlags.SetMegaLights(false);

		FSceneViewStateInterface* ViewStateInterface = nullptr;

		// Screen percentage is not supported in thumbnail.
		ViewFamily.EngineShowFlags.ScreenPercentage = false;

		switch (Quality)
		{
		case EThumbnailQuality::Draft:
			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.SetPostProcessing(false);
			break;

		case EThumbnailQuality::Normal:
		case EThumbnailQuality::Best:
			ViewFamily.EngineShowFlags.SetMotionBlur(false);

			// Default eye adaptation requires a viewstate.
			ViewFamily.EngineShowFlags.EyeAdaptation = true;
			UMovieSceneUserThumbnailSettings* ThumbnailSettings = GetMutableDefault<UMovieSceneUserThumbnailSettings>();
			FSceneViewStateInterface* Ref = ThumbnailSettings->ViewState.GetReference();
			if (!Ref)
			{
				ThumbnailSettings->ViewState.Allocate(ViewFamily.GetFeatureLevel());
			}
			ViewStateInterface = ThumbnailSettings->ViewState.GetReference();
			break;
		}

		FSceneViewInitOptions ViewInitOptions;

		// Use target exposure without blend. 
		ViewInitOptions.bInCameraCut = true;
		ViewInitOptions.SceneViewStateInterface = ViewStateInterface;

		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint::ZeroValue, RenderTargetSize));
		ViewInitOptions.ViewFamily = &ViewFamily;

		ViewInitOptions.ViewOrigin = ViewInfo.Location;
		ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(ViewInfo.Rotation) * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		ViewInitOptions.ProjectionMatrix = ViewInfo.CalculateProjectionMatrix();

		FSceneView* NewView = new FSceneView(ViewInitOptions);
		ViewFamily.Views.Add(NewView);
		if (OverridePostProcessSettings && CVarSequencerUsePostProcessThumbnails->GetInt() > 0)
		{
			FPostProcessSettings ProcessSettings = *OverridePostProcessSettings;
			
			// Temporal effects need time to warm up, which we don't do for thumbnail rendering for performance reasons.
			// Auto-exposure is a temporal effect.
			// Basic causes the AutoExposureBias to be used which will prevent the thumbnail from being mostly white in very bright scenes.
			// This will not work in all cases but should in most cases (since it depends on what value is set for AutoExposureBias).
			ProcessSettings.AutoExposureMethod = AEM_Basic;
			ProcessSettings.bOverride_AutoExposureMethod = true;
			ProcessSettings.AutoExposureBias = 1.0f;
			ProcessSettings.bOverride_AutoExposureBias = true;
			
			NewView->OverridePostProcessSettings(ProcessSettings, 1.f);
		}

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.AntiAliasing = 0;

		const float GlobalResolutionFraction = 1.f;
		ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, GlobalResolutionFraction));

		FCanvas Canvas(&ThumbnailRenderTarget, nullptr, FGameTime::GetTimeSinceAppStart(), Scene.GetFeatureLevel());
		Canvas.Clear(FLinearColor::Transparent);

		ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(&Scene));
		for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
		{
			Extension->SetupViewFamily(ViewFamily);
			Extension->SetupView(ViewFamily, *NewView);
		}

		GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);
	}
}
