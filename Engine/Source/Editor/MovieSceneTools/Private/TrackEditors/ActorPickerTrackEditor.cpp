// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/ActorPickerTrackEditor.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "ActorPickerMode.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "SSocketChooser.h"
#include "LevelEditor.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "MovieSceneToolHelpers.h"
#include "SComponentChooser.h"
#include "ActorTreeItem.h"
#include "SequencerSettings.h"

#define LOCTEXT_NAMESPACE "FActorPickerTrackEditor"

FActorPickerTrackEditor::FActorPickerTrackEditor(TSharedRef<ISequencer> InSequencer) 
	: FMovieSceneTrackEditor( InSequencer ) 

{
}

void FActorPickerTrackEditor::PickActorInteractive(const TArray<FGuid>& ObjectBindings, UMovieSceneSection* Section)
{
	if(GUnrealEd->GetSelectedActorCount())
	{
		FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

		ActorPickerMode.BeginActorPickingMode(
			FOnGetAllowedClasses(), 
			FOnShouldFilterActor::CreateSP(this, &FActorPickerTrackEditor::IsActorPickable, ObjectBindings[0], Section), 
			FOnActorSelected::CreateSP(this, &FActorPickerTrackEditor::ActorPicked, ObjectBindings, Section)
			);
	}
}

FText FActorPickerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ActorPickerTrackEditor_DisplayName", "Actor");
}

void FActorPickerTrackEditor::ShowActorSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section)
{
	struct Local
	{
		static FReply OnInteractiveActorPickerClicked(FActorPickerTrackEditor* ActorPickerTrackEditor, TArray<FGuid> TheObjectBindings, UMovieSceneSection* TheSection)
		{
			FSlateApplication::Get().DismissAllMenus();
			ActorPickerTrackEditor->PickActorInteractive(TheObjectBindings, TheSection);
			return FReply::Handled();
		}
	};

	auto CreateNewBinding = 
		[this, ObjectBindings, Section](FMenuBuilder& SubMenuBuilder)
	{
		FSceneOutlinerInitializationOptions InitOptions;
		{	
			InitOptions.bShowHeaderRow = false;
			InitOptions.bFocusSearchBoxWhenOpened = true;
			InitOptions.bShowTransient = true;
			InitOptions.bShowCreateNewFolder = false;
			// Only want the actor label column
			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

			// Only display Actors that we can attach too
			InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateSP(this, &FActorPickerTrackEditor::IsActorPickable, ObjectBindings[0], Section));
		}		

		// Actor selector to allow the user to choose a parent actor
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>( "SceneOutliner" );

		TSharedPtr<ISequencer> Sequencer = GetSequencer();
		const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		TSharedRef< SWidget > MenuWidget = 
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(WidthOverride)
				.HeightOverride(HeightOverride)
				[
					SceneOutlinerModule.CreateActorPicker(
						InitOptions,
						FOnActorPicked::CreateSP(this, &FActorPickerTrackEditor::ActorPicked, ObjectBindings, Section )
						)
				]
			]
	
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ToolTipText( LOCTEXT( "PickButtonLabel", "Pick a parent actor to attach to") )
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(FOnClicked::CreateStatic(&Local::OnInteractiveActorPickerClicked, this, ObjectBindings, Section))
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.EyeDropper"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];

		SubMenuBuilder.AddWidget(MenuWidget, FText::GetEmpty(), false);
	};

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	// Always recreate the binding picker to ensure we have the correct sequence ID
	BindingIDPicker = MakeShared<FTrackEditorBindingIDPicker>(SequencerPtr->GetFocusedTemplateID(), SequencerPtr);
	BindingIDPicker->OnBindingPicked().AddRaw(this, &FActorPickerTrackEditor::ExistingBindingPicked, ObjectBindings, Section);

	FText ExistingBindingText = LOCTEXT("ExistingBinding", "Existing Binding");
	FText NewBindingText = LOCTEXT("NewBinding", "New Binding");

	const bool bHasExistingBindings = !BindingIDPicker->IsEmpty();
	if (bHasExistingBindings)
	{
		MenuBuilder.AddSubMenu(
			NewBindingText,
			LOCTEXT("NewBinding_Tip", "Add a new section by creating a new binding to an object in the world."),
			FNewMenuDelegate::CreateLambda(CreateNewBinding)
		);

		MenuBuilder.BeginSection(NAME_None, ExistingBindingText);
		{
			BindingIDPicker->GetPickerMenu(MenuBuilder);
		}
		MenuBuilder.EndSection();
	}
	else
	{
		MenuBuilder.BeginSection(NAME_None, NewBindingText);
		{
			CreateNewBinding(MenuBuilder);
		}
		MenuBuilder.EndSection();
	}
}


void FActorPickerTrackEditor::ActorPicked(AActor* ParentActor, TArray<FGuid> ObjectGuids, UMovieSceneSection* Section)
{
	ActorPickerIDPicked(FActorPickerID(ParentActor, UE::MovieScene::FFixedObjectBindingID()), ObjectGuids, Section);
}


void FActorPickerTrackEditor::ExistingBindingPicked(FMovieSceneObjectBindingID ExistingBindingID, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	FMovieSceneSequenceID SequenceID = SequencerPtr->GetFocusedTemplateID();

	if (ExistingBindingID.IsValid())
	{
		// Ensure that this ID is resolvable from the root, based on the current local sequence ID
		SequenceID = ExistingBindingID.ResolveSequenceID(SequenceID, *SequencerPtr);
	}

	for (TWeakObjectPtr<> RuntimeObject : SequencerPtr->FindBoundObjects(ExistingBindingID.GetGuid(), SequenceID))
	{
		if (AActor* Actor = Cast<AActor>(RuntimeObject.Get()))
		{
			ActorPickerIDPicked(FActorPickerID(Actor, ExistingBindingID), ObjectBindings, Section);
			return;
		}
	}

	ActorPickerIDPicked(FActorPickerID(nullptr, ExistingBindingID), ObjectBindings, Section);
}


void FActorPickerTrackEditor::ActorPickerIDPicked(FActorPickerID ActorPickerID, const TArray<FGuid>& ObjectGuids, UMovieSceneSection* Section)
{
	TArray<USceneComponent*> ComponentsWithSockets;
	if (ActorPickerID.ActorPicked.IsValid())
	{
		TInlineComponentArray<USceneComponent*> Components(ActorPickerID.ActorPicked.Get());

		for(USceneComponent* Component : Components)
		{
			if (Component->HasAnySockets())
			{
				ComponentsWithSockets.Add(Component);
			}
		}
	}

	if (ComponentsWithSockets.Num() == 0)
	{
		FSlateApplication::Get().DismissAllMenus();
		ActorSocketPicked( NAME_None, nullptr, ActorPickerID, ObjectGuids, Section );
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;

	if (ComponentsWithSockets.Num() > 1)
	{			
		MenuWidget = 
			SNew(SComponentChooserPopup)
			.Actor(ActorPickerID.ActorPicked.Get())
			.OnComponentChosen(this, &FActorPickerTrackEditor::ActorComponentPicked, ActorPickerID, ObjectGuids, Section);		

		// Create as context menu
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);
	}
	else
	{
		ActorComponentPicked(ComponentsWithSockets[0]->GetFName(), ActorPickerID, ObjectGuids, Section);
	}
}


void FActorPickerTrackEditor::ActorComponentPicked(FName ComponentName, FActorPickerID ActorPickerID, TArray<FGuid> ObjectGuids, UMovieSceneSection* Section)
{
	USceneComponent* ComponentWithSockets = nullptr;
	if (ActorPickerID.ActorPicked.IsValid())
	{
		TInlineComponentArray<USceneComponent*> Components(ActorPickerID.ActorPicked.Get());
	
		for(USceneComponent* Component : Components)
		{
			if (Component->GetFName() == ComponentName)
			{
				ComponentWithSockets = Component;
				break;
			}
		}
	}

	if (ComponentWithSockets == nullptr)
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;

	MenuWidget = 
		SNew(SSocketChooserPopup)
		.SceneComponent(ComponentWithSockets)
		.OnSocketChosen(this, &FActorPickerTrackEditor::ActorSocketPicked, ComponentWithSockets, ActorPickerID, ObjectGuids, Section);		

	// Create as context menu
	FSlateApplication::Get().PushMenu(
		LevelEditor.ToSharedRef(),
		FWidgetPath(),
		MenuWidget.ToSharedRef(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
}

#undef LOCTEXT_NAMESPACE
