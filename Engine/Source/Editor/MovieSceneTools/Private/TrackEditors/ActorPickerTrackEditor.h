// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Misc/Guid.h"
#include "ISequencer.h"
#include "MovieSceneTrackEditor.h"

class AActor;
class FMenuBuilder;
class USceneComponent;
class FTrackEditorBindingIDPicker;

struct FMovieSceneObjectBindingID;

/**
 * Track editor actor picker
 */
class FActorPickerTrackEditor : public FMovieSceneTrackEditor
{
public:

	struct FActorPickerID
	{
		FActorPickerID(AActor* InActorPicked, const FMovieSceneObjectBindingID& InExistingBindingID) : ActorPicked(InActorPicked), ExistingBindingID(InExistingBindingID) {}

		TWeakObjectPtr<AActor> ActorPicked;
		FMovieSceneObjectBindingID ExistingBindingID;
	};

	FActorPickerTrackEditor(TSharedRef<ISequencer> InSequencer);

	//~ Begin ISequencerTrackEditor
	virtual FText GetDisplayName() const override;
	//~ End ISequencerTrackEditor

	/** Is this actor pickable? */
	virtual bool IsActorPickable( const AActor* const ParentActor, FGuid ObjectBinding, UMovieSceneSection* InSection) { return false; }

	/** Actor socket was picked */
	virtual void ActorSocketPicked(const FName SocketName, USceneComponent* Component, FActorPickerID ActorPickerID, TArray<FGuid> ObjectBinding, UMovieSceneSection* Section) {}

	/** Show a sub menu of the pickable actors */
	void ShowActorSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section);

private: 

	/** Actor was picked */
	void ActorPicked(AActor* ParentActor, TArray<FGuid> ObjectBinding, UMovieSceneSection* Section);

	/** Existing binding was picked */
	void ExistingBindingPicked(FMovieSceneObjectBindingID ExistingBindingID, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section);

	/** Actor and/or existing binding was picked */
	void ActorPickerIDPicked(FActorPickerID, const TArray<FGuid>& ObjectBindings, UMovieSceneSection* Section);

	/** Actor component was picked */
	void ActorComponentPicked(FName ComponentName, FActorPickerID ActorPickerID, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section);

	/** Interactively pick an actor from the viewport */
	void PickActorInteractive(const TArray<FGuid>& ObjectBindings, UMovieSceneSection* Section);

	/** A binding ID picker that allows us to create a new section from an existing binding */
	TSharedPtr<FTrackEditorBindingIDPicker> BindingIDPicker;
};
