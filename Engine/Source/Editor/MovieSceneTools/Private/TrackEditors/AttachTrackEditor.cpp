// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/AttachTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "SequencerSectionPainter.h"
#include "GameFramework/WorldSettings.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DAttachSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "ActorEditorUtils.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "MovieSceneToolHelpers.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "Algo/Transform.h"
#include "Algo/Copy.h"
#include "Containers/Union.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "F3DAttachTrackEditor"

/**
 * Class that draws an attach section in the sequencer
 */
class F3DAttachSection
	: public ISequencerSection
{
public:

	F3DAttachSection( UMovieSceneSection& InSection, F3DAttachTrackEditor* InAttachTrackEditor )
		: Section( InSection )
		, AttachTrackEditor(InAttachTrackEditor)
	{ }

	/** ISequencerSection interface */
	virtual UMovieSceneSection* GetSectionObject() override
	{ 
		return &Section;
	}

	virtual FText GetSectionTitle() const override 
	{ 
		UMovieScene3DAttachSection* AttachSection = Cast<UMovieScene3DAttachSection>(&Section);
		if (AttachSection)
		{
			TSharedPtr<ISequencer> Sequencer = AttachTrackEditor->GetSequencer();
			if (Sequencer.IsValid())
			{
				TArrayView<TWeakObjectPtr<UObject>> RuntimeObjects = AttachSection->GetConstraintBindingID().ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer);
				if (RuntimeObjects.Num() == 1 && RuntimeObjects[0].IsValid())
				{
					if (AActor* Actor = Cast<AActor>(RuntimeObjects[0].Get()))
					{
						if (AttachSection->AttachSocketName.IsNone())
						{
							return FText::FromString(Actor->GetActorLabel());
						}
						else
						{
							return FText::Format(LOCTEXT("SectionTitleFormat", "{0} ({1})"), FText::FromString(Actor->GetActorLabel()), FText::FromName(AttachSection->AttachSocketName));
						}
					}
				}
			}
		}

		return FText::GetEmpty(); 
	}

	virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override 
	{
		return InPainter.PaintSectionBackground();
	}
	
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override
	{
		TArray<FGuid> ObjectBindings;
		ObjectBindings.Add(ObjectBinding);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("AttachSectionOptions", "Attach Section Options"));

		MenuBuilder.AddSubMenu(
			LOCTEXT("SetAttach", "Attach"), LOCTEXT("SetAttachTooltip", "Set attach"),
			FNewMenuDelegate::CreateRaw(AttachTrackEditor, &FActorPickerTrackEditor::ShowActorSubMenu, ObjectBindings, &Section));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrimRightPreserve", "Trim Right and Preserve"),
			LOCTEXT("TrimRightPreserveToolTip", "Trims the right side of this attach at the current time and preserves the last key's world coordinates"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(AttachTrackEditor, &F3DAttachTrackEditor::TrimAndPreserve, ObjectBinding, &Section, false),
				FCanExecuteAction::CreateLambda([this] { 
					TSet<UMovieSceneSection*> Sections;
					Sections.Add(&Section);
					TSharedPtr<ISequencer> Sequencer = AttachTrackEditor->GetSequencer();
					FQualifiedFrameTime Time = Sequencer->GetLocalTime();
					return MovieSceneToolHelpers::CanTrimSectionRight(Sections, Time);
				}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrimLeftPreserve", "Trim Left and Preserve"),
			LOCTEXT("TrimLeftPreserveToolTip", "Trims the left side of this attach at the current time and preserves the first key's world coordinates"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(AttachTrackEditor, &F3DAttachTrackEditor::TrimAndPreserve, ObjectBinding, &Section, true),
				FCanExecuteAction::CreateLambda([this] {
					TSet<UMovieSceneSection*> Sections;
					Sections.Add(&Section);
					TSharedPtr<ISequencer> Sequencer = AttachTrackEditor->GetSequencer();
					FQualifiedFrameTime Time = Sequencer->GetLocalTime();
					return MovieSceneToolHelpers::CanTrimSectionLeft(Sections, Time);
				}))
		);

		MenuBuilder.EndSection();
	}

private:

	/** The section we are visualizing */
	UMovieSceneSection& Section;

	/** The attach track editor */
	F3DAttachTrackEditor* AttachTrackEditor;
};

F3DAttachTrackEditor::F3DAttachTrackEditor( TSharedRef<ISequencer> InSequencer )
: FActorPickerTrackEditor( InSequencer )
, PreserveType(ETransformPreserveType::None)
, Interrogator(MakeUnique<UE::MovieScene::FSystemInterrogator>())
{
}

F3DAttachTrackEditor::~F3DAttachTrackEditor()
{
}

TSharedRef<ISequencerTrackEditor> F3DAttachTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new F3DAttachTrackEditor( InSequencer ) );
}

bool F3DAttachTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	// We support animatable transforms
	return Type == UMovieScene3DAttachTrack::StaticClass();
}

bool F3DAttachTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieScene3DAttachTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

TSharedRef<ISequencerSection> F3DAttachTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );

	return MakeShareable( new F3DAttachSection( SectionObject, this ) );
}


void F3DAttachTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass != nullptr && (ObjectClass->IsChildOf(AActor::StaticClass()) || ObjectClass->IsChildOf(USceneComponent::StaticClass())))
	{
		UMovieSceneSection* DummySection = nullptr;

		MenuBuilder.AddSubMenu(
			LOCTEXT("AddAttach", "Attach"), LOCTEXT("AddAttachTooltip", "Adds an attach track."),
			FNewMenuDelegate::CreateRaw(this, &F3DAttachTrackEditor::ShowPickerSubMenu, ObjectBindings, DummySection));
	}
}

void F3DAttachTrackEditor::ShowPickerSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section)
{
	ShowActorSubMenu(MenuBuilder, ObjectBindings, Section);

	FText PreserveText = LOCTEXT("ExistingBinding", "Existing Binding");

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AttachOptions", "Attach Options"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TogglePreserveCurrentTransform", "Preserve Current"),
		LOCTEXT("TogglePreserveCurrentTransformTooltip", "Preserve this object's transform in world space for first frame of attach"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { 
				PreserveType = ETransformPreserveType::CurrentKey;
			}), 
			FCanExecuteAction::CreateLambda([]() { return true; }),
			FIsActionChecked::CreateLambda([this]() { return PreserveType == ETransformPreserveType::CurrentKey; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TogglePreserveAllTransform", "Preserve All"),
		LOCTEXT("TogglePreserveAllTransformTooltip", "Preserve this object's transform in world space for every child and parent key in attach range"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { 
				PreserveType = ETransformPreserveType::AllKeys; 
			}),
			FCanExecuteAction::CreateLambda([this]() { return true; }),
			FIsActionChecked::CreateLambda([this]() { return PreserveType == ETransformPreserveType::AllKeys; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TogglePreserveBake", "Preserve with Bake"),
		LOCTEXT("TogglePreserveBakeTooltip", "Object's relative transform will be calculated every frame to preserve original world space transform"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { 
				PreserveType = ETransformPreserveType::Bake; 
			}),
			FCanExecuteAction::CreateLambda([this]() { return true; }),
			FIsActionChecked::CreateLambda([this]() { return PreserveType == ETransformPreserveType::Bake; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TogglePreserveNone", "None"),
		LOCTEXT("TogglePreserveNoneTooltip", "Object's transform will not be compensated"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() {
				PreserveType = ETransformPreserveType::None; 
			}),
			FCanExecuteAction::CreateLambda([this]() { return true; }),
			FIsActionChecked::CreateLambda([this]() { return PreserveType == ETransformPreserveType::None; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
		);

	MenuBuilder.EndSection();
}

bool F3DAttachTrackEditor::IsActorPickable(const AActor* const ParentActor, FGuid ObjectBinding, UMovieSceneSection* InSection)
{
	// Can't pick the object that this track binds
	TArrayView<TWeakObjectPtr<>> Objects = GetSequencer()->FindObjectsInCurrentSequence(ObjectBinding);
	if (Objects.Contains(ParentActor))
	{
		return false;
	}

	for (auto Object : Objects)
	{
		if (Object.IsValid())
		{
			AActor* ChildActor = Cast<AActor>(Object.Get());
			if (ChildActor)
			{
				USceneComponent* ChildRoot = ChildActor->GetRootComponent();
				USceneComponent* ParentRoot = ParentActor->GetDefaultAttachComponent();

				if (!ChildRoot || !ParentRoot || ParentRoot->IsAttachedTo(ChildRoot))
				{
					return false;
				}
			}
		}
	}

	if (ParentActor->IsListedInSceneOutliner() &&
		!FActorEditorUtils::IsABuilderBrush(ParentActor) &&
		!ParentActor->IsA( AWorldSettings::StaticClass() ) &&
		IsValid(ParentActor))
	{			
		return true;
	}
	return false;
}


void F3DAttachTrackEditor::ActorSocketPicked(const FName SocketName, USceneComponent* Component, FActorPickerID ActorPickerID, TArray<FGuid> ObjectGuids, UMovieSceneSection* Section)
{
	if (Section != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoSetAttach", "Set Attach"));

		UMovieScene3DAttachSection* AttachSection = (UMovieScene3DAttachSection*)(Section);

		FMovieSceneObjectBindingID ConstraintBindingID;

		if (ActorPickerID.ExistingBindingID.IsValid())
		{
			ConstraintBindingID = ActorPickerID.ExistingBindingID;
		}
		else if (AActor* Actor = ActorPickerID.ActorPicked.Get())
		{
			TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

			TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
			if (Spawnable.IsSet())
			{
				// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
				ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(SequencerPtr->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, *SequencerPtr);
			}
			else
			{
				FGuid ParentActorId = FindOrCreateHandleToObject(Actor).Handle;
				ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(ParentActorId);
			}
		}

		if (ConstraintBindingID.IsValid())
		{
			AttachSection->SetConstraintBindingID(ConstraintBindingID);
		}

		AttachSection->AttachSocketName = SocketName;			
		AttachSection->AttachComponentName = Component ? Component->GetFName() : NAME_None;
	}
	else
	{
		TArray<TWeakObjectPtr<>> OutObjects;

		for (FGuid ObjectGuid : ObjectGuids)
		{
			if (ObjectGuid.IsValid())
			{
				for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectGuid))
				{
					OutObjects.Add(Object);
				}
			}
		}

		AnimatablePropertyChanged( FOnKeyProperty::CreateRaw( this, &F3DAttachTrackEditor::AddKeyInternal, OutObjects, SocketName, Component ? Component->GetFName() : NAME_None, ActorPickerID) );
	}
}

void F3DAttachTrackEditor::FindOrCreateTransformTrack(const TRange<FFrameNumber>& InAttachRange, UMovieScene* InMovieScene, const FGuid& InObjectHandle, UMovieScene3DTransformTrack*& OutTransformTrack, UMovieScene3DTransformSection*& OutTransformSection)
{
	OutTransformTrack = nullptr;
	OutTransformSection = nullptr;

	FName TransformPropertyName("Transform");

	// Create a transform track if it doesn't exist
	UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(InMovieScene->FindTrack<UMovieScene3DTransformTrack>(InObjectHandle));
	if (!TransformTrack)
	{
		InMovieScene->Modify();
		FFindOrCreateTrackResult TransformTrackResult = FindOrCreateTrackForObject(InObjectHandle, UMovieScene3DTransformTrack::StaticClass());
		TransformTrack = Cast<UMovieScene3DTransformTrack>(TransformTrackResult.Track);

		if (TransformTrack)
		{
			TransformTrack->SetPropertyNameAndPath(TransformPropertyName, TransformPropertyName.ToString());
		}
	}

	if (!TransformTrack)
	{
		return;
	}

	// Create a transform section if it doesn't exist
	UMovieScene3DTransformSection* TransformSection = nullptr;
	if (TransformTrack->IsEmpty())
	{
		TransformTrack->Modify();
		TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());
		if (TransformSection)
		{
			TransformSection->SetRange(TRange<FFrameNumber>::All());

			TransformTrack->AddSection(*TransformSection);
		}
	}
	// Reuse the transform section if it overlaps and check if there are no keys
	else if (TransformTrack->GetAllSections().Num() == 1)
	{
		TRange<FFrameNumber> TransformRange = TransformTrack->GetAllSections()[0]->GetRange();
		if (TRange<FFrameNumber>::Intersection(InAttachRange, TransformRange).IsEmpty())
		{
			return;
		}

		TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->GetAllSections()[0]);
	}

	OutTransformTrack = TransformTrack;
	OutTransformSection = TransformSection;
}

/**
 * Helper method to safely return an array allocated to store the proper the number of float channels if not already allocated
 */
TArray<FMovieSceneDoubleValue>& ResizeAndAddKey(const FFrameNumber& InKey, int32 InNum, TMap<FFrameNumber, TArray<FMovieSceneDoubleValue>>& OutTransformMap, TSet<FFrameNumber>* OutTimesAdded)
{
	TArray<FMovieSceneDoubleValue>& Transform = OutTransformMap.FindOrAdd(InKey);
	if (Transform.Num() == 0)
	{
		Transform.SetNum(InNum);
		if (OutTimesAdded)
		{
			OutTimesAdded->Add(InKey);
		}
	}
	return Transform;
}

/**
 * Helper method which adds keys from a list of float channels to a map mapping the time to a full transform
 */
void AddKeysFromChannels(TArrayView<FMovieSceneDoubleChannel*> InChannels, const TRange<FFrameNumber>& InAttachRange, TMap<FFrameNumber, TArray<FMovieSceneDoubleValue>>& OutTransformMap, TSet<FFrameNumber>& OutTimesAdded)
{
	const int32 NumChannels = 9;
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
	{
		TArray<FFrameNumber> TimesInRange;
		InChannels[ChannelIndex]->GetKeys(InAttachRange, &TimesInRange, nullptr);
		if (TimesInRange.Num() == 0)
		{
			continue;
		}

		const int32 BeginRangeIndex = InChannels[ChannelIndex]->GetTimes().FindLastByPredicate(
			[FirstKey = TimesInRange[0]](const FFrameNumber& FrameNum) { return FrameNum.Value == FirstKey.Value; });
		if (BeginRangeIndex == INDEX_NONE)
		{
			continue;
		}

		const int32 NumValsInRange = TimesInRange.Num();
		TArrayView<const FMovieSceneDoubleValue> ValuesInRange = InChannels[ChannelIndex]->GetValues().Slice(BeginRangeIndex, NumValsInRange);
		for (int32 KeyIndex = 0; KeyIndex < ValuesInRange.Num(); KeyIndex++)
		{
			TArray<FMovieSceneDoubleValue>& Transform = ResizeAndAddKey(TimesInRange[KeyIndex], InChannels.Num(), OutTransformMap, &OutTimesAdded);
			Transform[ChannelIndex] = ValuesInRange[KeyIndex];
		}
	}
}

/**
 * Helper method which updates the values in each channel in a list of movie scene float values given a 
 * transform, preserving the interpolation style and other attributes
 */
void UpdateDoubleValueTransform(const FTransform& InTransform, TArrayView<FMovieSceneDoubleValue> OutDoubleValueTransform)
{
	OutDoubleValueTransform[0].Value = InTransform.GetTranslation().X;
	OutDoubleValueTransform[1].Value = InTransform.GetTranslation().Y;
	OutDoubleValueTransform[2].Value = InTransform.GetTranslation().Z;

	OutDoubleValueTransform[3].Value = InTransform.GetRotation().Euler().X;
	OutDoubleValueTransform[4].Value = InTransform.GetRotation().Euler().Y;
	OutDoubleValueTransform[5].Value = InTransform.GetRotation().Euler().Z;

	OutDoubleValueTransform[6].Value = InTransform.GetScale3D().X;
	OutDoubleValueTransform[7].Value = InTransform.GetScale3D().Y;
	OutDoubleValueTransform[8].Value = InTransform.GetScale3D().Z;
}

/**
 * Helper method which converts a list of float values to a transform
 */
FORCEINLINE FTransform DoubleValuesToTransform(TArrayView<const FMovieSceneDoubleValue> InDoubleValues)
{
	return FTransform(FRotator::MakeFromEuler(FVector(InDoubleValues[3].Value, InDoubleValues[4].Value, InDoubleValues[5].Value)), 
		FVector(InDoubleValues[0].Value, InDoubleValues[1].Value, InDoubleValues[2].Value), FVector(InDoubleValues[6].Value, InDoubleValues[7].Value, InDoubleValues[8].Value));
}

/**
 * Evaluates the transform of an object at a certain point in time
 */
FTransform GetLocationAtTime(UMovieScene3DTransformTrack* TransformTrack, FFrameNumber KeyTime, UE::MovieScene::FSystemInterrogator& Interrogator)
{
	Interrogator.Reset();
	Interrogator.ImportTrack(TransformTrack, UE::MovieScene::FInterrogationChannel::Default());
	Interrogator.AddInterrogation(KeyTime);
	Interrogator.Update();

	TArray<FTransform> Transforms;
	Interrogator.QueryWorldSpaceTransforms(UE::MovieScene::FInterrogationChannel::Default(), Transforms);

	if (Transforms.Num())
	{
		FTransform Transform(Transforms[0].GetRotation(), Transforms[0].GetTranslation(), Transforms[0].GetScale3D());
		return Transform;
	}

	return FTransform::Identity;
}

UObject* GetConstraintObject(TSharedPtr<ISequencer> InSequencer, const FMovieSceneObjectBindingID& InConstraintBindingID)
{
	TArrayView<TWeakObjectPtr<UObject>> RuntimeObjects = InConstraintBindingID.ResolveBoundObjects(InSequencer->GetFocusedTemplateID(), *InSequencer);

	if (RuntimeObjects.Num() >= 1 && RuntimeObjects[0].IsValid())
	{
		if (AActor* Actor = Cast<AActor>(RuntimeObjects[0].Get()))
		{
			return Actor->GetRootComponent();
		}
		return RuntimeObjects[0].Get();
	}

	return nullptr;
}

struct ITransformEvaluator
{
	virtual FTransform operator()(const FFrameNumber& InTime) const { return FTransform::Identity; };
	virtual ~ITransformEvaluator() {}
};

/**
 * Helper functor for evaluating the local transform for an object.
 * It can be animated by sequencer but does not have to be
 */
struct FLocalTransformEvaluator : ITransformEvaluator
{
	FLocalTransformEvaluator() = default;

	/**
	 * Creates an evaluator for an object. Uses the evaluation track if it exists, otherwise uses the actor's transform
	 */
	FLocalTransformEvaluator(TWeakPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, UObject* InObject)
		: WeakAttachTrackEditor(InWeakAttachTrackEditor)
	{
		TSharedPtr<F3DAttachTrackEditor> AttachTrackEditor = InWeakAttachTrackEditor.Pin();
		if (!AttachTrackEditor)
		{
			return;
		}

		ISequencer* Sequencer = AttachTrackEditor->GetSequencer().Get();
		if (!Sequencer)
		{
			return;
		}

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

		USceneComponent* SceneComponent = Cast<USceneComponent>(InObject);
		if (AActor* Actor = Cast<AActor>(InObject))
		{
			SceneComponent = Actor->GetRootComponent();
		}

		if (!SceneComponent)
		{
			return;
		}

		FTransform ComponentTransform = SceneComponent->GetComponentTransform();
		TransformEval.SetSubtype<FTransform>(ComponentTransform);

		FGuid ObjectHandle = Sequencer->GetHandleToObject(InObject, false);
		if (ObjectHandle.IsValid())
		{
			UMovieScene3DTransformTrack* ActorTransformTrack = Cast<UMovieScene3DTransformTrack>(MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectHandle));
			if (ActorTransformTrack)
			{
				TransformEval.SetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>(TTuple<UMovieScene3DTransformTrack*, UObject*>(ActorTransformTrack, SceneComponent));
			}
		}
	}

	/**
	 * Creates an evaluator for an object with an already existing evaluation track
	 */
	FLocalTransformEvaluator(TWeakPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, UObject* InObject, UMovieScene3DTransformTrack* InTransformTrack)
		: WeakAttachTrackEditor(InWeakAttachTrackEditor)
	{
		TransformEval.SetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>(TTuple<UMovieScene3DTransformTrack*, UObject*>(InTransformTrack, InObject));
	}

	/**
	 * Evaluates the transform for this object at the given time
	 */
	FTransform operator()(const FFrameNumber& InTime) const override
	{
		TSharedPtr<F3DAttachTrackEditor> AttachTrackEditor = WeakAttachTrackEditor.Pin();
		if (!AttachTrackEditor)
		{
			return FTransform::Identity;
		}

		ISequencer* Sequencer = AttachTrackEditor->GetSequencer().Get();
		if (!Sequencer)
		{
			return FTransform::Identity;
		}
		
		const bool bEvalParentTransform = TransformEval.GetCurrentSubtypeIndex() == 1;
		if (bEvalParentTransform)
		{
			UMovieScene3DTransformTrack* TransformTrack = TransformEval.GetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>().Get<0>();
		 	return GetLocationAtTime(TransformTrack, InTime, *WeakAttachTrackEditor.Pin()->Interrogator.Get());
		}

		return TransformEval.GetSubtype<FTransform>();
	}

private:
	TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>> TransformEval;
	TWeakPtr<F3DAttachTrackEditor> WeakAttachTrackEditor;
};

/**
 * Helper functor for finding the world transform of actors
 * World transform evaluator gets the world transform of an object during an animation by accumulating the transforms of its parents.
 * The parents can be animated by sequencer but do not have to be
 */
struct FWorldTransformEvaluator : ITransformEvaluator
{
	FWorldTransformEvaluator() = default;

	/**
	 * Creates a new evaluator for a given object
	 * @param InSocketName is the socket to evaluate for if this is a skeletal mesh
	 */
	FWorldTransformEvaluator(TWeakPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, UObject* InObject, const FName InSocketName = NAME_None, const FName InComponentName = NAME_None)
		: WeakAttachTrackEditor(InWeakAttachTrackEditor)
	{
		TSharedPtr<F3DAttachTrackEditor> AttachTrackEditor = InWeakAttachTrackEditor.Pin();
		if (!AttachTrackEditor)
		{
			return;
		}

		ISequencer* Sequencer = AttachTrackEditor->GetSequencer().Get();
		if (!Sequencer)
		{
			return;
		}

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

		FName SocketName = InSocketName;
		FName ComponentName = InComponentName;

		USceneComponent* SceneComponent = Cast<USceneComponent>(InObject);
		if (AActor* Actor = Cast<AActor>(InObject))
		{
			UE::MovieScene::FComponentAttachParamsDestination AttachParams;
			AttachParams.SocketName = SocketName;
			AttachParams.ComponentName = ComponentName;

			SceneComponent = AttachParams.ResolveAttachment(Actor);
		}

		if (!SceneComponent)
		{
			return;
		}

		// Loop through all parents to get an accumulated array of evaluators
		do
		{
			TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>> ActorEval;
			// If we find a socket, get the world transform of the socket and break out immediately
			if (SceneComponent->DoesSocketExist(SocketName))
			{
				const FTransform SocketWorldSpace = SceneComponent->GetSocketTransform(SocketName);
				ActorEval.SetSubtype<FTransform>(SocketWorldSpace);
				TransformEvals.Add(ActorEval);
				return;
			}
			
			FTransform ComponentTransform = SceneComponent->GetComponentTransform();
			ActorEval.SetSubtype<FTransform>(ComponentTransform);

			FGuid ObjectHandle = Sequencer->GetHandleToObject(SceneComponent, false);
			if (ObjectHandle.IsValid())
			{
				UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectHandle);
				if (TransformTrack)
				{
					ActorEval.SetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>(TTuple<UMovieScene3DTransformTrack*, UObject*>(TransformTrack, SceneComponent));
				}
			}

			TransformEvals.Add(ActorEval);

			SceneComponent = SceneComponent->GetAttachParent();
			if (SceneComponent)
			{
				SocketName = SceneComponent->GetAttachSocketName();
			}
		} 
		while (SceneComponent);
	}

	/**
	 * Copies the array of all individual actor evaluators to create a new evaluator
	 */
	FWorldTransformEvaluator(TWeakPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, TArrayView<const TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>>> InTransformEvals)
		: WeakAttachTrackEditor(InWeakAttachTrackEditor)
	{
		Algo::Copy(InTransformEvals, TransformEvals);
	}

	/**
	 * Adds an evaluation track for the child of the first transform evaluator
	 */
	void PrependTransformEval(UObject* InObject, UMovieScene3DTransformTrack* InTransformTrack)
	{
		TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>> ActorEval;
		ActorEval.SetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>(TTuple<UMovieScene3DTransformTrack*, UObject*>(InTransformTrack, InObject));
		TransformEvals.Insert(ActorEval, 0);
	}

	/**
	 *  Adds a transform for the child of the first transform evaluator
	 */
	void PrependTransformEval(const FTransform& InTransform)
	{
		TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>> ActorEval;
		ActorEval.SetSubtype<FTransform>(InTransform);
		TransformEvals.Insert(ActorEval, 0);
	}

	/**
	 * Evaluates the world transform for this object at a certain time
	 */
	FTransform operator()(const FFrameNumber& InTime) const override
	{
		TSharedPtr<F3DAttachTrackEditor> AttachTrackEditor = WeakAttachTrackEditor.Pin();
		if (!AttachTrackEditor)
		{
			return FTransform::Identity;
		}

		ISequencer* Sequencer = AttachTrackEditor->GetSequencer().Get();
		if (!Sequencer)
		{
			return FTransform::Identity;
		}

		FTransform Accumulated = FTransform::Identity;
		for (TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>> TransformEval : TransformEvals)
		{
			FTransform ActorTransform;
			const bool bEvalParentTransform = TransformEval.GetCurrentSubtypeIndex() == 1;
			if (bEvalParentTransform)
			{
				UMovieScene3DTransformTrack* TransformTrack = TransformEval.GetSubtype<TTuple<UMovieScene3DTransformTrack*, UObject*>>().Get<0>();
				ActorTransform = GetLocationAtTime(TransformTrack, InTime, *WeakAttachTrackEditor.Pin()->Interrogator.Get());
			}
			else
			{
				ActorTransform = TransformEval.GetSubtype<FTransform>();
			}

			Accumulated *= ActorTransform;
		}

		return Accumulated;
	}

	/**
	 * Gets the individual actor evaluators for each parent
	 */
	TArrayView <const TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>>> GetTransformEvalsView() const 
	{
		return TransformEvals;
	}

private:
	TArray<TUnion<FTransform, TTuple<UMovieScene3DTransformTrack*, UObject*>>> TransformEvals;
	TWeakPtr<F3DAttachTrackEditor> WeakAttachTrackEditor;
};

/**
 * Helper functor to revert transforms that are in the relative space of a constraint
 */
struct FAttachRevertModifier
{
	/**
	 * Constructor finds the constraint for the given attach section and finds the evaluation track/transform for it.
	 * @param bInFullRevert If true: Does a full revert with a simple compensation for the first frame,
	 *                               modifying the object's movements to how they were before the attach.
	 *                      If false: Parent's movement is kept and transforms are simply converted to world space.
	 */
	FAttachRevertModifier(TSharedPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, const TRange<FFrameNumber>& InRevertRange, UMovieScene3DAttachSection* InAttachSection, const FName InSocketName, const FName InComponentName, bool bInFullRevert)
		: bFullRevert(bInFullRevert)
		, RevertRange(InRevertRange)
	{
		FMovieSceneObjectBindingID ConstraintID = InAttachSection->GetConstraintBindingID();
		UObject* ConstraintObject = GetConstraintObject(InWeakAttachTrackEditor->GetSequencer(), ConstraintID);

		TransformEvaluator = FWorldTransformEvaluator(InWeakAttachTrackEditor, ConstraintObject, InSocketName, InComponentName);

		if (InRevertRange.HasLowerBound())
		{
			BeginConstraintTransform = TransformEvaluator(InRevertRange.GetLowerBoundValue());
		}
	}

	/**
	 * Creates a new revert modifier with a given evaluator for a parent transform to undo compensation
	 */
	FAttachRevertModifier(TSharedPtr<F3DAttachTrackEditor> InWeakAttachTrackEditor, const TRange<FFrameNumber>& InRevertRange, const FWorldTransformEvaluator& InTransformEvaluator, bool bInFullRevert)
		: bFullRevert(bInFullRevert)
		, TransformEvaluator(InTransformEvaluator)
		, RevertRange(InRevertRange)
	{
		if (InRevertRange.HasLowerBound())
		{
			BeginConstraintTransform = InTransformEvaluator(InRevertRange.GetLowerBoundValue());
		}
	}

	/** Reverts a transform in relative space to world space */
	FTransform operator()(const FTransform& InTransform, const FFrameNumber& InTime)
	{
		FTransform OutTransform = InTransform;

		FTransform ConstraintTransform = TransformEvaluator(InTime);

		// If in revert range, revert the transform to world coordinates first
		if (RevertRange.Contains(InTime))
		{
			OutTransform = OutTransform * ConstraintTransform;
		}

		if (bFullRevert)
		{
			const FTransform ConstraintChange = BeginConstraintTransform.GetRelativeTransform(ConstraintTransform);
			OutTransform = OutTransform * ConstraintChange;
		}

		return OutTransform;
	}

private:
	bool bFullRevert;

	FWorldTransformEvaluator TransformEvaluator;
	FTransform BeginConstraintTransform;
	TRange<FFrameNumber> RevertRange;
};

/**
 * Updates an array of float channels with the keys in a given transform map mapping times to float values
 */
void UpdateChannelTransforms(const TRange<FFrameNumber>& InAttachRange, TMap<FFrameNumber, TArray<FMovieSceneDoubleValue>>& InTransformMap, TArrayView<FMovieSceneDoubleChannel*>& InChannels, int32 InNumChannels, bool bInBakedData)
{
	// Remove all handles in range so we can add the new ones
	for (FMovieSceneDoubleChannel* Channel : InChannels)
	{
		TArray<FKeyHandle> KeysToRemove;
		Channel->GetKeys(InAttachRange, nullptr, &KeysToRemove);
		Channel->DeleteKeys(KeysToRemove);
	}

	// Find max extent of all channels
	TRange<FFrameNumber> TotalRange = TRange<FFrameNumber>(TNumericLimits<FFrameNumber>::Lowest(), TNumericLimits<FFrameNumber>::Max());
	TArray<TRange<FFrameNumber>> ExcludedRanges = TRange<FFrameNumber>::Difference(TotalRange, InAttachRange);

	InTransformMap.KeySort([](const FFrameNumber& LHS, const FFrameNumber& RHS) { return LHS.Value < RHS.Value; });
	TArray<FFrameNumber> NewKeyFrames;
	InTransformMap.GetKeys(NewKeyFrames);
	TArray<FMovieSceneDoubleValue> NewKeyValues;

	// Update keys in channels
	for (int32 ChannelIndex = 0; ChannelIndex < InNumChannels; ChannelIndex++)
	{
		NewKeyValues.Reset();
		Algo::Transform(InTransformMap, NewKeyValues, [ChannelIndex](const auto& Pair) { return Pair.Value[ChannelIndex]; });

		// All the keys in this channel must be sorted, as adding a set of keys in the curve model before all the others will cause problems.
		// In order to do this, we assume all 3 sets of keys (before attach range, in attach range, and after attach range) are already sorted
		// and simply remove and re-add all of the keys from first to last
		TArray<FKeyHandle> LowerKeyHandles, UpperKeyHandles;
		TArray<FFrameNumber> LowerKeyTimes, UpperKeyTimes;
		TArray<FMovieSceneDoubleValue> PrevKeyValues;
		Algo::Copy(InChannels[ChannelIndex]->GetValues(), PrevKeyValues);

		// Get the keys contained in before attach and after attach ranges
		InChannels[ChannelIndex]->GetKeys(ExcludedRanges[0], &LowerKeyTimes, &LowerKeyHandles);
		if (ExcludedRanges.Top().HasLowerBound())
		{
			ExcludedRanges.Top().SetLowerBound(TRangeBound<FFrameNumber>::Exclusive(ExcludedRanges.Top().GetLowerBoundValue()));
		}
		InChannels[ChannelIndex]->GetKeys(ExcludedRanges.Top(), &UpperKeyTimes, &UpperKeyHandles);

		// Add all keys before attach range if they exist
		int32 ValueIndex = 0;
		if (ExcludedRanges.Num() > 0 && 
			ExcludedRanges[0].HasUpperBound() &&
			InAttachRange.HasLowerBound() && 
			ExcludedRanges[0].GetUpperBoundValue() <= InAttachRange.GetLowerBoundValue() && LowerKeyTimes.Num() > 0)
		{
			InChannels[ChannelIndex]->DeleteKeys(LowerKeyHandles);
			TArray<FMovieSceneDoubleValue> ValuesToAdd;
			Algo::Copy(TArrayView<const FMovieSceneDoubleValue>(PrevKeyValues).Slice(ValueIndex, LowerKeyTimes.Num()), ValuesToAdd);
			InChannels[ChannelIndex]->AddKeys(LowerKeyTimes, ValuesToAdd);
			ValueIndex += LowerKeyTimes.Num();
		}

		// Add all keys in the attach range if they exist
		InChannels[ChannelIndex]->AddKeys(NewKeyFrames, NewKeyValues);

		// Add all keys after attach range if they exist
		if (ExcludedRanges.Num() > 0 && 
			ExcludedRanges.Top().HasLowerBound() && InAttachRange.HasUpperBound() &&
			ExcludedRanges.Top().GetLowerBoundValue() >= InAttachRange.GetUpperBoundValue() && ValueIndex < PrevKeyValues.Num() && UpperKeyTimes.Num() > 0)
		{
			InChannels[ChannelIndex]->DeleteKeys(UpperKeyHandles);
			TArray<FMovieSceneDoubleValue> ValuesToAdd;
			Algo::Copy(TArrayView<const FMovieSceneDoubleValue>(PrevKeyValues).Slice(ValueIndex, UpperKeyTimes.Num()), ValuesToAdd);
			InChannels[ChannelIndex]->AddKeys(UpperKeyTimes, ValuesToAdd);
		}

		// If the data is baked, then we also optimize the curves at this point, but do not set tangents since baked keys use linear interpolation
		if (bInBakedData)
		{
			FKeyDataOptimizationParams OptimizationParams;
			OptimizationParams.bAutoSetInterpolation = false;
			OptimizationParams.Range = InAttachRange;
			InChannels[ChannelIndex]->Optimize(OptimizationParams);
		}
		else
		{
			InChannels[ChannelIndex]->AutoSetTangents();
		}
	}
}

void F3DAttachTrackEditor::TrimAndPreserve(FGuid InObjectBinding, UMovieSceneSection* InSection, bool bInTrimLeft)
{
	// Find the transform track associated with the selected object
	UMovieScene3DTransformTrack* TransformTrack = GetMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(InObjectBinding);
	if (!TransformTrack || TransformTrack->GetAllSections().Num() != 1)
	{
		return;
	}

	TArrayView<TWeakObjectPtr<>> BoundObjects = GetSequencer()->FindBoundObjects(InObjectBinding, GetSequencer()->GetFocusedTemplateID());

	FQualifiedFrameTime QualifiedNewDetachTime = GetSequencer()->GetLocalTime();
	if (InSection && BoundObjects.Num() == 1 && BoundObjects[0].IsValid())
	{
		TRange<FFrameNumber> BeforeTrimRange = InSection->GetRange();
		const FScopedTransaction Transaction(LOCTEXT("TrimAttach", "Trim Attach"));

		UObject* Object = BoundObjects[0].Get();

		// Trim the section and find the range of the cut
		InSection->TrimSection(QualifiedNewDetachTime, bInTrimLeft, false);
		TArray<TRange<FFrameNumber>> ExcludedRanges = TRange<FFrameNumber>::Difference(BeforeTrimRange, InSection->GetRange());
		if (ExcludedRanges.Num() == 0)
		{
			return;
		}

		TRange<FFrameNumber> ExcludedRange = bInTrimLeft ? ExcludedRanges[0] : ExcludedRanges.Top();

		UMovieScene3DAttachSection* AttachSection = Cast<UMovieScene3DAttachSection>(InSection);
		check(AttachSection);

		// Create a revert modifier with the range and section as parameters
		FAttachRevertModifier RevertModifier(SharedThis(this), ExcludedRange, AttachSection, AttachSection->AttachSocketName, AttachSection->AttachComponentName, AttachSection->bFullRevertOnDetach);

		// Find the transform section associated with the track, so far we only support modifying transform tracks with one section
		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->GetAllSections()[0]);
		if (!TransformSection->TryModify())
		{
			return;
		}

		TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		FLocalTransformEvaluator LocalTransformEval(SharedThis(this), Object, TransformTrack);

		if (AttachSection->ReAttachOnDetach.IsValid())
		{
			FWorldTransformEvaluator ReAttachParentEvaluator(SharedThis(this), AttachSection->ReAttachOnDetach.Get());

			CompensateChildTrack(ExcludedRange, Channels, TOptional<TArrayView<FMovieSceneDoubleChannel*>>(), ReAttachParentEvaluator, LocalTransformEval, ETransformPreserveType::CurrentKey, RevertModifier);
		}
		else
		{
			TSet<FFrameNumber> KeyTimesToCompensate;
			TMap<FFrameNumber, TArray<FMovieSceneDoubleValue>> TransformMap;

			// Add all keys already existing in the range to the transform map
			AddKeysFromChannels(Channels, ExcludedRange, TransformMap, KeyTimesToCompensate);
			TArray<FFrameNumber> EdgeKeys;

			// Add the edge keys before and after the cut
			FFrameNumber RevertEdgeTime;
			FFrameNumber PreserveEdgeTime;
			if (bInTrimLeft)
			{
				if (ExcludedRange.HasUpperBound())
				{
					PreserveEdgeTime = ExcludedRange.GetUpperBoundValue();
				}
				else
				{
					PreserveEdgeTime = TNumericLimits<FFrameNumber>::Max();
				}
				RevertEdgeTime = PreserveEdgeTime.Value - 1;
				EdgeKeys = { PreserveEdgeTime, RevertEdgeTime };
				ResizeAndAddKey(PreserveEdgeTime, Channels.Num(), TransformMap, nullptr);
				ResizeAndAddKey(RevertEdgeTime, Channels.Num(), TransformMap, &KeyTimesToCompensate);
			}
			else
			{
				if (ExcludedRange.HasLowerBound())
				{
					RevertEdgeTime = ExcludedRange.GetLowerBoundValue();
				}
				else
				{
					RevertEdgeTime = TNumericLimits<FFrameNumber>::Min();
				}
				PreserveEdgeTime = RevertEdgeTime.Value - 1;
				EdgeKeys = { RevertEdgeTime, PreserveEdgeTime };
				ResizeAndAddKey(RevertEdgeTime, Channels.Num(), TransformMap, &KeyTimesToCompensate);
				ResizeAndAddKey(PreserveEdgeTime, Channels.Num(), TransformMap, nullptr);
			}

			// Evaluate the transform at all times with keys
			for (auto Itr = TransformMap.CreateIterator(); Itr; ++Itr)
			{
				UpdateDoubleValueTransform(LocalTransformEval(Itr->Key), Itr->Value);
			}

			// Modify each transform
			for (const FFrameNumber& CompTime : KeyTimesToCompensate)
			{
				const FTransform RevertedTransform = RevertModifier(DoubleValuesToTransform(TransformMap[CompTime]), CompTime);
				UpdateDoubleValueTransform(RevertedTransform, TransformMap[CompTime]);
			}

			// Manually set edge keys to have linear interpolation
			for (const FFrameNumber& EdgeKey : EdgeKeys)
			{
				for (FMovieSceneDoubleValue& Key : TransformMap[EdgeKey])
				{
					Key.InterpMode = ERichCurveInterpMode::RCIM_Linear;
				}
			}

			// Update the channels with the transform map
			UpdateChannelTransforms(ExcludedRange, TransformMap, Channels, 9, false);
		}

		// Remove previous boundary keys
		for (FMovieSceneDoubleChannel* Channel : Channels)
		{
			TArray<FKeyHandle> KeyAtTime;

			if (bInTrimLeft)
			{
				if (ExcludedRange.HasLowerBound())
				{
					Channel->GetKeys(TRange<FFrameNumber>::Inclusive(ExcludedRange.GetLowerBoundValue() - 1, ExcludedRange.GetLowerBoundValue() - 1), nullptr, &KeyAtTime);
				}
			}
			else
			{
				if (ExcludedRange.HasUpperBound())
				{
					Channel->GetKeys(TRange<FFrameNumber>::Inclusive(ExcludedRange.GetUpperBoundValue() - 1, ExcludedRange.GetUpperBoundValue() - 1), nullptr, &KeyAtTime);
				}
			}

			Channel->DeleteKeys(KeyAtTime);
			Channel->AutoSetTangents();
		}
	}
}

template<typename ModifierFuncType>
void F3DAttachTrackEditor::CompensateChildTrack(const TRange<FFrameNumber>& InAttachRange, TArrayView<FMovieSceneDoubleChannel*> Channels, TOptional<TArrayView<FMovieSceneDoubleChannel*>> ParentChannels,
	const ITransformEvaluator& InParentTransformEval, const ITransformEvaluator& InChildTransformEval,
	ETransformPreserveType InPreserveType, ModifierFuncType InModifyTransform)
{
	const FFrameNumber& KeyTime = InAttachRange.GetLowerBoundValue();
	const FFrameNumber& AttachEndTime = InAttachRange.GetUpperBoundValue();
	const int32 NumChannels = 9;

	TSet<FFrameNumber> KeyTimesToCompensate;
	TMap<FFrameNumber, TArray<FMovieSceneDoubleValue>> TransformMap;

	// Add all times with keys to the map
	if (PreserveType == ETransformPreserveType::Bake)
	{
		FFrameRate TickResolution = GetSequencer()->GetFocusedTickResolution();
		FFrameRate DisplayRate = GetSequencer()->GetFocusedDisplayRate();
		for (FFrameNumber FrameItr = InAttachRange.GetLowerBoundValue(); FrameItr < InAttachRange.GetUpperBoundValue(); 
			FrameItr += FMath::RoundToInt32(TickResolution.AsDecimal() / DisplayRate.AsDecimal()))
		{
			ResizeAndAddKey(FrameItr, Channels.Num(), TransformMap, &KeyTimesToCompensate);
			for (FMovieSceneDoubleValue& DoubleVal : TransformMap[FrameItr])
			{
				DoubleVal.InterpMode = ERichCurveInterpMode::RCIM_Linear;
			}
		}
	}
	else
	{
		AddKeysFromChannels(Channels, InAttachRange, TransformMap, KeyTimesToCompensate);
	}

	const bool bRangeEmpty = TransformMap.Num() == 0 || (TransformMap.Num() == 1 && TransformMap.CreateConstIterator()->Key.Value == KeyTime.Value);

	// Add keys at before and after attach times
	FFrameNumber BeforeAttachTime = KeyTime.Value - 1;
	FFrameNumber BeforeDetachTime = AttachEndTime.Value - 1;
	ResizeAndAddKey(BeforeAttachTime, Channels.Num(), TransformMap, nullptr);
	ResizeAndAddKey(KeyTime, Channels.Num(), TransformMap, &KeyTimesToCompensate);
	ResizeAndAddKey(BeforeDetachTime, Channels.Num(), TransformMap, &KeyTimesToCompensate);
	ResizeAndAddKey(AttachEndTime, Channels.Num(), TransformMap, nullptr);

	if (PreserveType == ETransformPreserveType::AllKeys && ParentChannels.IsSet())
	{
		AddKeysFromChannels(ParentChannels.GetValue(), InAttachRange, TransformMap, KeyTimesToCompensate);
	}

	KeyTimesToCompensate.Remove(AttachEndTime);
	KeyTimesToCompensate.Remove(BeforeAttachTime);
	TArray<FFrameNumber> EdgeKeys = { BeforeAttachTime, KeyTime, AttachEndTime, BeforeDetachTime };

	// Evaluate the transform at all times with keys
	for (auto Itr = TransformMap.CreateIterator(); Itr; ++Itr)
	{
		const FTransform TempTransform = InChildTransformEval(Itr->Key);
		UpdateDoubleValueTransform(TempTransform, Itr->Value);
	}

	if (InPreserveType == ETransformPreserveType::AllKeys || InPreserveType == ETransformPreserveType::Bake)
	{
		// If the parent has a transform track, evaluate it's transform at each of the key times found above and calculate the diffs with its child
		for (const FFrameNumber& CompTime : KeyTimesToCompensate)
		{
			const FTransform ParentTransformAtTime = InParentTransformEval(CompTime);
			const FTransform NewTransform = InModifyTransform(DoubleValuesToTransform(TransformMap[CompTime]), CompTime);
			const FTransform RelativeTransform = NewTransform.GetRelativeTransform(ParentTransformAtTime);
			UpdateDoubleValueTransform(RelativeTransform, TransformMap[CompTime]);
		}
	}
	else if (InPreserveType == ETransformPreserveType::CurrentKey)
	{
		// Find the relative transform on the first frame of the attach
		const FTransform BeginChildTransform = InModifyTransform(DoubleValuesToTransform(TransformMap[KeyTime]), KeyTime);
		const FTransform BeginParentTransform = InParentTransformEval(KeyTime);

		const FTransform BeginRelativeTransform = BeginChildTransform.GetRelativeTransform(BeginParentTransform);

		// offset each transform by initial relative transform calculated before
		for (const FFrameNumber& CompTime : KeyTimesToCompensate)
		{
			const FTransform ChildTransformAtTime = InModifyTransform(DoubleValuesToTransform(TransformMap[CompTime]), CompTime);
			const FTransform StartToCurrentTransform = ChildTransformAtTime.GetRelativeTransform(BeginChildTransform);

			UpdateDoubleValueTransform(BeginRelativeTransform * StartToCurrentTransform, TransformMap[CompTime]);
		}

		const FTransform EndParentTransform = InParentTransformEval(AttachEndTime);
		UpdateDoubleValueTransform(EndParentTransform * DoubleValuesToTransform(TransformMap[BeforeDetachTime]), TransformMap[AttachEndTime]);
	}

	// Manually set edge keys to have linear interpolation
	for (const FFrameNumber& EdgeKey : EdgeKeys)
	{
		for (FMovieSceneDoubleValue& Key : TransformMap[EdgeKey])
		{
			Key.InterpMode = ERichCurveInterpMode::RCIM_Linear;
		}
	}

	UpdateChannelTransforms(InAttachRange, TransformMap, Channels, NumChannels, PreserveType == ETransformPreserveType::Bake);
}

FKeyPropertyResult F3DAttachTrackEditor::AddKeyInternal( FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, const FName SocketName, const FName ComponentName, FActorPickerID ActorPickerID)
{
	FKeyPropertyResult KeyPropertyResult;

	FMovieSceneObjectBindingID ConstraintBindingID;

	if (ActorPickerID.ExistingBindingID.IsValid())
	{
		ConstraintBindingID = ActorPickerID.ExistingBindingID;
	}
	else if (AActor* Actor = ActorPickerID.ActorPicked.Get())
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

		TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
		if (Spawnable.IsSet())
		{
			// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
			ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(SequencerPtr->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, *SequencerPtr);
		}
		else
		{
			FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(ActorPickerID.ActorPicked.Get());
			KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
			ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(HandleResult.Handle);
		}
	}

	if (!ConstraintBindingID.IsValid())
	{
		return KeyPropertyResult;
	}

	FMovieSceneSequenceID CurrentSequenceID = GetSequencer()->GetFocusedTemplateID();
	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
	TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

	// It's possible that the objects bound to this parent binding ID are null, in which case there will be no compensation
	UObject* ParentObject = GetConstraintObject(GetSequencer(), ConstraintBindingID);	

	FWorldTransformEvaluator ParentTransformEval(SharedThis(this), ParentObject, SocketName);

	TOptional<TArrayView<FMovieSceneDoubleChannel*>> ParentChannels;

	// If the constraint exists within this sequence, we can perform transform compensation
	if (ConstraintBindingID.ResolveSequenceID(CurrentSequenceID,*GetSequencer()) == CurrentSequenceID)
	{
		UMovieScene3DTransformTrack* ParentTransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ConstraintBindingID.GetGuid());
		if (ParentTransformTrack && ParentTransformTrack->GetAllSections().Num() == 1)
		{
			ParentChannels = ParentTransformTrack->GetAllSections()[0]->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		}
	}

	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		UObject* Object = Objects[ObjectIndex].Get();

		// Disallow attaching an object to itself
		if (Object == ParentObject)
		{
			continue;
		}

		// Get handle to object
		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
		if (!ObjectHandle.IsValid())
		{
			continue;
		}

		// Get attach Track for Object
		FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieScene3DAttachTrack::StaticClass());
		UMovieSceneTrack* Track = TrackResult.Track;
		KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;
		if (!ensure(Track))
		{
			continue;
		}

		// Clamp to next attach section's start time or the end of the current movie scene range
		FFrameNumber AttachEndTime = PlaybackRange.GetUpperBoundValue();
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			FFrameNumber StartTime = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : 0;
			if (KeyTime < StartTime)
			{
				if (AttachEndTime > StartTime)
				{
					AttachEndTime = StartTime;
				}
			}
		}

		int32 Duration = FMath::Max(0, (AttachEndTime - KeyTime).Value);

		// Just add the constraint section if no preservation should be done
		if (PreserveType == ETransformPreserveType::None)
		{
			Track->Modify();

			UMovieSceneSection* ExistingSection = MovieSceneHelpers::FindSectionAtTime(Track->GetAllSections(), KeyTime);
			UMovieSceneSection* NextSection = MovieSceneHelpers::FindNextSection(Track->GetAllSections(), KeyTime);

			// If there's an existing section, clamp it to the start time of the new section
			if (ExistingSection)
			{
				ExistingSection->Modify();
				ExistingSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(KeyTime));
			}
			
			UMovieSceneSection* NewSection = Cast<UMovieScene3DAttachTrack>(Track)->AddConstraint(KeyTime, Duration, SocketName, ComponentName, ConstraintBindingID);
			
			// If there isn't an existing section, make the start frame of the new section infinite
			if (!ExistingSection)
			{
				NewSection->SectionRange.Value.SetLowerBound(TRangeBound<FFrameNumber>::Open());
			}

			// If there's a next section, make the end frame of the new section the start frame of the next section
			if (NextSection)
			{
				NewSection->SectionRange.Value.SetUpperBound(NextSection->GetInclusiveStartFrame());
			}
			// Otherwise, make the end frame of the new section infinite
			else
			{
				NewSection->SectionRange.Value.SetUpperBound(TRangeBound<FFrameNumber>::Open());
			}

			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			continue;
		}

		// Create a blank world transform evaluator, add parent evaluator if there is a parent
		FWorldTransformEvaluator WorldChildTransformEval(SharedThis(this), nullptr);

		USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
		if (AActor* Actor = Cast<AActor>(Object))
		{
			SceneComponent = Actor->GetRootComponent();
		}

		if (SceneComponent)
		{
			if (USceneComponent* PrevAttachParent = SceneComponent->GetAttachParent())
			{
				WorldChildTransformEval = FWorldTransformEvaluator(SharedThis(this), PrevAttachParent);
			}
		}

		// Create transform track for object
		TRange<FFrameNumber> AttachRange(KeyTime, AttachEndTime);
		UMovieScene3DTransformTrack* TransformTrack = nullptr;
		UMovieScene3DTransformSection* TransformSection = nullptr;
		FindOrCreateTransformTrack(AttachRange, MovieScene, ObjectHandle, TransformTrack, TransformSection);

		if (TransformTrack)
		{
			WorldChildTransformEval.PrependTransformEval(Object, TransformTrack);
		}
		else if (SceneComponent)
		{
			WorldChildTransformEval.PrependTransformEval(SceneComponent->GetComponentTransform());
		}

		if (!TransformSection || !TransformTrack)
		{
			continue;
		}

		if (!TransformSection->TryModify())
		{
			continue;
		}

		// Get transform track channels
		TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		// find intersecting section
		TOptional<UMovieSceneSection*> IntersectingSection;
		if (Track->GetAllSections().Num() > 0)
		{
			for (UMovieSceneSection* OtherSection : Track->GetAllSections())
			{
				if (OtherSection->GetRange().Contains(KeyTime))
				{
					IntersectingSection = OtherSection;
					break;
				}
			}
		}

		FFrameRate TickResolution = Track->GetTypedOuter<UMovieScene>()->GetTickResolution();
		
		Track->Modify();
		KeyPropertyResult.bTrackModified = true;
		KeyPropertyResult.bKeyCreated = true;

		TOptional<FAttachRevertModifier> RevertModifier;
		USceneComponent* ReAttachOnDetach = nullptr;

		// If there are existing channels, revert the transform from the previous parent's transform before setting the new relative transform
		// Currently don't handle objects with both other attach sections and are already attached to other objects because its hard to think about
		if (IntersectingSection)
		{
			// Calculate range to revert
			FFrameNumber IntersectionSectionStart = IntersectingSection.GetValue()->HasStartFrame() ? IntersectingSection.GetValue()->GetInclusiveStartFrame() : PlaybackRange.GetLowerBoundValue();
			FFrameNumber IntersectionSectionEnd = IntersectingSection.GetValue()->HasEndFrame() ? IntersectingSection.GetValue()->GetExclusiveEndFrame() : PlaybackRange.GetUpperBoundValue();

			TRange<FFrameNumber> RevertRange = TRange<FFrameNumber>(KeyTime, FMath::Min(AttachEndTime, IntersectionSectionEnd));

			// If the intersecting section starts at the same time as the new section, remove it
			if (IntersectionSectionStart == KeyTime)
			{
				Track->RemoveSection(*IntersectingSection.GetValue());
			}
			// Otherwise trim the end frame of the intersecting section
			else
			{
				if (!IntersectingSection.GetValue()->TryModify())
				{
					continue;
				}
				
				IntersectingSection.GetValue()->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(KeyTime));
			}

			UMovieScene3DAttachSection* IntersectingAttachSection = Cast<UMovieScene3DAttachSection>(IntersectingSection.GetValue());
			if (!IntersectingAttachSection)
			{
				continue;
			}

			RevertModifier = FAttachRevertModifier(SharedThis(this), RevertRange, IntersectingAttachSection, SocketName, ComponentName, PreserveType == ETransformPreserveType::CurrentKey);
		}
		// Existing parent that's not an attach track
		else if (WorldChildTransformEval.GetTransformEvalsView().Num() > 1)
		{
			// Calculate range to revert
			TRange<FFrameNumber> RevertRange = AttachRange;

			// Get the evaluator for the previous parent track
			const int32 NumChildEvals = WorldChildTransformEval.GetTransformEvalsView().Num();
			auto PrevParentTransformEvals = WorldChildTransformEval.GetTransformEvalsView().Slice(1, NumChildEvals - 1);
			FWorldTransformEvaluator PrevParentEvaluator(SharedThis(this), PrevParentTransformEvals);
		
			RevertModifier = FAttachRevertModifier(SharedThis(this), RevertRange, PrevParentEvaluator, PreserveType == ETransformPreserveType::CurrentKey);

			if (SceneComponent)
			{
				if (USceneComponent* PrevAttachParent = SceneComponent->GetAttachParent())
				{
					ReAttachOnDetach = PrevAttachParent;
				}
			}
		}

		if (RevertModifier.IsSet())
		{
			FLocalTransformEvaluator LocalChildTransformEval(SharedThis(this), Object, TransformTrack);

			// Add the new attach section to the track
			UMovieSceneSection* NewSection = Cast<UMovieScene3DAttachTrack>(Track)->AddConstraint(KeyTime, Duration, SocketName, ComponentName, ConstraintBindingID);
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			// Compensate
			CompensateChildTrack(AttachRange, Channels, ParentChannels, ParentTransformEval, LocalChildTransformEval, PreserveType, RevertModifier.GetValue());
		}
		else
		{
			// Add the new attach section to the track
			UMovieSceneSection* NewSection = Cast<UMovieScene3DAttachTrack>(Track)->AddConstraint(KeyTime, Duration, SocketName, ComponentName, ConstraintBindingID);
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			// Compensate
			CompensateChildTrack(AttachRange, Channels, ParentChannels, ParentTransformEval, WorldChildTransformEval, PreserveType, [&](const FTransform& InTransform, const FFrameNumber& InTime) { return InTransform; });
		}

		Cast<UMovieScene3DAttachSection>(Cast<UMovieScene3DAttachTrack>(Track)->GetAllSections().Top())->bFullRevertOnDetach = (PreserveType == ETransformPreserveType::CurrentKey);
		Cast<UMovieScene3DAttachSection>(Cast<UMovieScene3DAttachTrack>(Track)->GetAllSections().Top())->ReAttachOnDetach = ReAttachOnDetach;
	} // for

	return KeyPropertyResult;
}

#undef LOCTEXT_NAMESPACE
