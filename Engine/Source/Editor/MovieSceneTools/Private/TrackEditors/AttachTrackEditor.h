// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "TrackEditors/ActorPickerTrackEditor.h"
#include "Containers/Union.h"
#include "Containers/Map.h"
#include "Templates/Function.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

struct FMovieSceneDoubleChannel;

class AActor;
class FMenuBuilder;
class USceneComponent;
class UMovieScene3DTransformTrack;
class UMovieScene3DTransformSection;
class UMovieScene3DAttachSection;

struct ITransformEvaluator;

enum class ETransformPreserveType
{
	CurrentKey,
	AllKeys,
	Bake,
	None
};

/**
 * Tools for attaching an object to another object
 */
class F3DAttachTrackEditor
	: public FActorPickerTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	F3DAttachTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~F3DAttachTrackEditor();

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface

	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	// FTrackEditorActorPicker
	virtual bool IsActorPickable( const AActor* const ParentActor, FGuid ObjectBinding, UMovieSceneSection* InSection ) override;
	virtual void ActorSocketPicked(const FName SocketName, USceneComponent* Component, FActorPickerID ActorPickerID, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section) override;

	/** Trims an attach track on the left/right and preserves the world space transform immediately before/after attach/detach */
	void TrimAndPreserve(const FGuid InObjectBinding, UMovieSceneSection* InSection, bool bInTrimLeft);

private:

	void ShowPickerSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section);

	/** Helper for AddKeyInternal to get transform tracks for child */
	void FindOrCreateTransformTrack(const TRange<FFrameNumber>& InAttachRange, UMovieScene* InMovieScene, const FGuid& InObjectHandle, UMovieScene3DTransformTrack*& OutTransformTrack, UMovieScene3DTransformSection*& OutTransformSection);

	/** Helper for AddKeyInternal to offset child track's keys */
	template<typename ModifierFuncType>
	void CompensateChildTrack(const TRange<FFrameNumber>& InAttachRange, TArrayView<FMovieSceneDoubleChannel*> Channels, TOptional<TArrayView<FMovieSceneDoubleChannel*>> ParentChannels,
		const ITransformEvaluator& InParentTransformEval, const ITransformEvaluator& InChildTransformEval, ETransformPreserveType InPreserveType, ModifierFuncType InModifyTransform);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, const FName SocketName, const FName ComponentName, FActorPickerID ActorPickerID);

private:
	ETransformPreserveType PreserveType;

public:
	TUniquePtr<UE::MovieScene::FSystemInterrogator> Interrogator;
};
