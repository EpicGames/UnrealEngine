// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/AudioTrackEditor.h"

#include "AnimatedRange.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Audio.h"
#include "AudioDecompress.h"
#include "AudioDevice.h"
#include "ContentBrowserModule.h"
#include "Decorations/MovieSceneLanguagePreviewDecoration.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDetailCustomization.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Internationalization/TextLocalizationManager.h"
#include "ISectionLayoutBuilder.h"
#include "ISequencer.h"
#include "Misc/QualifiedFrameTime.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "MovieSceneToolHelpers.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneAudioSection.h"
#include "SequencerSectionPainter.h"
#include "SequencerSettings.h"
#include "Slate/SlateTextures.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundWave.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateTextureData.h"
#include "TimeToPixel.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FAudioTrackEditor"

static bool bOutlineAudioActorAttachments = false;

static FAutoConsoleVariableRef CVarOutlineAudioActorAttachments(
	TEXT("au.OutlineAudioActorAttachments"),
	bOutlineAudioActorAttachments,
	TEXT("Enables red outlines of audio sections in sequencer when the actor binding is broken. Off by default\n")
	TEXT("0: Disable red outlines on audio sections\n")
	TEXT("1: Enable red outlines on audio sections\n"),
	ECVF_Default
);

// Tolerance for Section vs. Sound duration to be considered nearly equal.
constexpr float SectionToSoundDurationToleranceInSeconds = 0.1f;

namespace AnimatableAudioEditorConstants
{
	// Optimization - maximum samples per pixel this sound allows
	const uint32 MaxSamplesPerPixel = 60;
}

namespace UE::Sequencer::AudioTrack
{
	class SLanguagePreviewComboWidget : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SLanguagePreviewComboWidget){}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UMovieScene> InWeakMovieScene, TWeakObjectPtr<UMovieSceneTrack> InWeakAudioTrack)
		{
			WeakMovieScene = InWeakMovieScene;
			WeakAudioTrack = InWeakAudioTrack;

			ChildSlot
			[
				SNew(SComboButton)
				.Visibility(this, &SLanguagePreviewComboWidget::GetLanguagePreviewVisibility)
				.OnGetMenuContent(this, &SLanguagePreviewComboWidget::GetMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &SLanguagePreviewComboWidget::GetCurrentPreviewLanguage)
				]
			];
		}

	private:

		TSharedRef<SWidget> GetMenuContent()
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			// Get a list of available localization cultures

			constexpr bool bIncludeDerivedCultures = false;

			const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
			TArray<FCultureRef> Cultures = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, bIncludeDerivedCultures);

			for (FCultureRef Culture : Cultures)
			{
				MenuBuilder.AddMenuEntry(
					FText::FromString(Culture->GetName()),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SLanguagePreviewComboWidget::SetPreviewLanguage, Culture->GetName())
					)
				);
			}

			return MenuBuilder.MakeWidget();
		}

		EVisibility GetLanguagePreviewVisibility() const
		{
			UMovieSceneTrack* AudioTrack = WeakAudioTrack.Get();
			if (AudioTrack)
			{
				const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
				TArray<FCultureRef> Cultures = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, false);

				for (FCultureRef Culture : Cultures)

				for (UMovieSceneSection* Section : AudioTrack->GetAllSections())
				{
					UMovieSceneAudioSection* AudioSection = CastChecked<UMovieSceneAudioSection>(Section);
					if (AudioSection)
					{
						if (USoundBase* Sound = AudioSection->GetSound())
						{
							FName LocalizedName = FPackageLocalizationManager::Get().FindLocalizedPackageNameForCulture(*Sound->GetOutermost()->GetPathName(), Culture->GetName());
							if (LocalizedName != NAME_None)
							{
								return EVisibility::Visible;
							}
						}
					}
				}
			}

			return EVisibility::Collapsed;
		}

		FText GetCurrentPreviewLanguage() const
		{
			UMovieScene* MovieScene = WeakMovieScene.Get();
			if (MovieScene)
			{
				UMovieSceneLanguagePreviewDecoration* PreviewLanguage = MovieScene->FindDecoration<UMovieSceneLanguagePreviewDecoration>();
				if (PreviewLanguage && PreviewLanguage->PreviewLanguage.Len() > 0)
				{
					return FText::FromString(PreviewLanguage->PreviewLanguage);
				}
			}
			return LOCTEXT("PreviewLanguage", "Preview Language");
		}

		void SetPreviewLanguage(FString InCultureName)
		{
			UMovieScene* MovieScene = WeakMovieScene.Get();
			if (MovieScene)
			{
				UMovieSceneLanguagePreviewDecoration* Decoration = MovieScene->GetOrCreateDecoration<UMovieSceneLanguagePreviewDecoration>();
				Decoration->SetFlags(RF_Transient);
				Decoration->PreviewLanguage = InCultureName;

				// Mark as changed to force an evaluation
				MovieScene->MarkAsChanged();
			}
		}

		FString FindCurrentPreviewCulture() const
		{
			UMovieScene* MovieScene = WeakMovieScene.Get();
			UMovieSceneLanguagePreviewDecoration* PreviewLanguage = MovieScene->FindDecoration<UMovieSceneLanguagePreviewDecoration>();
			if (PreviewLanguage)
			{
				return PreviewLanguage->PreviewLanguage;
			}
			return FString();
		}

		TWeakObjectPtr<UMovieScene> WeakMovieScene;
		TWeakObjectPtr<UMovieSceneTrack> WeakAudioTrack;
	};
}

// Binding picker to handle batch editing the actor bindings
class FAudioAttachBindingPicker : public FMovieSceneObjectBindingIDPicker
{
public:
	friend class FAudioSectionDetailsCustomization;

	FAudioAttachBindingPicker(TWeakPtr<ISequencer> InSequencer,
		TArray<TWeakObjectPtr<UMovieSceneAudioSection>> InSections)
		: FMovieSceneObjectBindingIDPicker(GetFocusedSequenceID(InSequencer), InSequencer)
		, WeakSections(MoveTemp(InSections))
	{
		Initialize();
	}

	virtual UMovieSceneSequence* GetSequence() const override
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		return Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	}

	virtual void SetCurrentValue(const FMovieSceneObjectBindingID& NewBinding) override
	{
		FScopedTransaction Transaction(LOCTEXT("SetAttachActor", "Set Attach Actor"));
		for (const TWeakObjectPtr<UMovieSceneAudioSection>& WeakSection : WeakSections)
		{
			if (TStrongObjectPtr<UMovieSceneAudioSection> Section = WeakSection.Pin())
			{
				Section->SetDefaultActorKey(NewBinding);
			}
		}
		if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}

	virtual bool HasMultipleValues() const override
	{
		if (WeakSections.Num() <= 1)
		{
			return false;
		}
		
		// Iterate over all sections to see if they have the same actor binding value set
		// this allows us to determine if we show "multiple values" in the dropdown or the single actor they all are bound to
		FMovieSceneObjectBindingID FirstBinding;
		bool bFirst = true;
		for (const TWeakObjectPtr<UMovieSceneAudioSection>& WeakSection : WeakSections)
		{
			if (const TStrongObjectPtr<UMovieSceneAudioSection> Section = WeakSection.Pin())
			{
				const FMovieSceneObjectBindingID Binding = Section->GetAttachActorData().GetDefault().Object;
				if (bFirst)
				{
					FirstBinding = Binding;
					bFirst = false;
				}
				else if (Binding != FirstBinding)
				{
					return true;
				}
			}
		}
		return false;
	}

	virtual FMovieSceneObjectBindingID GetCurrentValue() const override
	{
		// If there are multiple values selected, return an empty object otherwise return the object that is selected
		if (HasMultipleValues())
		{
			return FMovieSceneObjectBindingID();
		}

		if (!WeakSections.IsEmpty())
		{
			if (const TStrongObjectPtr<UMovieSceneAudioSection> Section = WeakSections[0].Pin())
			{
				return Section->GetAttachActorData().GetDefault().Object;
			}
		}
		return FMovieSceneObjectBindingID();
	}

private:
	static FMovieSceneSequenceID GetFocusedSequenceID(TWeakPtr<ISequencer> InSequencer)
	{
		TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
		return Sequencer.IsValid() ? Sequencer->GetFocusedTemplateID() : MovieSceneSequenceID::Root;
	}

	TArray<TWeakObjectPtr<UMovieSceneAudioSection>> WeakSections;
};

// Adds a custom detail to appear in the property window so batch property editing sees the actor attach data
class FAudioSectionDetailsCustomization : public IDetailCustomization
{
public:
	FAudioSectionDetailsCustomization(
		const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams)
		: Params(InParams)
	{
	}

	static TSharedRef<IDetailCustomization> MakeInstance(
		FSequencerSectionPropertyDetailsViewCustomizationParams InParams)
	{
		return MakeShared<FAudioSectionDetailsCustomization>(InParams);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		// Collect all selected audio sections from the outer objects
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);

		TArray<TWeakObjectPtr<UMovieSceneAudioSection>> AudioSections;
		for (const TWeakObjectPtr<UObject>& Obj : Objects)
		{
			if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Obj.Get()))
			{
				AudioSections.Add(AudioSection);
			}
		}

		if (AudioSections.IsEmpty())
		{
			return;
		}

		// Set up the dropdown combo button with the audio sections selected
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("Attach"), LOCTEXT("AttachCategory", "Attach"));
		TSharedRef<FAudioAttachBindingPicker> Picker = MakeShared<FAudioAttachBindingPicker>(Params.SequencerWeak, AudioSections);

		Category.AddCustomRow(LOCTEXT("AttachActorLabel", "Attach Actor"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AttachActorLabel", "Attach Actor"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SComboButton)
					.OnGetMenuContent_Lambda([Picker]() { return Picker->GetPickerMenu(); })
					.ContentPadding(FMargin(4.f, 2.f))
					.ButtonContent()
					[
						Picker->GetCurrentItemWidget(SNew(STextBlock).Font(IDetailLayoutBuilder::GetDetailFont()))
					]
			];
	}

private:
	FSequencerSectionPropertyDetailsViewCustomizationParams Params;
};

/** These utility functions should go away once we start handling sound cues properly */
USoundWave* DeriveSoundWave(UMovieSceneAudioSection* AudioSection)
{
	USoundWave* SoundWave = NULL;
	
	USoundBase* Sound = AudioSection->GetPlaybackSound();
	if (!Sound)
	{
		return SoundWave;
	}

	if (Sound->IsA<USoundWave>())
	{
		SoundWave = Cast<USoundWave>(Sound);
	}
	else if (Sound->IsA<USoundCue>())
	{
		USoundCue* SoundCue = Cast<USoundCue>(Sound);

		// @todo Sequencer - Right now for sound cues, we just use the first sound wave in the cue
		// In the future, it would be better to properly generate the sound cue's data after forcing determinism
		const TArray<USoundNode*>& AllNodes = SoundCue->AllNodes;
		for (int32 Index = 0; Index < AllNodes.Num() && SoundWave == nullptr; ++Index)
		{
			if (AllNodes[Index])
			{
				if (USoundNodeWavePlayer* SoundNodeWavePlayer = Cast<USoundNodeWavePlayer>(AllNodes[Index]))
				{
					SoundWave = SoundNodeWavePlayer->GetSoundWave();
				}
				else if (USoundNodeDialoguePlayer* SoundNodeDialoguePlayer = Cast<USoundNodeDialoguePlayer>(AllNodes[Index]))
				{
					if (UDialogueWave* DialogueWave = SoundNodeDialoguePlayer->GetDialogueWave())
					{
						if (DialogueWave->ContextMappings.Num() > 0)
						{
							SoundWave = DialogueWave->ContextMappings[0].SoundWave;
						}
					}
				}
			}
		}
	}

	return SoundWave;
}


float DeriveUnloopedDuration(UMovieSceneAudioSection* AudioSection)
{
	USoundBase* Sound = AudioSection->GetPlaybackSound();
	if (Sound && Sound->GetDuration() != INDEFINITELY_LOOPING_DURATION)
	{
		return Sound->GetDuration();
	}

	USoundWave* SoundWave = DeriveSoundWave(AudioSection);
	const float Duration = (SoundWave ? SoundWave->GetDuration() : 0.f);
	return Duration == INDEFINITELY_LOOPING_DURATION ? SoundWave->Duration : Duration;
}

/** The maximum number of channels we support */
inline constexpr int32 MaxSupportedChannels = 2;
/** The number of pixels between which to place control points for cubic interpolation */
inline constexpr int32 SmoothingAmount = 6;
/** The size of the sroked border of the audio wave */
inline constexpr int32 StrokeBorderSize = 2;

/** A specific sample from the audio, specifying peak and average amplitude over the sample's range */
struct FAudioSample
{
	FAudioSample() : RMS(0.f), Peak(0), NumSamples(0) {}

	float RMS;
	int32 Peak;
	int32 NumSamples;
};

/** A segment in a cubic spline */
struct FSplineSegment
{
	FSplineSegment() : A(0.f), B(0.f), C(0.f), D(0.f) {}

	/** Cubic polynomial coefficients for the equation f(x) = A + Bx + Cx^2 + Dx^3*/
	float A, B, C, D;
	/** The width of this segment */
	float SampleSize;
	/** The x-position of this segment */
	float Position;
};



/**
 * The audio thumbnail, which holds a texture which it can pass back to a viewport to render
 */
class FAudioThumbnail
	: public ISlateViewport
	, public TSharedFromThis<FAudioThumbnail>
{
public:
	FAudioThumbnail(UMovieSceneSection& InSection, TRange<float> DrawRange, int InTextureSize, const FLinearColor& BaseColor, float DisplayScale);
	~FAudioThumbnail();

	/* ISlateViewport interface */
	virtual FIntPoint GetSize() const override;
	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	virtual bool RequiresVsync() const override;

	/** Returns whether this thumbnail has a texture to render */
	virtual bool ShouldRender() const {return TextureSize > 0;}
	
private:
	/** Generates the waveform preview and dumps it out to the OutBuffer */
	void GenerateWaveformPreview(TArray<uint8>& OutBuffer, TRange<float> DrawRange, float DisplayScale);

	/** Sample the audio data at the given lookup position. Appends the sample result to the Samples array */
	void SampleAudio(int32 NumChannels, const int16* LookupData, int32 LookupStartIndex, int32 LookupEndIndex, int32 LookupSize, int32 MaxAmplitude);

	/** Generate a natural cubic spline from the sample buffer */
	void GenerateSpline(int32 NumChannels, int32 SamplePositionOffset);

private:

	void DestroyTexture();

	/** The section we are visualizing */
	UMovieSceneSection& Section;

	/** The Texture RHI that holds the thumbnail */
	class FSlateTexture2DRHIRef* Texture;

	/** Size of the texture */
	int32 TextureSize;

	/** Accumulation of audio samples for each channel */
	TArray<FAudioSample> Samples[MaxSupportedChannels];

	/** Spline segments generated from the above Samples array */
	TArray<FSplineSegment> SplineSegments[MaxSupportedChannels];

	/** Waveform colors */
	FLinearColor BoundaryColorHSV;
	FLinearColor FillColor_A, FillColor_B;
};

float Modulate(float Value, float Delta, float Range)
{
	Value = FMath::Fmod(Value + Delta, Range);
	if (Value < 0.0f)
	{
		Value += Range;
	}
	return Value;
}

FAudioThumbnail::FAudioThumbnail(UMovieSceneSection& InSection, TRange<float> DrawRange, int32 InTextureSize, const FLinearColor& BaseColor, float DisplayScale)
	: Section(InSection)
	, Texture(NULL)
	, TextureSize(InTextureSize)
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	
	const FLinearColor BaseHSV = BaseColor.LinearRGBToHSV();

	const float BaseValue = FMath::Min(BaseHSV.B, .5f) * BaseHSV.A;
	const float BaseSaturation = FMath::Max((BaseHSV.G - .45f), 0.f) * BaseHSV.A;

	FillColor_A = FLinearColor(Modulate(BaseHSV.R, -2.5f, 360), BaseSaturation + .35f, BaseValue);
	FillColor_B = FLinearColor(Modulate(BaseHSV.R,  2.5f, 360), BaseSaturation + .4f, BaseValue + .15f);

	BoundaryColorHSV = FLinearColor(BaseHSV.R, BaseSaturation, BaseValue + .35f);

	if (ShouldRender())
	{
		uint32 Size = GetSize().X * GetSize().Y * GPixelFormats[PF_B8G8R8A8].BlockBytes;
		TArray<uint8> RawData;
		RawData.AddZeroed(Size);

		GenerateWaveformPreview(RawData, DrawRange, DisplayScale);

		FSlateTextureDataPtr BulkData(new FSlateTextureData(GetSize().X, GetSize().Y, GPixelFormats[PF_B8G8R8A8].BlockBytes, RawData));

		Texture = new FSlateTexture2DRHIRef(GetSize().X, GetSize().Y, PF_B8G8R8A8, BulkData, TexCreate_None);

		BeginInitResource( Texture );
	}
}


FAudioThumbnail::~FAudioThumbnail()
{
	DestroyTexture();
}

void
FAudioThumbnail::DestroyTexture()
{
	if (Texture)
	{
		// UE-114425: Defer the destroy until the next tick to work around the RHI getting destroyed before the render command completes.
		FSlateTexture2DRHIRef* InTexture = Texture;

		Texture = nullptr;

		GEditor->GetTimerManager()->SetTimerForNextTick([this, InTexture]()
		{
			ENQUEUE_RENDER_COMMAND(DestroyTexture)(
				[InTexture](FRHICommandList& RHICmdList)
				{
					if (InTexture)
					{
						InTexture->ReleaseResource();
						delete InTexture;
					}
				}
			);
		});
	}
}

FIntPoint FAudioThumbnail::GetSize() const {return FIntPoint(TextureSize, Section.GetTypedOuter<UMovieSceneAudioTrack>()->GetRowHeight());}
FSlateShaderResource* FAudioThumbnail::GetViewportRenderTargetTexture() const {return Texture;}
bool FAudioThumbnail::RequiresVsync() const {return false;}

/** Lookup a pixel in the given data buffer based on the specified X and Y */
uint8* LookupPixel(TArray<uint8>& Data, int32 X, int32 YPos, int32 Width, int32 Height, int32 Channel, int32 NumChannels)
{
	int32 Y = Height - YPos - 1;
	if (NumChannels == 2)
	{
		Y = Channel == 0 ? Height/2 - YPos : Height/2 + YPos;
	}

	int32 Index = (Y * Width + X) * GPixelFormats[PF_B8G8R8A8].BlockBytes;
	return &Data[Index];
}

/** Lerp between 2 HSV space colors */
FLinearColor LerpHSV(const FLinearColor& A, const FLinearColor& B, float Alpha)
{
	float SrcHue = A.R;
	float DestHue = B.R;

	// Take the shortest path to the new hue
	if (FMath::Abs(SrcHue - DestHue) > 180.0f)
	{
		if (DestHue > SrcHue)
		{
			SrcHue += 360.0f;
		}
		else
		{
			DestHue += 360.0f;
		}
	}

	float NewHue = FMath::Fmod(FMath::Lerp(SrcHue, DestHue, Alpha), 360.0f);
	if (NewHue < 0.0f)
	{
		NewHue += 360.0f;
	}

	return FLinearColor(
		NewHue,
		FMath::Lerp(A.G, B.G, Alpha),
		FMath::Lerp(A.B, B.B, Alpha),
		FMath::Lerp(A.A, B.A, Alpha)
		);
}

void FAudioThumbnail::GenerateWaveformPreview(TArray<uint8>& OutData, TRange<float> DrawRange, float DisplayScale)
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);

	USoundWave* SoundWave = DeriveSoundWave(AudioSection);
	check(SoundWave);
	
	check(SoundWave->NumChannels == 1 || SoundWave->NumChannels == 2);

	// If this SoundWave is generated procedurally, it's not possible to render a thumbnail.
	if (SoundWave->IsProcedurallyGenerated())
	{
		return;
	}

	uint32 SampleRate;
	uint16 NumChannels;
	TArray<uint8> RawPCMData;
	if (!SoundWave->GetImportedSoundWaveData(RawPCMData, SampleRate, NumChannels))
	{
		UE_LOGF(LogMovieScene, Warning, "Failed to get sound wave data for: %ls Waveform will not be visible.", *SoundWave->GetPathName());
		return;
	}

	const int16* LookupData = (int16*)RawPCMData.GetData();
	const int32 LookupSize = RawPCMData.Num() * sizeof(uint8) / sizeof(int16);

	if (!LookupData || !AudioSection->HasStartFrame() || !AudioSection->HasEndFrame())
	{
		return;
	}

	FFrameRate FrameRate            = AudioSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	float      PitchMultiplierValue = AudioSection->GetPitchMultiplierChannel().GetDefault().Get(1.f);
	double     SectionStartTime     = AudioSection->GetInclusiveStartFrame() / FrameRate;

	// @todo Sequencer This fixes looping drawing by pretending we are only dealing with a SoundWave
	TRange<float> AudioTrueRange = TRange<float>(
		SectionStartTime - FrameRate.AsSeconds(AudioSection->GetStartOffset()),
		SectionStartTime - FrameRate.AsSeconds(AudioSection->GetStartOffset()) + DeriveUnloopedDuration(AudioSection) * (1.0f / PitchMultiplierValue));

	float TrueRangeSize = AudioTrueRange.Size<float>();
	float DrawRangeSize = DrawRange.Size<float>();

	const int32 MaxAmplitude = NumChannels == 1 ? GetSize().Y : GetSize().Y / 2;
	const int32 DrawOffsetPx = FMath::Max(FMath::RoundToInt((DrawRange.GetLowerBoundValue() - SectionStartTime) / DisplayScale), 0);

	// In order to prevent flickering waveforms when moving the display position/range around, we have to lock our sample position and spline segments to the view range
	const float RangeLookupFraction = (SmoothingAmount * DisplayScale) / TrueRangeSize;
	const int32 LookupRange = FMath::Clamp(FMath::TruncToInt(RangeLookupFraction * LookupSize), 1, LookupSize);

	const int32 SampleLockOffset = DrawOffsetPx % SmoothingAmount;

	const FIntPoint ThumbnailSize = GetSize();
	const int32 FirstSample = -2.f * SmoothingAmount - SampleLockOffset;
	const int32 LastSample = ThumbnailSize.X + 2.f * SmoothingAmount;

	// Sample the audio one pixel to the left and right
	for (int32 X = FirstSample; X < LastSample; ++X)
	{
		float LookupTime = ((float)(X - 0.5f) / (float)ThumbnailSize.X) * DrawRangeSize + DrawRange.GetLowerBoundValue();
		float LookupFraction = (LookupTime - AudioTrueRange.GetLowerBoundValue()) / TrueRangeSize;
		float LookupFractionLooping = FMath::Fmod(LookupFraction, 1.f);
		int32 LookupIndex = FMath::TruncToInt(LookupFractionLooping * LookupSize);
		
		float NextLookupTime = ((float)(X + 0.5f) / (float)ThumbnailSize.X) * DrawRangeSize + DrawRange.GetLowerBoundValue();
		float NextLookupFraction = (NextLookupTime - AudioTrueRange.GetLowerBoundValue()) / TrueRangeSize;
		float NextLookupFractionLooping = FMath::Fmod(NextLookupFraction, 1.f);
		int32 NextLookupIndex = FMath::TruncToInt(NextLookupFractionLooping * LookupSize);
		
		// Draw looped waveform if section is repeating or asset is natively looping
		const bool bDrawLoopedWaveform = AudioSection->IsLoopingOrRepeating();
		if (!bDrawLoopedWaveform && LookupFraction > 1.f)
		{
			break;
		}

		SampleAudio(SoundWave->NumChannels, LookupData, LookupIndex, NextLookupIndex, LookupSize, MaxAmplitude);
	}

	// Generate a spline
	GenerateSpline(SoundWave->NumChannels, FirstSample);

	// Now draw the spline
	const int32 Width = ThumbnailSize.X;
	const int32 Height = ThumbnailSize.Y;

	FLinearColor BoundaryColor = BoundaryColorHSV.HSVToLinearRGB();

	for (int32 ChannelIndex = 0; ChannelIndex < SoundWave->NumChannels; ++ChannelIndex)
	{
		int32 SplineIndex = 0;

		for (int32 X = 0; X < Width; ++X)
		{
			bool bOutOfRange = SplineIndex >= SplineSegments[ChannelIndex].Num();
			while (!bOutOfRange && X >= SplineSegments[ChannelIndex][SplineIndex].Position+SplineSegments[ChannelIndex][SplineIndex].SampleSize)
			{
				++SplineIndex;
				bOutOfRange = SplineIndex >= SplineSegments[ChannelIndex].Num();
			}
			
			if (bOutOfRange)
			{
				break;
			}

			// Evaluate the spline
			const float DistBetweenPts = (X-SplineSegments[ChannelIndex][SplineIndex].Position)/SplineSegments[ChannelIndex][SplineIndex].SampleSize;
			const float Amplitude = 
				SplineSegments[ChannelIndex][SplineIndex].A +
				SplineSegments[ChannelIndex][SplineIndex].B * DistBetweenPts +
				SplineSegments[ChannelIndex][SplineIndex].C * FMath::Pow(DistBetweenPts, 2) +
				SplineSegments[ChannelIndex][SplineIndex].D * FMath::Pow(DistBetweenPts, 3);

			// @todo: draw border according to gradient of curve to prevent aliasing on steep gradients? This would be non-trivial...
			const float BoundaryStart = Amplitude - StrokeBorderSize * 0.5f;
			const float BoundaryEnd = Amplitude + StrokeBorderSize * 0.5f;

			const FAudioSample& Sample = Samples[ChannelIndex][X - FirstSample];

			for (int32 PixelIndex = 0; PixelIndex < MaxAmplitude; ++PixelIndex)
			{
				uint8* Pixel = LookupPixel(OutData, X, PixelIndex, Width, Height, ChannelIndex, NumChannels);

				const float PixelCenter = PixelIndex + 0.5f;

				const float Dither = FMath::FRand() * .025f - .0125f;
				const float GradLerp = FMath::Clamp(float(PixelIndex) / MaxAmplitude + Dither, 0.f, 1.f);
				FLinearColor SolidFilledColor = LerpHSV(FillColor_A, FillColor_B, GradLerp);

				float BorderBlend = 1.f;
				if (PixelIndex <= FMath::TruncToInt(BoundaryStart))
				{
					BorderBlend = 1.f - FMath::Clamp(BoundaryStart - PixelIndex, 0.f, 1.f);
				}
				
				FLinearColor Color = PixelIndex == Sample.Peak ? FillColor_B.HSVToLinearRGB() : LerpHSV(SolidFilledColor, BoundaryColorHSV, BorderBlend).HSVToLinearRGB();

				// Calculate alpha based on how far from the boundary we are
				float Alpha = FMath::Max(FMath::Clamp(BoundaryEnd - PixelCenter, 0.f, 1.f), FMath::Clamp(float(Sample.Peak) - PixelIndex + 0.25f, 0.f, 1.f));
				if (Alpha <= 0.f)
				{
					break;
				}

				// Slate viewports must have pre-multiplied alpha
				*Pixel++ = Color.B*Alpha*255;
				*Pixel++ = Color.G*Alpha*255;
				*Pixel++ = Color.R*Alpha*255;
				*Pixel++ = Alpha*255;
			}
		}
	}
}

void FAudioThumbnail::GenerateSpline(int32 NumChannels, int32 SamplePositionOffset)
{
	// Generate a cubic polynomial spline interpolating the samples
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{
		TArray<FSplineSegment>& Segments = SplineSegments[ChannelIndex];

		const int32 NumSamples = Samples[ChannelIndex].Num();

		struct FControlPoint
		{
			float Value;
			float Position;
			int32 SampleSize;
		};
		TArray<FControlPoint> ControlPoints;

		for (int SampleIndex = 0; SampleIndex < NumSamples; SampleIndex += SmoothingAmount)
		{
			float RMS = 0.f;
			int32 NumAvgs = FMath::Min(SmoothingAmount, NumSamples - SampleIndex);
			
			for (int32 SubIndex = 0; SubIndex < NumAvgs; ++SubIndex)
			{
				RMS += FMath::Pow(Samples[ChannelIndex][SampleIndex + SubIndex].RMS, 2);
			}

			const int32 SegmentSize2 = NumAvgs / 2;
			const int32 SegmentSize1 = NumAvgs - SegmentSize2;

			RMS = FMath::Sqrt(RMS / NumAvgs);

			FControlPoint& StartPoint = ControlPoints[ControlPoints.AddZeroed()];
			StartPoint.Value = Samples[ChannelIndex][SampleIndex].RMS;
			StartPoint.SampleSize = SegmentSize1;
			StartPoint.Position = SampleIndex + SamplePositionOffset;

			if (SegmentSize2 > 0)
			{
				FControlPoint& MidPoint = ControlPoints[ControlPoints.AddZeroed()];
				MidPoint.Value = RMS;
				MidPoint.SampleSize = SegmentSize2;
				MidPoint.Position = SampleIndex + SamplePositionOffset + SegmentSize1;
			}
		}

		if (ControlPoints.Num() <= 1)
		{
			continue;
		}

		const int32 LastIndex = ControlPoints.Num() - 1;

		// Perform gaussian elimination on the following tridiagonal matrix that defines the piecewise cubic polynomial
		// spline for n control points, given f(x), f'(x) and f''(x) continuity. Imposed boundary conditions are f''(0) = f''(n) = 0.
		//	(D[i] = f[i]'(x))
		//	1	2						D[i]	= 3(y[1] - y[0])
		//	1	4	1					D[i+1]	= 3(y[2] - y[1])
		//		1	4	1				|		|
		//		\	\	\	\	\		|		|
		//					1	4	1	|		= 3(y[n-1] - y[n-2])
		//						1	2	D[n]	= 3(y[n] - y[n-1])
		struct FMinimalMatrixComponent
		{
			float DiagComponent;
			float KnownConstant;
		};

		TArray<FMinimalMatrixComponent> GaussianCoefficients;
		GaussianCoefficients.AddZeroed(ControlPoints.Num());

		// Setup the top left of the matrix
		GaussianCoefficients[0].KnownConstant = 3.f * (ControlPoints[1].Value - ControlPoints[0].Value);
		GaussianCoefficients[0].DiagComponent = 2.f;

		// Calculate the diagonal component of each row, based on the eliminated value of the last
		for (int32 Index = 1; Index < GaussianCoefficients.Num() - 1; ++Index)
		{
			GaussianCoefficients[Index].KnownConstant = (3.f * (ControlPoints[Index+1].Value - ControlPoints[Index-1].Value)) - (GaussianCoefficients[Index-1].KnownConstant / GaussianCoefficients[Index-1].DiagComponent);
			GaussianCoefficients[Index].DiagComponent = 4.f - (1.f / GaussianCoefficients[Index-1].DiagComponent);
		}
		
		// Setup the bottom right of the matrix
		GaussianCoefficients[LastIndex].KnownConstant = (3.f * (ControlPoints[LastIndex].Value - ControlPoints[LastIndex-1].Value)) - (GaussianCoefficients[LastIndex-1].KnownConstant / GaussianCoefficients[LastIndex-1].DiagComponent);
		GaussianCoefficients[LastIndex].DiagComponent = 2.f - (1.f / GaussianCoefficients[LastIndex-1].DiagComponent);

		// Now we have an upper triangular matrix, we can use reverse substitution to calculate D[n] -> D[0]

		TArray<float> FirstOrderDerivatives;
		FirstOrderDerivatives.AddZeroed(GaussianCoefficients.Num());

		FirstOrderDerivatives[LastIndex] = GaussianCoefficients[LastIndex].KnownConstant / GaussianCoefficients[LastIndex].DiagComponent;

		for (int32 Index = GaussianCoefficients.Num() - 2; Index >= 0; --Index)
		{
			FirstOrderDerivatives[Index] = (GaussianCoefficients[Index].KnownConstant - FirstOrderDerivatives[Index+1]) / GaussianCoefficients[Index].DiagComponent;
		}

		// Now we know the first-order derivatives of each control point, calculating the interpolating polynomial is trivial
		// f(x) = a + bx + cx^2 + dx^3
		//	a = y
		//	b = D[i]
		//	c = 3(y[i+1] - y[i]) - 2D[i] - D[i+1]
		//	d = 2(y[i] - y[i+1]) + 2D[i] + D[i+1]
		for (int32 Index = 0; Index < FirstOrderDerivatives.Num() - 2; ++Index)
		{
			Segments.Emplace();
			Segments.Last().A = ControlPoints[Index].Value;
			Segments.Last().B = FirstOrderDerivatives[Index];
			Segments.Last().C = 3.f*(ControlPoints[Index+1].Value - ControlPoints[Index].Value) - 2*FirstOrderDerivatives[Index] - FirstOrderDerivatives[Index+1];
			Segments.Last().D = 2.f*(ControlPoints[Index].Value - ControlPoints[Index+1].Value) + FirstOrderDerivatives[Index] + FirstOrderDerivatives[Index+1];

			Segments.Last().Position = ControlPoints[Index].Position;
			Segments.Last().SampleSize = ControlPoints[Index].SampleSize;
		}
	}
}

void FAudioThumbnail::SampleAudio(int32 NumChannels, const int16* LookupData, int32 LookupStartIndex, int32 LookupEndIndex, int32 LookupSize, int32 MaxAmplitude)
{
	LookupStartIndex = NumChannels == 2 ? (LookupStartIndex % 2 == 0 ? LookupStartIndex : LookupStartIndex - 1) : LookupStartIndex;
	LookupEndIndex = FMath::Max(LookupEndIndex, LookupStartIndex + 1);
	
	int32 StepSize = NumChannels;

	// optimization - don't take more than a maximum number of samples per pixel
	int32 Range = LookupEndIndex - LookupStartIndex;
	int32 SampleCount = Range / StepSize;
	int32 MaxSampleCount = AnimatableAudioEditorConstants::MaxSamplesPerPixel;
	int32 ModifiedStepSize = StepSize;
	
	if (SampleCount > MaxSampleCount)
	{
		// Always start from a common multiple
		int32 Adjustment = LookupStartIndex % MaxSampleCount;
		LookupStartIndex = FMath::Clamp(LookupStartIndex - Adjustment, 0, LookupSize);
		LookupEndIndex = FMath::Clamp(LookupEndIndex - Adjustment, 0, LookupSize);
		ModifiedStepSize *= (SampleCount / MaxSampleCount);
	}

	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{
		FAudioSample& NewSample = Samples[ChannelIndex][Samples[ChannelIndex].Emplace()];

		for (int32 Index = LookupStartIndex; Index < LookupEndIndex; Index += ModifiedStepSize)
		{
			if (Index < 0 || Index >= LookupSize)
			{
				NewSample.RMS += 0.f;
				++NewSample.NumSamples;
				continue;
			}

			int32 DataPoint = LookupData[Index + ChannelIndex];
			int32 Sample = FMath::Clamp(FMath::TruncToInt(FMath::Abs(DataPoint) / 32768.f * MaxAmplitude), 0, MaxAmplitude - 1);

			NewSample.RMS += FMath::Pow(Sample, 2.f);
			NewSample.Peak = FMath::Max(NewSample.Peak, Sample);
			++NewSample.NumSamples;
		}

		if (NewSample.NumSamples)
		{
			NewSample.RMS = (FMath::Sqrt(NewSample.RMS / NewSample.NumSamples));
		}
	}
}


FAudioSection::FAudioSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer )
	: Section( InSection )
	, StoredDrawRange(TRange<float>::Empty())
	, StoredSectionHeight(0.f)
	, bStoredRepeating(true)
	, Sequencer(InSequencer)
	, InitialStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{
}


FAudioSection::~FAudioSection()
{
}

UMovieSceneSection* FAudioSection::GetSectionObject()
{ 
	return &Section;
}


FText FAudioSection::GetSectionTitle() const
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	if (AudioSection)
	{
		if (USoundBase* Sound = AudioSection->GetPlaybackSound())
		{
			// Return the asset name if it exists
			return FText::FromString(Sound->GetName());
		}
		else
		{
			// There is no asset during record so return empty string
			return FText();
		}
	}
	
	return NSLOCTEXT("FAudioSection", "NoAudioTitleName", "No Audio");
}

FText FAudioSection::GetSectionToolTip() const
{
	FText ToolTip;

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return ToolTip;
	}

	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	if (!AudioSection)
	{
		return ToolTip;
	}

	FText AttachToolTip;

	const FMovieSceneActorReferenceChannel& AttachActorData = AudioSection->GetAttachActorData();

	TMovieSceneChannelData<const FMovieSceneActorReferenceKey> ChannelData = AttachActorData.GetData();
	TArrayView<const FMovieSceneActorReferenceKey> Values = ChannelData.GetValues().Num() != 0
		? ChannelData.GetValues()
		: MakeArrayView(&AttachActorData.GetDefault(), 1);

	FMovieSceneActorReferenceKey CurrentValue;
	AttachActorData.Evaluate(SequencerPtr->GetLocalTime().Time, CurrentValue);

	for (int32 Index = 0; Index < Values.Num(); ++Index)
	{
		FMovieSceneObjectBindingID AttachBindingID = Values[Index].Object;
		FName AttachSocketName = Values[Index].SocketName;

		for (TWeakObjectPtr<> WeakObject : AttachBindingID.ResolveBoundObjects(SequencerPtr->GetFocusedTemplateID(), *SequencerPtr))
		{
			AActor* AttachActor = Cast<AActor>(WeakObject.Get());
			if (AttachActor)
			{
				if (AttachSocketName.IsNone())
				{
					AttachToolTip = FText::FromString(AttachActor->GetActorLabel());
				}
				else
				{
					AttachToolTip = FText::Format(LOCTEXT("ToolTipAttachSocket", "{0} ({1})"), FText::FromString(AttachActor->GetActorLabel()), FText::FromName(AttachSocketName));
				}
			}
		}
	}

	const USoundWave* Sound = DeriveSoundWave(AudioSection);

	if (Sound && AudioSection->HasStartFrame() && AudioSection->HasEndFrame())
	{
		UMovieScene* MovieScene = AudioSection->GetTypedOuter<UMovieScene>();
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		const float AudioStartTime = AudioSection->GetStartOffset() / TickResolution;
		const float SectionDuration = (AudioSection->GetExclusiveEndFrame() - AudioSection->GetInclusiveStartFrame())/ TickResolution;
		
		FText BaseToolTip;

		if (AudioSection->IsRepeating())
		{
			BaseToolTip = FText::Format(LOCTEXT("ToolTipContentFormatRepeating", "Start: {0}s\nDuration: {1}s\nRepeating"),
				AudioStartTime,
				SectionDuration);
		}
		else
		{
			const float SoundDuration = Sound->Duration - AudioStartTime;
			const float Duration = FMath::Min(SoundDuration, SectionDuration);

			if (Duration > 0.0f)
			{
				BaseToolTip = FText::Format(LOCTEXT("ToolTipContentFormat", "{0}s - {1}s ({2} seconds)"),
					AudioStartTime,
					AudioStartTime + Duration,
					Duration);
			}
		}

		// EvalOptions.bCanEditCompletionMode is the section's CVAR mirror: false when
		// Sequencer.Audio.UsePlayUntilFinished is on, true (the default) for the legacy path.
		FText CompletionModeToolTip;
		if (!AudioSection->EvalOptions.bCanEditCompletionMode)
		{
			CompletionModeToolTip = AudioSection->GetPlayUntilFinished()
				? LOCTEXT("ToolTipPlayUntilFinished", "\nPlays until sound finishes")
				: LOCTEXT("ToolTipStopsOnSectionEnd", "\nStops on section end");
		}
		else
		{
			switch (AudioSection->EvalOptions.CompletionMode)
			{
			case EMovieSceneCompletionMode::KeepState:
				CompletionModeToolTip = LOCTEXT("ToolTipCompletionModeKeep", "\nPlays to end of sound");
				break;
			case EMovieSceneCompletionMode::RestoreState:
				CompletionModeToolTip = LOCTEXT("ToolTipCompletionModeRestore", "\nStops on section end");
				break;
			default:
				CompletionModeToolTip = LOCTEXT("ToolTipCompletionModeDefault", "\nCompletionMode: ProjectDefault");
			}
		}

		FText LoopStateDivergenceToolTip;
		if (AudioSection->IsRepeating() && !AudioSection->IsSoundLoopingAsset())
		{
			// Section repeat state does not match the underlying Sound.
			LoopStateDivergenceToolTip = LOCTEXT("ToolTipRepeatStateDivergence", "\nRepeating non-looping content");
		}

		FText DurationDivergenceToolTip;
		check(AudioSection->HasStartFrame());
		check(AudioSection->HasEndFrame());
		if (!AudioSection->IsLoopingOrRepeating() && !Sound->IsProcedurallyGenerated())
		{
			const float SoundDurationInSeconds = DeriveUnloopedDuration(AudioSection);
			if (!FMath::IsNearlyEqual(SectionDuration, SoundDurationInSeconds, SectionToSoundDurationToleranceInSeconds))
			{
				// Section duration does not match the underlying Sound.
				DurationDivergenceToolTip = LOCTEXT("ToolTipDurationDivergence", "\nOverriding duration from source content");
			}
		}

		ToolTip = FText::Format(LOCTEXT("FullToolTipContentFormat", "{0}{1}{2}{3}"), BaseToolTip, LoopStateDivergenceToolTip, DurationDivergenceToolTip, CompletionModeToolTip);
	}

	ToolTip = FText::Join(FText::GetEmpty(), ToolTip, AttachToolTip);

	return ToolTip;
}

float FAudioSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	if (UMovieSceneAudioTrack* Track = Section.GetTypedOuter<UMovieSceneAudioTrack>())
	{
		return Track->GetRowHeight();
	}
	return ISequencerSection::GetSectionHeight(ViewDensity);
}

int32 FAudioSection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	UMovieScene* MovieScene = Section.GetTypedOuter<UMovieScene>();

	int32 LayerId = Painter.PaintSectionBackground();

	// If outline highlights are enabled, detect if the attach actor bindings are valid and put a red outline if they are broken
	if (bOutlineAudioActorAttachments && AudioSection != nullptr)
	{
		const FMovieSceneActorReferenceChannel& AttachActorData = AudioSection->GetAttachActorData();

		// Fetch the binding guid
		FMovieSceneActorReferenceKey SectionAttachKey;
		AttachActorData.Evaluate(0, SectionAttachKey);
		FMovieSceneObjectBindingID SectionAttachBindingID = SectionAttachKey.Object;
		FGuid BindingGuid = SectionAttachKey.Object.GetGuid();

		bool bActorBindingHasValidActor = false;

		// Attempt to evaluate through the current sequence for bindings as this also evaluates subsequences
		// This path resolves the binding ids properly in the hierarchy allowing for resolving to the movie scene the binding object belongs to which may not be the same scene as the audio section
		if (TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin())
		{
			const FMovieSceneSequenceID ResolvedSequenceID = SectionAttachBindingID.ResolveSequenceID(SequencerPin->GetFocusedTemplateID(), *SequencerPin);

			if (const UMovieSceneSequence* ResolvedSequence = SequencerPin->GetSharedPlaybackState()->GetSequence(ResolvedSequenceID))
			{
				if (UMovieScene* ResolvedMovieScene = ResolvedSequence->GetMovieScene())
				{
					bActorBindingHasValidActor = ResolvedMovieScene->FindPossessable(BindingGuid) != nullptr
						|| ResolvedMovieScene->FindSpawnable(BindingGuid) != nullptr;
				}
			}
		}
		else if (MovieScene != nullptr)// Fallback in case the sequencer is invalid to just check the current scene asset only
		{
			bActorBindingHasValidActor = MovieScene->FindPossessable(BindingGuid) != nullptr
				|| MovieScene->FindSpawnable(BindingGuid) != nullptr;
		}

		// If the section binding is set, but the binding does not point to a valid object then we consider it a broken binding and highlight it to the user
		if (SectionAttachBindingID.IsValid() && !bActorBindingHasValidActor)
		{
			static const FSlateBrush* ErroredSectionOverlay = FAppStyle::Get().GetBrush("Sequencer.Section.ErroredSectionOverlay");
			const ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;

			FLinearColor ErrorColor = FLinearColor::Red;
			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				Painter.LayerId,
				Painter.SectionGeometry.ToPaintGeometry(Painter.SectionGeometry.GetLocalSize() - FVector2D(1.f, 1.f), FSlateLayoutTransform(FVector2D(1.f, 1.f))),
				ErroredSectionOverlay,
				DrawEffects,
				ErrorColor.CopyWithNewOpacity(0.8f)
			);
			Painter.LayerId++;
		}
	}

	if (WaveformThumbnail.IsValid() && WaveformThumbnail->ShouldRender())
	{
		// @todo Sequencer draw multiple times if looping possibly - requires some thought about SoundCues
		FSlateDrawElement::MakeViewport(
			Painter.DrawElements,
			++LayerId,
			Painter.SectionGeometry.ToPaintGeometry(FVector2f(StoredXSize, StoredSectionHeight), FSlateLayoutTransform(FVector2f(StoredXOffset, 0))),
			WaveformThumbnail,
			(Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect) | ESlateDrawEffect::NoGamma,
			FLinearColor::White
		);
	}

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	TimeToPixel = Painter.GetTimeConverter();

	if (!AudioSection || !AudioSection->GetPlaybackSound())
	{
		return LayerId;
	}

	// Add lines where the animation starts and ends/loops
	FFrameRate TickResolution = TimeToPixel->GetTickResolution();
	const float AudioDuration = DeriveUnloopedDuration(AudioSection);

	if (AudioDuration > KINDA_SMALL_NUMBER)
	{
		const float MaxOffset = Section.GetRange().Size<FFrameTime>() / TickResolution;
		const float StartOffsetTime = TickResolution.AsSeconds(AudioSection->GetStartOffset());
		const float SectionStartTime = TickResolution.AsSeconds(AudioSection->GetInclusiveStartFrame());

		float OffsetTime = AudioDuration - StartOffsetTime;
		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixel->SecondsToPixel(SectionStartTime + OffsetTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += AudioDuration;

			if (!AudioSection->IsLoopingOrRepeating())
			{
				break;
			}
		}
	}

	return LayerId;
}


void FAudioSection::Tick( const FGeometry& AllottedGeometry, const FGeometry& ParentGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Defer regenerating waveforms if playing or scrubbing
	TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin();
	if (!SequencerPin.IsValid())
	{
		return;
	}
	
	EMovieScenePlayerStatus::Type PlaybackState = SequencerPin->GetPlaybackStatus();

	if (!TimeToPixel.IsSet() || PlaybackState == EMovieScenePlayerStatus::Playing || PlaybackState == EMovieScenePlayerStatus::Scrubbing)
	{
		return;
	}

	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	UMovieSceneTrack* Track = Section.GetTypedOuter<UMovieSceneTrack>();

	USoundWave* SoundWave = DeriveSoundWave(AudioSection);
	if (Track && SoundWave && (SoundWave->NumChannels == 1 || SoundWave->NumChannels == 2))
	{
		const FSlateRect ParentRect = TransformRect(
			Concatenate(
				ParentGeometry.GetAccumulatedLayoutTransform(),
				AllottedGeometry.GetAccumulatedLayoutTransform().Inverse()
			),
			FSlateRect(FVector2D(0, 0), ParentGeometry.GetLocalSize())
		);

		// Always draw audio with linear space
		FTimeToPixel LinearTimeToPixel = *TimeToPixel;
		LinearTimeToPixel.NonLinearTransform = nullptr;

		const float LeftMostVisiblePixel = FMath::Max(ParentRect.Left, 0.f);
		const float RightMostVisiblePixel = FMath::Min(ParentRect.Right, AllottedGeometry.GetLocalSize().X);

		float StartTime = TimeToPixel->PixelToSeconds(LeftMostVisiblePixel);
		float EndTime   = StartTime + (LinearTimeToPixel.PixelToSeconds(RightMostVisiblePixel) - LinearTimeToPixel.PixelToSeconds(LeftMostVisiblePixel));
		TRange<float> DrawRange = TRange<float>(StartTime, EndTime);

		// generate texture x offset and x size
		int32 XOffset = LeftMostVisiblePixel;
		int32 XSize = RightMostVisiblePixel - LeftMostVisiblePixel;

		if (!FMath::IsNearlyEqual(DrawRange.GetLowerBoundValue(), StoredDrawRange.GetLowerBoundValue()) ||
			!FMath::IsNearlyEqual(DrawRange.GetUpperBoundValue(), StoredDrawRange.GetUpperBoundValue()) ||
			XOffset != StoredXOffset || XSize != StoredXSize || Track->GetColorTint() != StoredColor ||
			StoredSoundWave != SoundWave ||
			StoredSectionHeight != GetSectionHeight(SequencerPin->GetViewModel()->GetViewDensity()) ||
			StoredStartOffset != AudioSection->GetStartOffset() ||
			bStoredRepeating != AudioSection->IsRepeating())
		{
			float DisplayScale = XSize / DrawRange.Size<float>();

			// Use the view range if possible, as it's much more stable than using the texture size and draw range
			DisplayScale = SequencerPin->GetViewRange().Size<float>() / ParentGeometry.GetLocalSize().X;	

			RegenerateWaveforms(DrawRange, XOffset, XSize, Track->GetColorTint(), DisplayScale);
			StoredSoundWave = SoundWave;
		}
	}
	else
	{
		WaveformThumbnail.Reset();
		StoredDrawRange = TRange<float>::Empty();
		StoredSoundWave.Reset();
	}
}

void FAudioSection::BeginResizeSection()
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	InitialStartOffsetDuringResize = AudioSection->GetStartOffset();
	InitialStartTimeDuringResize = AudioSection->HasStartFrame() ? AudioSection->GetInclusiveStartFrame() : 0;
}

void FAudioSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);

	if (ResizeMode == SSRM_LeadingEdge && AudioSection)
	{
		FFrameNumber NewStartOffset = ResizeTime - InitialStartTimeDuringResize;
		NewStartOffset += InitialStartOffsetDuringResize;

		// Ensure start offset is not less than 0
		if (NewStartOffset < 0)
		{
			ResizeTime = ResizeTime - NewStartOffset;
			NewStartOffset = FFrameNumber(0);
		}

		AudioSection->SetStartOffset(NewStartOffset);
	}
	else if (ResizeMode == SSRM_TrailingEdge && AudioSection && AudioSection->GetPlayUntilFinished() && AudioSection->HasEndFrame())
	{
		// Auto-disable PlayUntilFinished only when the trailing edge crosses below the sound's natural duration.
		// Subsequent trims of an already-too-short section (e.g. user re-enabled PUF manually) leave PUF alone.
		if (TOptional<FFrameNumber> SoundEndFrame = AudioSection->ComputeDeterministicSoundEndFrame())
		{
			const FFrameNumber PreResizeEnd = AudioSection->GetExclusiveEndFrame();

			if (PreResizeEnd >= *SoundEndFrame && ResizeTime < *SoundEndFrame)
			{
				AudioSection->SetPlayUntilFinished(false);
			}
		}
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FAudioSection::BeginSlipSection()
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	InitialStartOffsetDuringResize = AudioSection->GetStartOffset();
	InitialStartTimeDuringResize = AudioSection->HasStartFrame() ? AudioSection->GetInclusiveStartFrame() : 0;
}

void FAudioSection::SlipSection(FFrameNumber SlipTime)
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);

	FFrameNumber NewStartOffset = SlipTime - InitialStartTimeDuringResize;
	NewStartOffset += InitialStartOffsetDuringResize;

	// Ensure start offset is not less than 0
	AudioSection->SetStartOffset(FMath::Max(NewStartOffset, FFrameNumber(0)));

	ISequencerSection::SlipSection(SlipTime);
}

TOptional<FFrameTime> FAudioSection::GetSectionTime(FSequencerSectionPainter& InPainter) const
{
	if (!InPainter.bIsSelected || !Sequencer.Pin())
	{
		return TOptional<FFrameTime>();
	}

	const FQualifiedFrameTime CurrentTime = Sequencer.Pin()->GetLocalTime();
	if (!Section.GetRange().Contains(CurrentTime.Time.FrameNumber))
	{
		return TOptional<FFrameTime>();
	}

	FQualifiedFrameTime HintFrameTime;

	TOptional<FSoundWaveTimecodeInfo> TimecodeInfo;
	if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section))
	{
		if (const USoundWave* SoundWave = DeriveSoundWave(AudioSection))
		{
			TimecodeInfo = SoundWave->GetTimecodeInfo();
		}

		const FFrameRate SectionFrameRate = AudioSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		const double AudioStartOffset = SectionFrameRate.AsSeconds(AudioSection->GetStartOffset());

		const FFrameRate TickResolution = CurrentTime.Rate;
		const FFrameTime FramesSinceMidnight = TimecodeInfo.IsSet() ? TickResolution.AsFrameTime(TimecodeInfo->GetNumSecondsSinceMidnight() + AudioStartOffset) : TickResolution.AsFrameTime(AudioStartOffset);

		const FFrameTime SectionOffset = CurrentTime.Time - AudioSection->GetInclusiveStartFrame();
		const FFrameTime CursorFrameTime = FramesSinceMidnight + SectionOffset;

		HintFrameTime = FQualifiedFrameTime(FFrameTime(CursorFrameTime.CeilToFrame()), TickResolution);
	}

	return HintFrameTime.Time;
}

void FAudioSection::CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const
{
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UMovieSceneAudioSection::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FAudioSectionDetailsCustomization::MakeInstance, InParams));
}

void FAudioSection::RegenerateWaveforms(TRange<float> DrawRange, int32 XOffset, int32 XSize, const FColor& ColorTint, float DisplayScale)
{
	UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(&Section);
	TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin();
	if (!SequencerPin)
	{
		return;
	}

	StoredDrawRange = DrawRange;
	StoredXOffset = XOffset;
	StoredXSize = XSize;
	StoredColor = ColorTint;
	StoredStartOffset = AudioSection->GetStartOffset();
	StoredSectionHeight = GetSectionHeight(SequencerPin->GetViewModel()->GetViewDensity());
	bStoredRepeating = AudioSection->IsRepeating();

	if (DrawRange.IsDegenerate() || DrawRange.IsEmpty() || AudioSection->GetPlaybackSound() == NULL)
	{
		WaveformThumbnail.Reset();
	}
	else
	{
		WaveformThumbnail = MakeShareable(new FAudioThumbnail(Section, DrawRange, XSize, ColorTint, DisplayScale));
	}
}

FAudioTrackEditor::FAudioTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer ) 
{
}

FAudioTrackEditor::~FAudioTrackEditor()
{
}

FText FAudioTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AudioTrackEditor_DisplayName", "Audio");
}

void FAudioTrackEditor::OnInitialize()
{
	RegisterMovieSceneChangedDelegate();
}

void FAudioTrackEditor::OnRelease()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid() && MovieSceneChangedDelegate.IsValid())
	{
		SequencerPtr->OnMovieSceneDataChanged().Remove(MovieSceneChangedDelegate);
		MovieSceneChangedDelegate.Reset();
	}
}

TSharedRef<ISequencerTrackEditor> FAudioTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FAudioTrackEditor( InSequencer ) );
}


void FAudioTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Audio Track"),
		LOCTEXT("AddTooltip", "Adds a new audio track that can play sounds."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Audio"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FAudioTrackEditor::HandleAddAudioTrackMenuEntryExecute)
		)
	);
}

void FAudioTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass != nullptr && ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddAttachedAudioTrack", "Audio"),
			LOCTEXT("AddAttachedAudioTooltip", "Adds an audio track attached to the object."),
			FNewMenuDelegate::CreateSP(this, &FAudioTrackEditor::HandleAddAttachedAudioTrackMenuEntryExecute, ObjectBindings));
	}
}


bool FAudioTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneAudioTrack::StaticClass();
}


bool FAudioTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneAudioTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}


const FSlateBrush* FAudioTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Audio");
}

bool FAudioTrackEditor::IsResizable(UMovieSceneTrack* InTrack) const
{
	return true;
}

void FAudioTrackEditor::Resize(float NewSize, UMovieSceneTrack* InTrack)
{
	UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(InTrack);
	if (AudioTrack)
	{
		AudioTrack->Modify();

		const int32 MaxNumRows = AudioTrack->GetMaxRowIndex() + 1;
		AudioTrack->SetRowHeight(FMath::RoundToInt(NewSize) / MaxNumRows);
	}
}

bool FAudioTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid() || !DragDropParams.Track.Get()->IsA(UMovieSceneAudioTrack::StaticClass()))
	{
		return false;
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return false;
	}
	
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return false;
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return false;
	}

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		if (USoundBase* Sound = Cast<USoundBase>(AssetData.GetAsset()))
		{
			FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();
			FFrameNumber LengthInFrames = TickResolution.AsFrameNumber(Sound->GetDuration());
			DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, DragDropParams.FrameNumber + LengthInFrames);
			return true;
		}
	}

	return false;
}


FReply FAudioTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid() || !DragDropParams.Track.Get()->IsA(UMovieSceneAudioTrack::StaticClass()))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return FReply::Unhandled();
	}
	
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return FReply::Unhandled();
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return FReply::Unhandled();
	}

	UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(DragDropParams.Track);

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		USoundBase* Sound = Cast<USoundBase>(AssetData.GetAsset());

		if (Sound)
		{
			if (DragDropParams.TargetObjectGuid.IsValid())
			{
				TArray<TWeakObjectPtr<>> OutObjects;
				for (TWeakObjectPtr<> Object : SequencerPtr->FindObjectsInCurrentSequence(DragDropParams.TargetObjectGuid))
				{
					OutObjects.Add(Object);
				}

				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FAudioTrackEditor::AddNewAttachedSound, Sound, AudioTrack, OutObjects));
			}
			else
			{
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FAudioTrackEditor::AddNewSound, Sound, AudioTrack, DragDropParams.RowIndex));
			}

			bAnyDropped = true;
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}

TSharedRef<ISequencerSection> FAudioTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	return MakeShareable( new FAudioSection(SectionObject, GetSequencer()) );
}

TSharedPtr<SWidget> FAudioTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	TViewModelPtr<ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<ITrackExtension>(true);
	UMovieSceneTrack* Track = TrackModel ? TrackModel->GetTrack() : nullptr;
	if (!Track)
	{
		return SNullWidget::NullWidget;
	}

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		if (TrackModel.AsModel()->IsA<FTrackModel>())
		{
			return SNew(AudioTrack::SLanguagePreviewComboWidget, Track->GetTypedOuter<UMovieScene>(), Track);
		}
	}

	if (ColumnName == FCommonOutlinerNames::Add)
	{
		return UE::Sequencer::MakeAddButton(
			LOCTEXT("AudioText", "Audio"),
			FOnGetContent::CreateSP(
				this, &FAudioTrackEditor::BuildAudioSubMenu,
				FOnAssetSelected::CreateRaw(this, &FAudioTrackEditor::OnAudioAssetSelected, Track),
				FOnAssetEnterPressed::CreateRaw(this, &FAudioTrackEditor::OnAudioAssetEnterPressed, Track)
			),
			Params.ViewModel
		);
	}

	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

bool FAudioTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (Asset->IsA<USoundBase>())
	{
		auto Sound = Cast<USoundBase>(Asset);
		UMovieSceneAudioTrack* DummyTrack = nullptr;
		
		const FScopedTransaction Transaction(LOCTEXT("AddAudio_Transaction", "Add Audio"));

		if (TargetObjectGuid.IsValid())
		{
			TArray<TWeakObjectPtr<>> OutObjects;
			for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(TargetObjectGuid))
			{
				OutObjects.Add(Object);
			}

			AnimatablePropertyChanged( FOnKeyProperty::CreateRaw(this, &FAudioTrackEditor::AddNewAttachedSound, Sound, DummyTrack, OutObjects));
		}
		else
		{
			int32 RowIndex = INDEX_NONE;
			AnimatablePropertyChanged( FOnKeyProperty::CreateRaw(this, &FAudioTrackEditor::AddNewSound, Sound, DummyTrack, RowIndex));
		}

		return true;
	}
	return false;
}


FKeyPropertyResult FAudioTrackEditor::AddNewSound( FFrameNumber KeyTime, USoundBase* Sound, UMovieSceneAudioTrack* AudioTrack, int32 RowIndex )
{
	FKeyPropertyResult KeyPropertyResult;

	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene->IsReadOnly())
	{
		return KeyPropertyResult;
	}

	FocusedMovieScene->Modify();

	FFindOrCreateRootTrackResult<UMovieSceneAudioTrack> TrackResult;
	TrackResult.Track = AudioTrack;
	if (!AudioTrack)
	{
		TrackResult = FindOrCreateRootTrack<UMovieSceneAudioTrack>();
		AudioTrack = TrackResult.Track;
	}

	if (ensure(AudioTrack))
	{
		AudioTrack->Modify();

		UMovieSceneSection* NewSection = AudioTrack->AddNewSoundOnRow( Sound, KeyTime, RowIndex );

		if (TrackResult.bWasCreated)
		{
			AudioTrack->SetDisplayName(LOCTEXT("AudioTrackName", "Audio"));

			if (GetSequencer().IsValid())
			{
				GetSequencer()->OnAddTrack(AudioTrack, FGuid());
			}
		}

		KeyPropertyResult.bTrackModified = true;
		KeyPropertyResult.SectionsCreated.Add(NewSection);
	}

	return KeyPropertyResult;
}


FKeyPropertyResult FAudioTrackEditor::AddNewAttachedSound( FFrameNumber KeyTime, USoundBase* Sound, UMovieSceneAudioTrack* AudioTrack, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo )
{
	FKeyPropertyResult KeyPropertyResult;

	for( int32 ObjectIndex = 0; ObjectIndex < ObjectsToAttachTo.Num(); ++ObjectIndex )
	{
		UObject* Object = ObjectsToAttachTo[ObjectIndex].Get();

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult;
			TrackResult.Track = AudioTrack;
			if (!AudioTrack)
			{
				TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneAudioTrack::StaticClass());
				AudioTrack = Cast<UMovieSceneAudioTrack>(TrackResult.Track);
			}

			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(AudioTrack))
			{
				AudioTrack->Modify();

				UMovieSceneSection* NewSection = AudioTrack->AddNewSound(Sound, KeyTime);
				AudioTrack->SetDisplayName(LOCTEXT("AudioTrackName", "Audio"));				
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);

				GetSequencer()->EmptySelection();
				GetSequencer()->SelectSection(NewSection);
				GetSequencer()->ThrobSectionSelection();
			}
		}
	}

	return KeyPropertyResult;
}


/* FAudioTrackEditor callbacks
 *****************************************************************************/

void FAudioTrackEditor::HandleAddAudioTrackMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddAudioTrack_Transaction", "Add Audio Track"));
	FocusedMovieScene->Modify();
	
	auto NewTrack = FocusedMovieScene->AddTrack<UMovieSceneAudioTrack>();
	ensure(NewTrack);

	NewTrack->SetDisplayName(LOCTEXT("AudioTrackName", "Audio"));

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

void FAudioTrackEditor::HandleAddAttachedAudioTrackMenuEntryExecute(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	MenuBuilder.AddWidget(BuildAudioSubMenu(FOnAssetSelected::CreateRaw(this, &FAudioTrackEditor::OnAttachedAudioAssetSelected, ObjectBindings), FOnAssetEnterPressed::CreateRaw(this, &FAudioTrackEditor::OnAttachedAudioEnterPressed, ObjectBindings)), FText::GetEmpty(), true);
}


TSharedRef<SWidget> FAudioTrackEditor::BuildAudioSubMenu(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FTopLevelAssetPath> ClassNames;
	ClassNames.Add(USoundBase::StaticClass()->GetClassPathName());
	TSet<FTopLevelAssetPath> DerivedClassNames;
	AssetRegistryModule.Get().GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

	FMenuBuilder MenuBuilder(true, nullptr);

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = OnAssetSelected;
		AssetPickerConfig.OnAssetEnterPressed = OnAssetEnterPressed;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		for (FTopLevelAssetPath ClassName : DerivedClassNames)
		{
			AssetPickerConfig.Filter.ClassPaths.Add(ClassName);
		}
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);

	return MenuBuilder.MakeWidget();
}


void FAudioTrackEditor::OnAudioAssetSelected(const FAssetData& AssetData, UMovieSceneTrack* Track)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();

	if (SelectedObject)
	{
		USoundBase* NewSound = CastChecked<USoundBase>(AssetData.GetAsset());
		if (NewSound != nullptr)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddAudio_Transaction", "Add Audio"));

			auto AudioTrack = Cast<UMovieSceneAudioTrack>(Track);
			AudioTrack->Modify();

			FFrameTime KeyTime = GetSequencer()->GetLocalTime().Time;
			UMovieSceneSection* NewSection = AudioTrack->AddNewSound( NewSound, KeyTime.FrameNumber );				

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();

			GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
		}
	}
}

void FAudioTrackEditor::OnAudioAssetEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* Track)
{
	if (AssetData.Num() > 0)
	{
		OnAudioAssetSelected(AssetData[0].GetAsset(), Track);
	}
}

void FAudioTrackEditor::OnAttachedAudioAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();

	if (SelectedObject)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddAudio_Transaction", "Add Audio"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			HandleAssetAdded(SelectedObject, ObjectBinding);
		}
	}
}

void FAudioTrackEditor::OnAttachedAudioEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings)
{
	if (AssetData.Num() > 0)
	{
		OnAttachedAudioAssetSelected(AssetData[0].GetAsset(), ObjectBindings);
	}
}

void FAudioTrackEditor::RegisterMovieSceneChangedDelegate()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		if (SequenceContainsAudioTrack(SequencerPtr->GetRootMovieSceneSequence()))
		{
			// This sequence already has an audio track. Don't install the delegate.
			return;
		}
	
		// Add delegate for scene data change events
		MovieSceneChangedDelegate = SequencerPtr->OnMovieSceneDataChanged().AddSP(this, &FAudioTrackEditor::OnMovieSceneDataChanged);
	}
}

void FAudioTrackEditor::OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType)
{
	if (InChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemAdded)
	{
		if (CheckSequenceClockSource())
		{
			// The user has been notified, remove the delegate
			TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
			if (SequencerPtr.IsValid() && MovieSceneChangedDelegate.IsValid())
			{
				SequencerPtr->OnMovieSceneDataChanged().Remove(MovieSceneChangedDelegate);
				MovieSceneChangedDelegate.Reset();
			}
		}
	}
}

bool FAudioTrackEditor::SequenceContainsAudioTrack(const UMovieSceneSequence* InSequence)
{
	if (!InSequence)
	{
		return false;
	}

	if (UMovieScene* MovieScene = InSequence->GetMovieScene())
	{
		for (const UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (Cast<UMovieSceneAudioTrack>(Track))
			{
				return true;
			}

			const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
			if (!SubTrack)
			{
				continue;
			}

			for (const UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				if (!SubSection)
				{
					continue;
				}

				UMovieSceneSequence* SubSequence = SubSection->GetSequence();
				if (!SubSequence)
				{
					continue;
				}
				else
				{
					if (SequenceContainsAudioTrack(SubSequence))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

bool FAudioTrackEditor::CheckSequenceClockSource()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* RootSequence = SequencerPtr.IsValid() ? SequencerPtr->GetRootMovieSceneSequence() : nullptr;

	if (RootSequence)
	{
		if (UMovieScene* MovieScene = RootSequence->GetMovieScene())
		{
			const bool bHasAudioTrack = SequenceContainsAudioTrack(RootSequence);
			const bool bIsUsingAudioClock = (MovieScene->GetClockSource() == EUpdateClockSource::Audio);

			if (bIsUsingAudioClock)
			{
				// If sequence is already using audio clock, we're done
				return true;
			} 
			else if (bHasAudioTrack)
			{
				if (!MovieScene->IsReadOnly())
				{
					PromptUserForClockSource();
				}

				// Only prompt once per sequencer instance to avoid dialog thrashing
				return true;
			}
		}
	}

	return false;
}

void FAudioTrackEditor::PromptUserForClockSource()
{
	FSuppressableWarningDialog::FSetupInfo SetupInfo(
		LOCTEXT("AutoSelectAudioClockSource_Message", "It is recommended to use the audio clock as the clock source when working with audio tracks in sequencer for improved synchronization between animation and audio. Would you like to switch the clock source now?"),
		LOCTEXT("AutoSelectAudioClockSource_Title", "Use Audio Clock Source?"),
		TEXT("AutoSelectAudioClockSource_SuppressDialog"));

	SetupInfo.ConfirmText = LOCTEXT("AutoSelectAudioClockSource_ConfirmText", "Yes");
	SetupInfo.CancelText = LOCTEXT("AutoSelectAudioClockSource_CancelText", "No");
	SetupInfo.CheckBoxText = LOCTEXT("AutoSelectAudioClockSource_CheckBoxText", "Don't show this again");
	SetupInfo.bDefaultToSuppressInTheFuture = false;
	SetupInfo.DialogMode = FSuppressableWarningDialog::EMode::PersistUserResponse;
	
	FSuppressableWarningDialog SwitchToAudioClockSourceDialog(SetupInfo);
	FSuppressableWarningDialog::EResult Result = SwitchToAudioClockSourceDialog.ShowModal();

	if (Result == FSuppressableWarningDialog::Confirm)
	{
		// Configure this sequence's clock source to use the audio clock
		SetClockSoureToAudioClock();
	}
}

void FAudioTrackEditor::SetClockSoureToAudioClock()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		UMovieSceneSequence* RootSequence = SequencerPtr->GetRootMovieSceneSequence();
		UMovieScene* MovieScene = RootSequence ? RootSequence->GetMovieScene() : nullptr;

		if (MovieScene)
		{
			if (MovieScene->GetClockSource() != EUpdateClockSource::Audio && !MovieScene->IsReadOnly())
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("SetClockSoureToAudioClock", "Set Clock Source"));

				MovieScene->Modify();
				MovieScene->SetClockSource(EUpdateClockSource::Audio);

				SequencerPtr->ResetTimeController();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
