// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/BindingLifetimeTrackEditor.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "MovieSceneTrackEditor.h"
#include "Sections/BindingLifetimeSection.h"
#include "Widgets/SBoxPanel.h"
#include "MVVM/Views/ViewUtilities.h"

class ISequencerTrackEditor;


#define LOCTEXT_NAMESPACE "FBindingLifetimeTrackEditor"


TSharedRef<ISequencerTrackEditor> FBindingLifetimeTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FBindingLifetimeTrackEditor(InSequencer));
}


FBindingLifetimeTrackEditor::FBindingLifetimeTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }

TSharedRef<ISequencerSection> FBindingLifetimeTrackEditor::MakeSectionInterface(UMovieSceneSection & SectionObject, UMovieSceneTrack & Track, FGuid ObjectBinding)
{
	return MakeShared<FBindingLifetimeSection>(SectionObject, GetSequencer());
}

void FBindingLifetimeTrackEditor::CreateNewSection(UMovieSceneTrack* Track, bool bSelect)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
		FQualifiedFrameTime CurrentTime = SequencerPtr->GetLocalTime();

		FScopedTransaction Transaction(LOCTEXT("CreateNewSectionTransactionText", "Add Section"));


		UMovieSceneSection* NewSection = nullptr;

		Track->Modify();

		// first section by default should be infinite
		if (Track->GetAllSections().Num() == 0)
		{
			NewSection = NewObject<UMovieSceneSection>(Track, UMovieSceneBindingLifetimeSection::StaticClass(), NAME_None, RF_Transactional);
			NewSection->SetRange(TRange<FFrameNumber>::All());
			Track->AddSection(*NewSection);
		}
		else
		{
			// If the start time overlaps an existing section, split that section at the start time
			TArray<UMovieSceneSection*> ExistingSections = Track->GetAllSections();
			for(UMovieSceneSection* Section : ExistingSections)
			{
				TRange<FFrameNumber> SectionRange = Section->GetRange();
				if (SectionRange.Contains(CurrentTime.Time.FrameNumber))
				{
					// Edge case- the section the start is overlapping we're just at the start of it (can happen when adding 2 sections back to back at the same time)
					// In that case, push back the start of the other section
					if (SectionRange.GetLowerBound().IsClosed() && SectionRange.GetLowerBoundValue() == CurrentTime.Time.FrameNumber)
					{
						FFrameNumber AdjustmentFrames = SequencerPtr->GetFocusedTickResolution().AsFrameNumber(1);
						if (SectionRange.HasUpperBound())
						{
							AdjustmentFrames = (SectionRange.GetUpperBoundValue() - SectionRange.GetLowerBoundValue()) / 2;
						}

						SectionRange.SetLowerBoundValue(SectionRange.GetLowerBoundValue() + AdjustmentFrames);
						Section->SetRange(SectionRange);

						NewSection = NewObject<UMovieSceneSection>(Track, UMovieSceneBindingLifetimeSection::StaticClass(), NAME_None, RF_Transactional);
						TRange<FFrameNumber> NewSectionRange;
						NewSectionRange.SetLowerBound(TRangeBound<FFrameNumber>(CurrentTime.Time.FrameNumber));
						TRangeBound<FFrameNumber> NewUpperBound(SectionRange.GetLowerBoundValue());
						if (NewUpperBound.IsInclusive())
						{
							NewUpperBound = TRangeBound<FFrameNumber>::FlipInclusion(NewUpperBound);
						}
						NewSectionRange.SetUpperBound(NewUpperBound);
						NewSection->SetRange(NewSectionRange);
						Track->AddSection(*NewSection);
					}
					else
					{
						// Splitting adds the section to the track
						NewSection = Section->SplitSection(CurrentTime, false);
					}
					break;
				}
			}

			// If we didn't overlap anything, add a new section starting at the frame time and ending either at the next section start, or if not existing, make the duration infinite
			if (!NewSection)
			{
				NewSection = NewObject<UMovieSceneSection>(Track, UMovieSceneBindingLifetimeSection::StaticClass(), NAME_None, RF_Transactional);
				TRange<FFrameNumber> NewSectionRange;
				NewSectionRange.SetLowerBound(TRangeBound<FFrameNumber>(CurrentTime.Time.FrameNumber));
				// By default, set the upper bound to open
				NewSectionRange.SetUpperBound(TRangeBound<FFrameNumber>::Open());

				UMovieSceneSection* NextSection = nullptr;
				TRangeBound<FFrameNumber> NextSectionLowerBound = TRangeBound<FFrameNumber>::Open();
				for (UMovieSceneSection* Section : ExistingSections)
				{
					if (Section->GetRange().HasLowerBound() && Section->GetRange().GetLowerBoundValue() > CurrentTime.Time.FrameNumber)
					{
						if (NextSectionLowerBound.IsOpen() || Section->GetRange().GetLowerBoundValue() < NextSectionLowerBound.GetValue())
						{
							NextSection = Section;
							NextSectionLowerBound = Section->GetRange().GetLowerBound();
						}
					}
				}

				if (NextSection)
				{
					// We found an existing section with a lower bound greater than time, so cap our upper bound value to its lower bound
					TRangeBound<FFrameNumber> NewUpperBound(NextSectionLowerBound.GetValue());
					if (NewUpperBound.IsInclusive())
					{
						NewUpperBound = TRangeBound<FFrameNumber>::FlipInclusion(NewUpperBound);
					}
					NewSectionRange.SetUpperBound(NewUpperBound);
				}

				NewSection->SetRange(NewSectionRange);
				Track->AddSection(*NewSection);
			}
		}

		Track->UpdateEasing();

		if (bSelect)
		{
			SequencerPtr->EmptySelection();
			SequencerPtr->SelectSection(NewSection);
			SequencerPtr->ThrobSectionSelection();
		}

		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

FText FBindingLifetimeTrackEditor::GetDisplayName() const
{
	return LOCTEXT("BindingLifetimeTrackEditor_DisplayName", "Binding Lifetime");
}

UMovieSceneTrack* FBindingLifetimeTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	UMovieSceneTrack* NewTrack = FMovieSceneTrackEditor::AddTrack(FocusedMovieScene, ObjectHandle, TrackClass, UniqueTypeName);

	if (auto* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(NewTrack))
	{
		BindingLifetimeTrack->Modify();
		CreateNewSection(BindingLifetimeTrack, false);
	}

	return NewTrack;
}


void FBindingLifetimeTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	UMovieSceneSequence* MovieSequence = GetSequencer()->GetFocusedMovieSceneSequence();

	// TODO: Do we want to restrict this to level sequences or no?
	if (!MovieSequence || MovieSequence->GetClass()->GetName() != TEXT("LevelSequence"))
	{
		return;
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddBindingLifetimeTrack", "Binding Lifetime"),
		LOCTEXT("AddBindingLifetimeTrackTooltip", "Adds a new track that controls the lifetime of the track's object binding."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FBindingLifetimeTrackEditor::HandleAddBindingLifetimeTrackMenuEntryExecute, ObjectBindings),
			FCanExecuteAction::CreateSP(this, &FBindingLifetimeTrackEditor::CanAddBindingLifetimeTrack, ObjectBindings[0])
		)
	);
}


bool FBindingLifetimeTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneBindingLifetimeTrack::StaticClass());
}


bool FBindingLifetimeTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneBindingLifetimeTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}


void FBindingLifetimeTrackEditor::HandleAddBindingLifetimeTrackMenuEntryExecute(TArray<FGuid> ObjectBindings)
{
	FScopedTransaction AddSpawnTrackTransaction(LOCTEXT("AddBindingLifetimeTrack_Transaction", "Add Binding Lifetime Track"));

	for (FGuid ObjectBinding : ObjectBindings)
	{
		AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectBinding, UMovieSceneBindingLifetimeTrack::StaticClass(), NAME_None);
	}
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}


bool FBindingLifetimeTrackEditor::CanAddBindingLifetimeTrack(FGuid ObjectBinding) const
{
	return !GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieSceneBindingLifetimeTrack>(ObjectBinding);
}

TSharedPtr<SWidget> FBindingLifetimeTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	check(Track);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	auto OnClickedCallback = [this, Track]() -> FReply
	{
		CreateNewSection(Track, true);
		return FReply::Handled();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddSection", "Section"), FOnClicked::CreateLambda(OnClickedCallback), Params.ViewModel);
}


#undef LOCTEXT_NAMESPACE
