// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CVarTrackEditor.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Sections/MovieSceneCVarSection.h"
#include "SequencerUtilities.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MovieSceneToolHelpers.h"

#include "SequencerSectionPainter.h"

#define LOCTEXT_NAMESPACE "FCVarTrackEditor"

FCVarTrackEditor::FCVarTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer) 
{
}

void FCVarTrackEditor::OnRelease()
{	
}

TSharedRef<ISequencerTrackEditor> FCVarTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCVarTrackEditor(InSequencer));
}

FText FCVarTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CVarTrackEditor_DisplayName", "CVar");
}

// ISequencerTrackEditor interface
void FCVarTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddCVarTrack", "Console Variable Track"),
		LOCTEXT("AddCVarTooltip", "Adds a new Console Variable track which will allow you to change console variables while a sequence plays."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.CVar"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FCVarTrackEditor::HandleAddCVarTrackMenuEntryExecute)
		)
	);
}

TSharedPtr<SWidget> FCVarTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencer();
	const int32 RowIndex = Params.TrackInsertRowIndex;
	auto OnClickedCallback = [=]() -> FReply
	{
		FSequencerUtilities::CreateNewSection(Track, WeakSequencer, RowIndex, EMovieSceneBlendType::Absolute);
		return FReply::Handled();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("VarText", "Section"), FOnClicked::CreateLambda(OnClickedCallback), Params.ViewModel);
}


TSharedRef<ISequencerSection> FCVarTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShareable(new FCVarSection(SectionObject, GetSequencer()));
}


bool FCVarTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneCVarTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FCVarTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneCVarTrack::StaticClass());
}

const FSlateBrush* FCVarTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.CVar");
}
// ~SequencerTrackEditor interface


// FCVarTrackEditor implementation
void FCVarTrackEditor::HandleAddCVarTrackMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddCVarTrack_Transaction", "Add Console Variable Track"));
	FocusedMovieScene->Modify();
	
	UMovieSceneCVarTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneCVarTrack>();
	ensure(NewTrack);

	NewTrack->SetDisplayName(LOCTEXT("CVarTrackName", "Console Variable"));

	UMovieSceneSection* NewSection = NewTrack->CreateNewSection();
	check(NewSection);

	NewTrack->Modify();
	NewTrack->AddSection(*NewSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}


FCVarSection::FCVarSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer )
	: Section( InSection )
	, Sequencer(InSequencer)
{
}

FCVarSection::~FCVarSection()
{
}

UMovieSceneSection* FCVarSection::GetSectionObject()
{ 
	return &Section;
}

FText FCVarSection::GetSectionTitle() const
{
	return GetCVarText();
}

FText FCVarSection::GetCVarText() const
{
	UMovieSceneCVarSection* CVarSection = Cast<UMovieSceneCVarSection>(&Section);
	if (CVarSection)
	{
		return FText::FromString(CVarSection->GetString());
	}
	return LOCTEXT("NoCVarTitleName", "No Variables");
}

FText FCVarSection::GetSectionToolTip() const
{
	return GetSectionTitle();
}

int32 FCVarSection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	int32 LayerId = Painter.PaintSectionBackground();
	return LayerId;
}

void FCVarSection::BeginResizeSection()
{
}

void FCVarSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FCVarSection::BeginSlipSection()
{

}

void FCVarSection::SlipSection(FFrameNumber SlipTime)
{
	ISequencerSection::SlipSection(SlipTime);
}

#undef LOCTEXT_NAMESPACE