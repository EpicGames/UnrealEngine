// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"
#include "MovieSceneToolHelpers.h"

class FMenuBuilder;
class FCVarTrackEditor;
class FCVarSection;

/**
 * Tools for Console Variable tracks
 */
class FCVarTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FCVarTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCVarTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer .
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void OnRelease() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	virtual const FSlateBrush* GetIconBrush() const override;
private:

	/** Callback for executing the "Add Console Variable Track" menu entry. */
	void HandleAddCVarTrackMenuEntryExecute();
};

/**
 * Class for Console Variable sections
 */
class FCVarSection
	: public ISequencerSection
	, public TSharedFromThis<FCVarSection>
{
public:

	/** Constructor. */
	FCVarSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCVarSection();

public:

	// ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual FText GetSectionTitle() const override;
	virtual FText GetSectionToolTip() const override;
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	virtual void BeginSlipSection() override;
	virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual float GetSectionHeight() const { return SequencerSectionConstants::DefaultSectionHeight*3.f; }

private:

	FText GetCVarText() const;

	/** The section we are visualizing. */
	UMovieSceneSection& Section;

	TWeakPtr<ISequencer> Sequencer;
};

