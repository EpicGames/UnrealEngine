// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CameraCutTrackEditor.h"

#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "MovieSceneToolHelpers.h"
#include "Sections/CameraCutSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"

#include "ActorEditorUtils.h"
#include "ActorTreeItem.h"
#include "Application/ThrottleManager.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/WorldSettings.h"
#include "ISequencer.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "SequencerSettings.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FCameraCutTrackEditor"


class FCameraCutTrackCommands
	: public TCommands<FCameraCutTrackCommands>
{
public:

	FCameraCutTrackCommands()
		: TCommands<FCameraCutTrackCommands>
	(
		"CameraCutTrack",
		NSLOCTEXT("Contexts", "CameraCutTrack", "CameraCutTrack"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
		, BindingCount(0)
	{ }
		
	/** Toggle the camera lock */
	TSharedPtr< FUICommandInfo > ToggleLockCamera;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

	mutable uint32 BindingCount;
};


void FCameraCutTrackCommands::RegisterCommands()
{
	UI_COMMAND( ToggleLockCamera, "Toggle Lock Camera", "Toggle locking the viewport to the camera cut track.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::C) );
}


/* FCameraCutTrackEditor structors
 *****************************************************************************/

FCameraCutTrackEditor::FCameraCutTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer) 
{
	ThumbnailPool = MakeShareable(new FTrackEditorThumbnailPool(InSequencer));

	FCameraCutTrackCommands::Register();
}

void FCameraCutTrackEditor::OnRelease()
{	
	const FCameraCutTrackCommands& Commands = FCameraCutTrackCommands::Get();
	Commands.BindingCount--;
	
	if (Commands.BindingCount < 1)
	{
		FCameraCutTrackCommands::Unregister();
	}
}

TSharedRef<ISequencerTrackEditor> FCameraCutTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCameraCutTrackEditor(InSequencer));
}


/* ISequencerTrackEditor interface
 *****************************************************************************/

FText FCameraCutTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CameraCutTrackEditor_DisplayName", "Camera Cut");
}

void FCameraCutTrackEditor::BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings)
{
	const FCameraCutTrackCommands& Commands = FCameraCutTrackCommands::Get();

	SequencerCommandBindings->MapAction(
		Commands.ToggleLockCamera,
		FExecuteAction::CreateSP( this, &FCameraCutTrackEditor::ToggleLockCamera) );

	TSharedPtr<FUICommandList> CurveEditorSharedBindings = GetSequencer()->GetCommandBindings(ESequencerCommandBindings::CurveEditor);
	if (CurveEditorSharedBindings)
	{	
		CurveEditorSharedBindings->MapAction(Commands.ToggleLockCamera, *SequencerCommandBindings->GetActionForCommand(Commands.ToggleLockCamera));
	}

	Commands.BindingCount++;
}

void FCameraCutTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddCameraCutTrack", "Camera Cut Track"),
		LOCTEXT("AddCameraCutTooltip", "Adds a camera cut track, as well as a new camera cut at the current scrubber location if a camera is selected."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.CameraCut"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FCameraCutTrackEditor::HandleAddCameraCutTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FCameraCutTrackEditor::HandleAddCameraCutTrackMenuEntryCanExecute)
		)
	);
}

void FCameraCutTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(Track);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CanBlendShots", "Can Blend"),
		LOCTEXT("CanBlendShotsTooltip", "Enable shot blending on this track, making it possible to overlap sections."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCameraCutTrackEditor::HandleToggleCanBlendExecute, CameraCutTrack),
			FCanExecuteAction::CreateLambda([=]() { return CameraCutTrack != nullptr; }),
			FIsActionChecked::CreateLambda([=]() { return CameraCutTrack->bCanBlend; })
			),
		"Edit",
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoArrangeShots", "Auto Arrange"),
		LOCTEXT("AutoArrangeShotsTooltip", "Auto-arrange and resize sections to fill gaps."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCameraCutTrackEditor::HandleToggleAutoArrangeSectionsExecute, CameraCutTrack),
			FCanExecuteAction::CreateLambda([=]() { return CameraCutTrack != nullptr; }),
			FIsActionChecked::CreateLambda([=]() { return CameraCutTrack->IsAutoManagingSections(); })
			),
		"Edit",
		EUserInterfaceActionType::ToggleButton
	);
}

void FCameraCutTrackEditor::HandleToggleCanBlendExecute(UMovieSceneCameraCutTrack* CameraCutTrack)
{
	const FScopedTransaction Transaction(LOCTEXT("CameraCutTrackSetCanBlend", "Set Camera Cut Track Can Blend"));

	CameraCutTrack->Modify();

	CameraCutTrack->bCanBlend = !CameraCutTrack->bCanBlend;

	if (!CameraCutTrack->bCanBlend)
	{
		CameraCutTrack->RearrangeAllSections();
	}
}

void FCameraCutTrackEditor::HandleToggleAutoArrangeSectionsExecute(UMovieSceneCameraCutTrack* CameraCutTrack)
{
	const FScopedTransaction Transaction(LOCTEXT("CameraCutTrackSetAutoArrangeSections", "Set Camera Cut Track Auto Arrange"));

	CameraCutTrack->Modify();

	CameraCutTrack->SetIsAutoManagingSections(!CameraCutTrack->IsAutoManagingSections());

	if (CameraCutTrack->IsAutoManagingSections())
	{
		CameraCutTrack->RearrangeAllSections();
	}
}

TSharedPtr<SWidget> FCameraCutTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	if (ColumnName == FCommonOutlinerNames::Add)
	{
		return UE::Sequencer::MakeAddButton(
			LOCTEXT("CameraCutText", "Camera"),
			FOnGetContent::CreateSP(this, &FCameraCutTrackEditor::HandleAddCameraCutComboButtonGetMenuContent),
			Params.ViewModel);
	}

	bool bAddCameraLock = false;
	if (ColumnName == FCommonOutlinerNames::Nav)
	{
		bAddCameraLock = true;
	}
	else if (ColumnName == FCommonOutlinerNames::KeyFrame)
	{
		// Add the camera lock button to the keyframe column if Nav is disabled
		bAddCameraLock = Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false;
	}
	else if (ColumnName == FCommonOutlinerNames::Edit)
	{
		// Add the camera lock button to the edit column if both Nav and KeyFrame are disabled
		bAddCameraLock = Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false &&
			Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame) == false;
	}

	if (bAddCameraLock)
	{
		TSharedRef<SWidget> Button = SNew(SCheckBox)
		.Style(FAppStyle::Get(), "Sequencer.Outliner.ToggleButton")
		.Type(ESlateCheckBoxType::ToggleButton)
		.IsFocusable(false)
		.IsChecked(this, &FCameraCutTrackEditor::IsCameraLocked)
		.OnCheckStateChanged(this, &FCameraCutTrackEditor::OnLockCameraClicked)
		.ToolTipText(this, &FCameraCutTrackEditor::GetLockCameraToolTip)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Sequencer.Outliner.CameraLock"))
		];

		if (ColumnName == FCommonOutlinerNames::Edit)
		{
			// Needs to be left aligned in the edit column because this column slot is set to fill
			return SNew(SBox)
			.HAlign(HAlign_Left)
			.Padding(4.f, 0.f)
			[
				Button
			];
		}
		else
		{
			return Button;
		}
	}

	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);;
}


TSharedRef<ISequencerSection> FCameraCutTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FCameraCutSection(GetSequencer(), ThumbnailPool, SectionObject));
}


bool FCameraCutTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneCameraCutTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}


bool FCameraCutTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneCameraCutTrack::StaticClass());
}


void FCameraCutTrackEditor::Tick(float DeltaTime)
{
	TSharedPtr<ISequencer> SequencerPin = GetSequencer();
	if (!SequencerPin.IsValid())
	{
		return;
	}

	EMovieScenePlayerStatus::Type PlaybackState = SequencerPin->GetPlaybackStatus();

	UObject* PlaybackContext = SequencerPin->GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	// Render thumbnails if allow expensive tasks and playback state is not playing or scrubbing and if in a PIE world, the sequence must be bound to it
	if (FSlateThrottleManager::Get().IsAllowingExpensiveTasks() 
		&& PlaybackState != EMovieScenePlayerStatus::Playing 
		&& PlaybackState != EMovieScenePlayerStatus::Scrubbing
		&& (!bIsInPIEOrSimulate || (bIsInPIEOrSimulate && GEditor->PlayWorld == World)) )
	{
		SequencerPin->EnterSilentMode();

		FQualifiedFrameTime SavedTime = SequencerPin->GetLocalTime();

		if (DeltaTime > 0.f && ThumbnailPool->DrawThumbnails())
		{
			SequencerPin->SetLocalTimeDirectly(SavedTime.Time);
		}

		SequencerPin->ExitSilentMode();
	}
}


const FSlateBrush* FCameraCutTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.CameraCut");
}


bool FCameraCutTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid() || !DragDropParams.Track.Get()->IsA(UMovieSceneCameraCutTrack::StaticClass()))
	{
		return false;
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FActorDragDropGraphEdOp>() )
	{
		return false;
	}
	
	UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(DragDropParams.Track.Get());

	TSharedPtr<FActorDragDropGraphEdOp> DragDropOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>( Operation );

	for (auto& ActorPtr : DragDropOp->Actors)
	{
		if (ActorPtr.IsValid())
		{
			AActor* Actor = ActorPtr.Get();
				
			UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(Actor);
			if (CameraComponent)
			{
				FFrameNumber EndFrameNumber = CameraCutTrack->FindEndTimeForCameraCut(DragDropParams.FrameNumber);
				DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, EndFrameNumber);
				return true;
			}
		}
	}

	return false;
}


FReply FCameraCutTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid() || !DragDropParams.Track.Get()->IsA(UMovieSceneCameraCutTrack::StaticClass()))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FActorDragDropGraphEdOp>() )
	{
		return FReply::Unhandled();
	}
	
	TSharedPtr<FActorDragDropGraphEdOp> DragDropOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>( Operation );

	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (auto& ActorPtr : DragDropOp->Actors)
	{
		if (ActorPtr.IsValid())
		{
			AActor* Actor = ActorPtr.Get();
				
			FGuid ObjectGuid = FindOrCreateHandleToObject(Actor).Handle;
	
			if (ObjectGuid.IsValid())
			{
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCameraCutTrackEditor::AddKeyInternal, ObjectGuid));
				
				bAnyDropped = true;
			}
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}


/* FCameraCutTrackEditor implementation
 *****************************************************************************/

FKeyPropertyResult FCameraCutTrackEditor::AddKeyInternal( FFrameNumber KeyTime, const FGuid ObjectGuid )
{
	FKeyPropertyResult KeyPropertyResult;

	UMovieSceneCameraCutTrack* CameraCutTrack = FindOrCreateCameraCutTrack();
	const TArray<UMovieSceneSection*>& AllSections = CameraCutTrack->GetAllSections();

	UMovieSceneCameraCutSection* NewSection = CameraCutTrack->AddNewCameraCut(UE::MovieScene::FRelativeObjectBindingID(ObjectGuid), KeyTime);
	KeyPropertyResult.bTrackModified = true;
	KeyPropertyResult.SectionsCreated.Add(NewSection);

	GetSequencer()->EmptySelection();
	GetSequencer()->SelectSection(NewSection);
	GetSequencer()->ThrobSectionSelection();

	return KeyPropertyResult;
}


UMovieSceneCameraCutTrack* FCameraCutTrackEditor::FindOrCreateCameraCutTrack()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene->IsReadOnly())
	{
		return nullptr;
	}

	UMovieSceneTrack* CameraCutTrack = FocusedMovieScene->GetCameraCutTrack();

	if (CameraCutTrack == nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddCameraCutTrack_Transaction", "Add Camera Cut Track"));
		FocusedMovieScene->Modify();
		
		CameraCutTrack = FocusedMovieScene->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
	}

	return CastChecked<UMovieSceneCameraCutTrack>(CameraCutTrack);
}


/* FCameraCutTrackEditor callbacks
 *****************************************************************************/

bool FCameraCutTrackEditor::HandleAddCameraCutTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->GetCameraCutTrack() == nullptr));
}


void FCameraCutTrackEditor::HandleAddCameraCutTrackMenuEntryExecute()
{
	UMovieSceneCameraCutTrack* CameraCutTrack = FindOrCreateCameraCutTrack();

	if (CameraCutTrack)
	{
		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(CameraCutTrack, FGuid());
		}
	}
}

bool FCameraCutTrackEditor::IsCameraPickable(const AActor* const PickableActor)
{
	if (PickableActor->IsListedInSceneOutliner() &&
		!FActorEditorUtils::IsABuilderBrush(PickableActor) &&
		!PickableActor->IsA( AWorldSettings::StaticClass() ) &&
		IsValid(PickableActor))
	{	
		UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(PickableActor);
		if (CameraComponent)	
		{
			return true;
		}
	}
	return false;
}

TSharedRef<SWidget> FCameraCutTrackEditor::HandleAddCameraCutComboButtonGetMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	auto CreateNewCamera =
		[this](FMenuBuilder& SubMenuBuilder)
		{
			FSceneOutlinerInitializationOptions InitOptions;
			{
				InitOptions.bShowHeaderRow = false;
				InitOptions.bFocusSearchBoxWhenOpened = true;
				InitOptions.bShowTransient = true;
				InitOptions.bShowCreateNewFolder = false;
				// Only want the actor label column
				InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

				// Only display Actors that we can attach too
				InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateRaw(this, &FCameraCutTrackEditor::IsCameraPickable));
			}		

			// Actor selector to allow the user to choose a parent actor
			FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>( "SceneOutliner" );

			TSharedPtr<ISequencer> Sequencer = GetSequencer();
			const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
			const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

			TSharedRef< SWidget > MenuWidget =
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(WidthOverride)
					.HeightOverride(HeightOverride)
					[
						SceneOutlinerModule.CreateActorPicker(
							InitOptions,
							FOnActorPicked::CreateSP(this, &FCameraCutTrackEditor::HandleAddCameraCutComboButtonMenuEntryExecute )
							)
					]
				];
			SubMenuBuilder.AddWidget(MenuWidget, FText::GetEmpty(), false);
		};

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	// Always recreate the binding picker to ensure we have the correct sequence ID
	BindingIDPicker = MakeShared<FTrackEditorBindingIDPicker>(SequencerPtr->GetFocusedTemplateID(), SequencerPtr);
	BindingIDPicker->OnBindingPicked().AddRaw(this, &FCameraCutTrackEditor::CreateNewSectionFromBinding);

	FText ExistingBindingText = LOCTEXT("ExistingBinding", "Existing Binding");
	FText NewBindingText = LOCTEXT("NewBinding", "New Binding");

	const bool bHasExistingBindings = !BindingIDPicker->IsEmpty();
	if (bHasExistingBindings)
	{
		MenuBuilder.AddSubMenu(
			NewBindingText,
			LOCTEXT("NewBinding_Tip", "Add a new camera cut by creating a new binding to an object in the world."),
			FNewMenuDelegate::CreateLambda(CreateNewCamera)
		);

		MenuBuilder.BeginSection(NAME_None, ExistingBindingText);
		{
			BindingIDPicker->GetPickerMenu(MenuBuilder);
		}
		MenuBuilder.EndSection();
	}
	else
	{
		MenuBuilder.BeginSection(NAME_None, NewBindingText);
		{
			CreateNewCamera(MenuBuilder);
		}
		MenuBuilder.EndSection();
	}
	
	return MenuBuilder.MakeWidget();
}


void FCameraCutTrackEditor::CreateNewSectionFromBinding(FMovieSceneObjectBindingID InBindingID)
{
	auto CreateNewSection = [this, InBindingID](FFrameNumber KeyTime)
	{
		FKeyPropertyResult KeyPropertyResult;

		UMovieSceneCameraCutSection* NewSection = FindOrCreateCameraCutTrack()->AddNewCameraCut(InBindingID, KeyTime);
		KeyPropertyResult.bTrackModified = true;
		KeyPropertyResult.SectionsCreated.Add(NewSection);

		GetSequencer()->EmptySelection();
		GetSequencer()->SelectSection(NewSection);
		GetSequencer()->ThrobSectionSelection();
		
		return KeyPropertyResult;
	};

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(CreateNewSection));
}


void FCameraCutTrackEditor::HandleAddCameraCutComboButtonMenuEntryExecute(AActor* Camera)
{
	FGuid ObjectGuid = FindOrCreateHandleToObject(Camera).Handle;

	if (ObjectGuid.IsValid())
	{
		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCameraCutTrackEditor::AddKeyInternal, ObjectGuid));
	}
}

ECheckBoxState FCameraCutTrackEditor::IsCameraLocked() const
{
	if (GetSequencer()->IsPerspectiveViewportCameraCutEnabled())
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}


void FCameraCutTrackEditor::OnLockCameraClicked(ECheckBoxState CheckBoxState)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	const bool bEnableCameraCuts = (CheckBoxState == ECheckBoxState::Checked);
	SequencerPtr->SetPerspectiveViewportCameraCutEnabled(bEnableCameraCuts);

	bool bNeedsRestoreViewport = true;
	if (const USequencerSettings* SequencerSettings = SequencerPtr->GetSequencerSettings())
	{
		bNeedsRestoreViewport = SequencerSettings->GetRestoreOriginalViewportOnCameraCutUnlock();
	}

	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	UMovieSceneCameraCutTrackInstance::ToggleCameraCutLock(Linker, bEnableCameraCuts, bNeedsRestoreViewport);

	SequencerPtr->ForceEvaluate();
}

void FCameraCutTrackEditor::ToggleLockCamera()
{
	OnLockCameraClicked(IsCameraLocked() == ECheckBoxState::Checked ?  ECheckBoxState::Unchecked :  ECheckBoxState::Checked);
}

FText FCameraCutTrackEditor::GetLockCameraToolTip() const
{
	const TSharedRef<const FInputChord> FirstActiveChord = FCameraCutTrackCommands::Get().ToggleLockCamera->GetFirstValidChord();

	FText Tooltip = IsCameraLocked() == ECheckBoxState::Checked ?
		LOCTEXT("UnlockCamera", "Unlock Viewport from Camera Cuts") :
		LOCTEXT("LockCamera", "Lock Viewport to Camera Cuts");
	
	if (FirstActiveChord->IsValidChord())
	{
		return FText::Join(FText::FromString(TEXT(" ")), Tooltip, FirstActiveChord->GetInputText());
	}
	return Tooltip;
}

#undef LOCTEXT_NAMESPACE
