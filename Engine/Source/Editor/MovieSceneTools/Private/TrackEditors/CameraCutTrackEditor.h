// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"
#include "MovieSceneToolHelpers.h"

class AActor;
class FMenuBuilder;
class FTrackEditorThumbnailPool;
class UFactory;
class UMovieSceneCameraCutTrack;
class FCameraCutTrackEditor;
class FTrackEditorBindingIDPicker;
struct FMovieSceneObjectBindingID;
class FCameraCutSection;

/**
 * Tools for camera cut tracks.
 */
class FCameraCutTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FCameraCutTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCameraCutTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer .
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void OnRelease() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual void Tick(float DeltaTime) override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;

protected:

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber AutoKeyTime, const FGuid ObjectGuid);

	/** Find or create a camera cut track in the currently focused movie scene. */
	UMovieSceneCameraCutTrack* FindOrCreateCameraCutTrack();

	UFactory* GetAssetFactoryForNewCameraCut(UClass* SequenceClass);

private:

	/** Callback for determining whether the "Add Camera Cut" menu entry can execute. */
	bool HandleAddCameraCutTrackMenuEntryCanExecute() const;

	/** Callback for executing the "Add Camera Cut Track" menu entry. */
	void HandleAddCameraCutTrackMenuEntryExecute();

	/** Callback for generating the menu of the "Add Camera Cut" combo button. */
	TSharedRef<SWidget> HandleAddCameraCutComboButtonGetMenuContent();

	/** Callback for whether a camera is pickable. */
	bool IsCameraPickable( const AActor* const PickableActor );

	/** Callback for executing a menu entry in the "Add Camera Cut" combo button. */
	void HandleAddCameraCutComboButtonMenuEntryExecute(AActor* Camera);

	/** Called to create a new section for the specified binding ID. */
	void CreateNewSectionFromBinding(FMovieSceneObjectBindingID InBindingID);

	/** Called when toggling whether the track can blend camera cuts. */
	void HandleToggleCanBlendExecute(UMovieSceneCameraCutTrack* CameraCutTrack);

	/** Called when toggling whether the track auto-manages camera cuts. */
	void HandleToggleAutoArrangeSectionsExecute(UMovieSceneCameraCutTrack* CameraCutTrack);

	/** Delegate for camera button lock state */
	ECheckBoxState IsCameraLocked() const; 

	/** Delegate for locked camera button */
	void OnLockCameraClicked(ECheckBoxState CheckBoxState);

	/** Toggle the state of the camera lock */
	void ToggleLockCamera();

	/** Delegate for camera button lock tooltip */
	FText GetLockCameraToolTip() const; 

private:

	/** The Thumbnail pool which draws all the viewport thumbnails for the camera cut track. */
	TSharedPtr<FTrackEditorThumbnailPool> ThumbnailPool;

	/** A binding ID picker that allows us to create a new section from an existing binding */
	TSharedPtr<FTrackEditorBindingIDPicker> BindingIDPicker;
};
