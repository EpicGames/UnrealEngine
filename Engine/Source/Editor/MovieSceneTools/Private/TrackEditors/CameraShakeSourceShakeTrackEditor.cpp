// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CameraShakeSourceShakeTrackEditor.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "TrackEditors/CameraShakeTrackEditorBase.h"
#include "Channels/MovieSceneCameraShakeSourceTriggerChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneCameraShakeSection.h"
#include "Sections/MovieSceneCameraShakeSourceShakeSection.h"
#include "Sections/MovieSceneCameraShakeSourceTriggerSection.h"
#include "SequencerSectionPainter.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/WidgetStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "TimeToPixel.h"
#include "Tracks/MovieSceneCameraShakeSourceShakeTrack.h"
#include "Tracks/MovieSceneCameraShakeSourceTriggerTrack.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
class UMovieScene;

#define LOCTEXT_NAMESPACE "FCameraShakeSourceShakeTrackEditor"

/**
 * Section interface for shake sections
 */
class FCameraShakeSourceShakeSection : public FCameraShakeSectionBase
{
public:
	FCameraShakeSourceShakeSection(const TSharedPtr<ISequencer> InSequencer, UMovieSceneCameraShakeSourceShakeSection& InSection, const FGuid& InObjectBinding)
		: FCameraShakeSectionBase(InSequencer, InSection, InObjectBinding)
	{}

private:
	virtual TSubclassOf<UCameraShakeBase> GetCameraShakeClass() const override
	{
		if (const UMovieSceneCameraShakeSourceShakeSection* SectionObject = GetSectionObjectAs<UMovieSceneCameraShakeSourceShakeSection>())
		{
			if (SectionObject->ShakeData.ShakeClass.Get() != nullptr)
			{
				return SectionObject->ShakeData.ShakeClass;
			}
		}

		TSharedPtr<ISequencer> Sequencer = GetSequencer();
		const FGuid ObjectBinding = GetObjectBinding();
		TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectBinding, Sequencer->GetFocusedTemplateID());
		if (BoundObjects.Num() > 0)
		{
			if (const UCameraShakeSourceComponent* Component = Cast<UCameraShakeSourceComponent>(BoundObjects[0].Get()))
			{
				return Component->CameraShake;
			}
		}

		return TSubclassOf<UCameraShakeBase>();
	}
};

/**
 * Section interface for shake triggers.
 */
class FCameraShakeSourceTriggerSection : public FSequencerSection
{
public:
	FCameraShakeSourceTriggerSection(const TSharedPtr<ISequencer> InSequencer, UMovieSceneCameraShakeSourceTriggerSection& InSectionObject)
		: FSequencerSection(InSectionObject)
		, Sequencer(InSequencer)
	{}

private:
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

	bool IsTrackSelected() const;
	void PaintShakeName(FSequencerSectionPainter& Painter, int32 LayerId, TSubclassOf<UCameraShakeBase> ShakeClass, float PixelPos) const;

	TWeakPtr<ISequencer> Sequencer;
};

bool FCameraShakeSourceTriggerSection::IsTrackSelected() const
{
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();

	TArray<UMovieSceneTrack*> SelectedTracks;
	SequencerPtr->GetSelectedTracks(SelectedTracks);

	UMovieSceneSection* Section = WeakSection.Get();
	UMovieSceneTrack*   Track   = Section ? CastChecked<UMovieSceneTrack>(Section->GetOuter()) : nullptr;
	return Track && SelectedTracks.Contains(Track);
}

void FCameraShakeSourceTriggerSection::PaintShakeName(FSequencerSectionPainter& Painter, int32 LayerId, TSubclassOf<UCameraShakeBase> ShakeClass, float PixelPos) const
{
	static const int32   FontSize      = 10;
	static const float   BoxOffsetPx   = 10.f;
	static const FString AutoShakeText = LOCTEXT("AutoShake", "(Automatic)").ToString();

	const FSlateFontInfo FontAwesomeFont = FAppStyle::Get().GetFontStyle("FontAwesome.10");
	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const FLinearColor   DrawColor       = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

	const FString ShakeText = (ShakeClass.Get() != nullptr) ? ShakeClass.Get()->GetName() : AutoShakeText;

	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const  FMargin   BoxPadding     = FMargin(4.0f, 2.0f);
	const FVector2f  TextSize       = UE::Slate::CastToVector2f(FontMeasureService->Measure(ShakeText, SmallLayoutFont));

	// Flip the text position if getting near the end of the view range
	const bool bDrawLeft = (Painter.SectionGeometry.Size.X - PixelPos) < (TextSize.X + 22.f) - BoxOffsetPx;
	const float BoxPositionX = FMath::Max(
			bDrawLeft ? PixelPos - TextSize.X - BoxOffsetPx : PixelPos + BoxOffsetPx,
			0.f);

	const FVector2f BoxOffset  = FVector2f(BoxPositionX, Painter.SectionGeometry.Size.Y*.5f - TextSize.Y*.5f);
	const FVector2f TextOffset = FVector2f(BoxPadding.Left, 0);

	// Draw the background box.
	FSlateDrawElement::MakeBox(
		Painter.DrawElements,
		LayerId + 1,
		Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(BoxOffset)),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.5f)
	);

	// Draw shake name.
	FSlateDrawElement::MakeText(
		Painter.DrawElements,
		LayerId + 2,
		Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(BoxOffset + TextOffset)),
		ShakeText,
		SmallLayoutFont,
		Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		DrawColor
	);
}

int32 FCameraShakeSourceTriggerSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	int32 LayerId = Painter.PaintSectionBackground();

	UMovieSceneCameraShakeSourceTriggerSection* TriggerSection = Cast<UMovieSceneCameraShakeSourceTriggerSection>(WeakSection.Get());
	if (!TriggerSection || !IsTrackSelected())
	{
		return LayerId;
	}

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();
	const FMovieSceneCameraShakeSourceTriggerChannel& TriggerChannel = TriggerSection->GetChannel();
	const TArrayView<const FFrameNumber> Times = TriggerChannel.GetData().GetTimes();
	const TArrayView<const FMovieSceneCameraShakeSourceTrigger> Values = TriggerChannel.GetData().GetValues();
	const TRange<FFrameNumber> SectionRange = TriggerSection->GetRange();

	for (int32 KeyIndex = 0; KeyIndex < Times.Num(); ++KeyIndex)
	{
		const FFrameNumber Time = Times[KeyIndex];
		if (SectionRange.Contains(Time))
		{
			const FMovieSceneCameraShakeSourceTrigger& Value = Values[KeyIndex];
			const float PixelPos = TimeToPixelConverter.FrameToPixel(Time);
			PaintShakeName(Painter, LayerId, Value.ShakeClass, PixelPos);
		}
	}

	return LayerId + 3;
}

FCameraShakeSourceShakeTrackEditor::FCameraShakeSourceShakeTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer) 
{ 
}

TSharedRef<ISequencerTrackEditor> FCameraShakeSourceShakeTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCameraShakeSourceShakeTrackEditor(InSequencer));
}

bool FCameraShakeSourceShakeTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (
		Type == UMovieSceneCameraShakeSourceShakeTrack::StaticClass() ||
		Type == UMovieSceneCameraShakeSourceTriggerTrack::StaticClass()
		);
}

TSharedRef<ISequencerSection> FCameraShakeSourceShakeTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	if (UMovieSceneCameraShakeSourceShakeSection* ShakeSection = Cast<UMovieSceneCameraShakeSourceShakeSection>(&SectionObject))
	{
		return MakeShareable(new FCameraShakeSourceShakeSection(GetSequencer(), *ShakeSection, ObjectBinding));
	}
	else if (UMovieSceneCameraShakeSourceTriggerSection* TriggerSection = Cast<UMovieSceneCameraShakeSourceTriggerSection>(&SectionObject))
	{
		return MakeShared<FCameraShakeSourceTriggerSection>(GetSequencer(), *TriggerSection);
	}

	check(false);
	return MakeShareable(new FSequencerSection(SectionObject));
}

FText FCameraShakeSourceShakeTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CameraShakeSourceShakeTrackEditor_DisplayName", "Camera Shake Source");
}

UMovieSceneTrack* FCameraShakeSourceShakeTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	UMovieSceneTrack* NewTrack = FMovieSceneTrackEditor::AddTrack(FocusedMovieScene, ObjectHandle, TrackClass, UniqueTypeName);

	if (UMovieSceneCameraShakeSourceTriggerTrack* ShakeTrack = Cast<UMovieSceneCameraShakeSourceTriggerTrack>(NewTrack))
	{
		// If it's a trigger track, auto-add an infinite section in which we can place our trigger keyframes.
		UMovieSceneSection* NewSection = ShakeTrack->CreateNewSection();
		NewSection->SetRange(TRange<FFrameNumber>::All());
		ShakeTrack->AddSection(*NewSection);
	}

	return NewTrack;
}

void FCameraShakeSourceShakeTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (const UCameraShakeSourceComponent* ShakeSourceComponent = AcquireCameraShakeSourceComponentFromGuid(ObjectBindings[0]))
	{
		MenuBuilder.AddSubMenu(
				LOCTEXT("AddShakeSourceShake", "Camera Shake"),
				LOCTEXT("AddShakeSourceShakeTooltip", "Adds a camera shake originating from the parent camera shake source."),
				FNewMenuDelegate::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::AddCameraShakeTracksMenu, ObjectBindings));
	}
}

void FCameraShakeSourceShakeTrackEditor::AddCameraShakeTracksMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	MenuBuilder.AddMenuEntry(
			LOCTEXT("AddShakeSourceShakeControlled", "Controlled"),
			LOCTEXT("AddShakeSourceShakeControlledTooltip", "Adds a track that lets you start and stop camera shakes originating from the parent camera shake source."),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateSP( this, &FCameraShakeSourceShakeTrackEditor::AddCameraShakeSection, ObjectBindings)
				)
			);

	MenuBuilder.AddMenuEntry(
			LOCTEXT("AddShakeSourceShakeTrigger", "Trigger"),
			LOCTEXT("AddShakeSourceShakeTriggerTooltip", "Adds a track that lets you trigger camera shakes originating from the parent camera shake source."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::AddCameraShakeTriggerTrack, ObjectBindings)
				)
			);
}

void FCameraShakeSourceShakeTrackEditor::AddCameraShakeSection(TArray<FGuid> ObjectHandles)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid() || !SequencerPtr->IsAllowedToChange())
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> Objects;
	for (FGuid ObjectHandle : ObjectHandles)
	{
		for (TWeakObjectPtr<UObject> Object : SequencerPtr->FindObjectsInCurrentSequence(ObjectHandle))
		{
			Objects.Add(Object);
		}
	}

	auto OnAddShakeSourceShakeSection = [this, Objects](FFrameNumber Time) -> FKeyPropertyResult
	{
		return this->AddCameraShakeSectionKeyInternal(Time, Objects, true);
	};

	const FScopedTransaction Transaction(LOCTEXT("AddCameraShakeSourceShake_Transaction", "Add Camera Shake"));

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(OnAddShakeSourceShakeSection));
}

TSharedPtr<SWidget> FCameraShakeSourceShakeTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	if (UMovieSceneCameraShakeSourceShakeTrack* ShakeTrack = Cast<UMovieSceneCameraShakeSourceShakeTrack>(Track))
	{
		return UE::Sequencer::MakeAddButton(
						LOCTEXT("AddShakeSourceShakeSection", "Camera Shake"),
						FOnGetContent::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::BuildCameraShakeSubMenu, ObjectBinding),
						Params.ViewModel);
	}
	else
	{
		return UE::Sequencer::MakeAddButton(
						LOCTEXT("AddSection", "Section"),
						FOnGetContent::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::BuildCameraShakeTracksMenu, ObjectBinding),
						Params.ViewModel);
	}
}

FKeyPropertyResult FCameraShakeSourceShakeTrackEditor::AddCameraShakeSectionKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, bool bSelect)
{
	return AddCameraShakeSectionKeyInternal(KeyTime, Objects, TSubclassOf<UCameraShakeBase>(), bSelect);
}

FKeyPropertyResult FCameraShakeSourceShakeTrackEditor::AddCameraShakeSectionKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> CameraShake, bool bSelect)
{
	FKeyPropertyResult KeyPropertyResult;

	TArray<UMovieSceneSection*> SectionsToSelect;

	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		UObject* Object = Objects[ObjectIndex].Get();

		if (AActor* Actor = Cast<AActor>(Object))
		{
			UCameraShakeSourceComponent* Component = Actor->FindComponentByClass<UCameraShakeSourceComponent>();
			if (ensure(Component != nullptr))
			{
				Object = Component;
			}
		}

		const bool bIsAutomaticShake = (CameraShake.Get() == nullptr);
		if (bIsAutomaticShake)
		{
			if (UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(Object))
			{
				CameraShake = ShakeSourceComponent->CameraShake;
			}
		}

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneCameraShakeSourceShakeTrack::StaticClass());
			UMovieSceneTrack* Track = TrackResult.Track;
			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(Track))
			{
				UMovieSceneSection* NewSection = Cast<UMovieSceneCameraShakeSourceShakeTrack>(Track)->AddNewCameraShake(KeyTime, CameraShake, bIsAutomaticShake);
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);
				SectionsToSelect.Add(NewSection);
			}
		}
	}

	if (bSelect)
	{
		const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		SequencerPtr->EmptySelection();
		for (UMovieSceneSection* SectionToSelect : SectionsToSelect)
		{
			SequencerPtr->SelectSection(SectionToSelect);
		}
		SequencerPtr->ThrobSectionSelection();
	}

	return KeyPropertyResult;
}

TSharedRef<SWidget> FCameraShakeSourceShakeTrackEditor::BuildCameraShakeSubMenu(FGuid ObjectBinding)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	AddCameraShakeSubMenu(MenuBuilder, ObjectBindings);

	return MenuBuilder.MakeWidget();
}

void FCameraShakeSourceShakeTrackEditor::AddCameraShakeSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	MenuBuilder.AddMenuEntry(
			LOCTEXT("AddAutoShake", "Automatic Shake"),
			LOCTEXT("AddAutoShakeTooltip", "Adds a section that plays the camera shake already configured on the shake source component."),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::OnAutoCameraShakeSelected, ObjectBindings)
			));

	MenuBuilder.AddSubMenu(
			LOCTEXT("AddOtherShake", "Other Shake"),
			LOCTEXT("AddOtherShakeTooltip", "Adds a section that plays a specific camera shake originating from the shake source component."),
			FNewMenuDelegate::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::AddOtherCameraShakeBrowserSubMenu, ObjectBindings));
}

TSharedRef<SWidget> FCameraShakeSourceShakeTrackEditor::BuildCameraShakeTracksMenu(FGuid ObjectBinding)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	AddCameraShakeTracksMenu(MenuBuilder, ObjectBindings);

	return MenuBuilder.MakeWidget();
}

void FCameraShakeSourceShakeTrackEditor::AddOtherCameraShakeBrowserSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::OnCameraShakeAssetSelected, ObjectBindings);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::OnCameraShakeAssetEnterPressed, ObjectBindings);
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::OnShouldFilterCameraShake);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));

		IAssetRegistry & AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		TArray<FTopLevelAssetPath> ClassNames;
		TSet<FTopLevelAssetPath> DerivedClassNames;
		ClassNames.Add(UCameraShakeBase::StaticClass()->GetClassPathName());
		AssetRegistry.GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);
						
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([DerivedClassNames](const FAssetData& AssetData)
		{
			const FString ParentClassFromData = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
			if (!ParentClassFromData.IsEmpty())
			{
				const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(ParentClassFromData));

				if (DerivedClassNames.Contains(ClassObjectPath))
				{
					return false;
				}
			}
			return true;
		});
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}

void FCameraShakeSourceShakeTrackEditor::OnCameraShakeAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings)
{
	FSlateApplication::Get().DismissAllMenus();

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UBlueprint* SelectedObject = Cast<UBlueprint>(AssetData.GetAsset());

	if (SelectedObject && SelectedObject->GeneratedClass && SelectedObject->GeneratedClass->IsChildOf(UCameraShakeBase::StaticClass()))
	{
		TSubclassOf<UCameraShakeBase> CameraShakeClass = *(SelectedObject->GeneratedClass);

		TArray<TWeakObjectPtr<>> OutObjects;
		for (FGuid ObjectBinding : ObjectBindings)
		{
			for (TWeakObjectPtr<> Object : SequencerPtr->FindObjectsInCurrentSequence(ObjectBinding))
			{
				OutObjects.Add(Object);
			}
		}
		
		const FScopedTransaction Transaction(LOCTEXT("AddCameraShakeSourceShake_Transaction", "Add Camera Shake"));

		AnimatablePropertyChanged(FOnKeyProperty::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::AddCameraShakeSectionKeyInternal, OutObjects, CameraShakeClass, true));
	}
}

void FCameraShakeSourceShakeTrackEditor::OnCameraShakeAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings)
{
	if (AssetData.Num() > 0)
	{
		OnCameraShakeAssetSelected(AssetData[0].GetAsset(), ObjectBindings);
	}
}

void FCameraShakeSourceShakeTrackEditor::OnAutoCameraShakeSelected(TArray<FGuid> ObjectBindings)
{
	TArray<TWeakObjectPtr<>> OutObjects;
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	for (FGuid ObjectBinding : ObjectBindings)
	{
		for (TWeakObjectPtr<> Object : SequencerPtr->FindObjectsInCurrentSequence(ObjectBinding))
		{
			OutObjects.Add(Object);
		}
	}

	const FScopedTransaction Transaction(LOCTEXT("AddCameraShakeSourceShake_Transaction", "Add Camera Shake"));

	AnimatablePropertyChanged(FOnKeyProperty::CreateSP(this, &FCameraShakeSourceShakeTrackEditor::AddCameraShakeSectionKeyInternal, OutObjects, true));
}

bool FCameraShakeSourceShakeTrackEditor::OnShouldFilterCameraShake(const FAssetData& AssetData)
{
	const UBlueprint* SelectedObject = Cast<UBlueprint>(AssetData.GetAsset());
	if (SelectedObject && SelectedObject->GeneratedClass && SelectedObject->GeneratedClass->IsChildOf(UCameraShakeBase::StaticClass()))
	{
		TSubclassOf<UCameraShakeBase> CameraShakeClass = *(SelectedObject->GeneratedClass);
		if (const UCameraShakeBase* CameraShakeCDO = Cast<UCameraShakeBase>(CameraShakeClass->GetDefaultObject(false)))
		{
			return CameraShakeCDO->bSingleInstance;
		}
	}
	return true;
}

FKeyPropertyResult FCameraShakeSourceShakeTrackEditor::AddCameraShakeTriggerTrackInternal(FFrameNumber Time, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> CameraShake)
{
	FKeyPropertyResult KeyPropertyResult;

	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		UObject* Object = Objects[ObjectIndex].Get();

		if (AActor* Actor = Cast<AActor>(Object))
		{
			UCameraShakeSourceComponent* Component = Actor->FindComponentByClass<UCameraShakeSourceComponent>();
			if (ensure(Component != nullptr))
			{
				Object = Component;
			}
		}

		const bool bIsAutomaticShake = (CameraShake.Get() == nullptr);
		if (bIsAutomaticShake)
		{
			if (UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(Object))
			{
				CameraShake = ShakeSourceComponent->CameraShake;
			}
		}

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneCameraShakeSourceTriggerTrack::StaticClass());
			UMovieSceneTrack* Track = TrackResult.Track;
			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(Track))
			{
				TArray<UMovieSceneSection*> AllSections = Track->GetAllSections();
				if (ensure(AllSections.Num() > 0))
				{
					UMovieSceneCameraShakeSourceTriggerSection* FirstSection = Cast<UMovieSceneCameraShakeSourceTriggerSection>(AllSections[0]);
					// TODO: add trigger key at given time.
					GetSequencer()->EmptySelection();
					GetSequencer()->SelectSection(FirstSection);
					GetSequencer()->ThrobSectionSelection();
				}
			}
		}
	}

	return KeyPropertyResult;
}

void FCameraShakeSourceShakeTrackEditor::AddCameraShakeTriggerTrack(const TArray<FGuid> ObjectBindings)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid() || !SequencerPtr->IsAllowedToChange())
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> Objects;
	for (FGuid ObjectBinding : ObjectBindings)
	{
		for (TWeakObjectPtr<UObject> Object : SequencerPtr->FindObjectsInCurrentSequence(ObjectBinding))
		{
			Objects.Add(Object);
		}
	}

	auto OnAddShakeSourceShakeSection = [this, Objects](FFrameNumber Time) -> FKeyPropertyResult
	{
		return this->AddCameraShakeTriggerTrackInternal(Time, Objects, nullptr);
	};

	const FScopedTransaction Transaction(LOCTEXT("AddCameraShakeSourceShake_Transaction", "Add Camera Shake"));

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(OnAddShakeSourceShakeSection));
}

UCameraShakeSourceComponent* FCameraShakeSourceShakeTrackEditor::AcquireCameraShakeSourceComponentFromGuid(const FGuid& Guid)
{
	TArray<UCameraShakeSourceComponent*> ShakeSourceComponents;

	for (TWeakObjectPtr<> WeakObject : GetSequencer()->FindObjectsInCurrentSequence(Guid))
	{
		if (UObject* Obj = WeakObject.Get())
		{
			if (AActor* Actor = Cast<AActor>(Obj))
			{
				TArray<UCameraShakeSourceComponent*> CurShakeSourceComponents;
				Actor->GetComponents(CurShakeSourceComponents);
				ShakeSourceComponents.Append(CurShakeSourceComponents);
			}
			else if (UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(Obj))
			{
				ShakeSourceComponents.Add(ShakeSourceComponent);
			}
		}
	}

	UCameraShakeSourceComponent** ActiveComponent = ShakeSourceComponents.FindByPredicate([](UCameraShakeSourceComponent* Component)
			{
				return Component->IsActive();
			});
	if (ActiveComponent != nullptr)
	{
		return *ActiveComponent;
	}

	if (ShakeSourceComponents.Num() > 0)
	{
		return ShakeSourceComponents[0];
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

