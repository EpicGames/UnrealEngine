// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FMenuBuilder;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class SWidget;
class UCameraShakeBase;
class UCameraShakeSourceComponent;
class UClass;
class UMovieScene;
class UMovieSceneSection;
class UMovieSceneTrack;
class UObject;
struct FAssetData;
struct FBuildEditWidgetParams;
struct FFrameNumber;
struct FGuid;

class FCameraShakeSourceShakeTrackEditor : public FMovieSceneTrackEditor
{
public:
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	FCameraShakeSourceShakeTrackEditor(TSharedRef<ISequencer> InSequencer);
	virtual ~FCameraShakeSourceShakeTrackEditor() = default;

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;

private:
	// Shake section members
	FKeyPropertyResult AddCameraShakeSectionKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, bool bSelect);
	FKeyPropertyResult AddCameraShakeSectionKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> CameraShake, bool bSelect);

	void AddCameraShakeSection(TArray<FGuid> ObjectHandles);

	TSharedRef<SWidget> BuildCameraShakeSubMenu(FGuid ObjectBinding);
	void AddCameraShakeSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);
	void AddOtherCameraShakeBrowserSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);
	void OnCameraShakeAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings);
	void OnCameraShakeAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings);
	void OnAutoCameraShakeSelected(TArray<FGuid> ObjectBindings);
	bool OnShouldFilterCameraShake(const FAssetData& AssetData);

	// Shake trigger members
	FKeyPropertyResult AddCameraShakeTriggerTrackInternal(FFrameNumber Time, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> CameraShake);

	void AddCameraShakeTriggerTrack(const TArray<FGuid> ObjectBindings);
	
	// Utility
	UCameraShakeSourceComponent* AcquireCameraShakeSourceComponentFromGuid(const FGuid& Guid);
	void AddCameraShakeTracksMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);
	TSharedRef<SWidget> BuildCameraShakeTracksMenu(FGuid ObjectBinding);
};

