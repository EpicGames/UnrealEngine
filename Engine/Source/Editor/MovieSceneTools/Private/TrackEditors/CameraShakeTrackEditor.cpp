// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CameraShakeTrackEditor.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraShakeBase.h"
#include "TrackEditors/CameraShakeTrackEditorBase.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneCameraShakeSection.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Tracks/MovieSceneCameraShakeTrack.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class ISequencerSection;
class SWidget;
class USkeleton;

#define LOCTEXT_NAMESPACE "FCameraShakeTrackEditor"

class FCameraShakeSection : public FCameraShakeSectionBase
{
public:
	FCameraShakeSection(const TSharedPtr<ISequencer> InSequencer, UMovieSceneSection& InSection, const FGuid& ObjectBinding)
		: FCameraShakeSectionBase(InSequencer, InSection, ObjectBinding)
	{}

private:
	virtual TSubclassOf<UCameraShakeBase> GetCameraShakeClass() const override
	{
		if (UMovieSceneCameraShakeSection const* const ShakeSection = GetSectionObjectAs<UMovieSceneCameraShakeSection>())
		{
			return ShakeSection->ShakeData.ShakeClass;
		}

		return TSubclassOf<UCameraShakeBase>();
	}
};


FCameraShakeTrackEditor::FCameraShakeTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor( InSequencer ) 
{ 
}


TSharedRef<ISequencerTrackEditor> FCameraShakeTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCameraShakeTrackEditor(InSequencer));
}


bool FCameraShakeTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneCameraShakeTrack::StaticClass();
}


TSharedRef<ISequencerSection> FCameraShakeTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShareable(new FCameraShakeSection(GetSequencer(), SectionObject, ObjectBinding));
}

bool FCameraShakeTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (TargetObjectGuid.IsValid())
	{
		UBlueprint* const SelectedObject = dynamic_cast<UBlueprint*>(Asset);
		if (SelectedObject && SelectedObject->GeneratedClass && SelectedObject->GeneratedClass->IsChildOf(UCameraShakeBase::StaticClass()))
		{
			TSubclassOf<UCameraShakeBase> const ShakeClass = *(SelectedObject->GeneratedClass);

			TArray<TWeakObjectPtr<>> OutObjects;
			for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(TargetObjectGuid))
			{
				OutObjects.Add(Object);
			}

			const FScopedTransaction Transaction(LOCTEXT("AddCameraShake_Transaction", "Add Camera Shake"));
			
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCameraShakeTrackEditor::AddKeyInternal, OutObjects, ShakeClass));

			return true;
		}
	}

	return false;
}

FText FCameraShakeTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CameraShakeTrackEditor_DisplayName", "Camera Shake");
}

void FCameraShakeTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// only offer this track if we can find a camera component
	UCameraComponent const* const CamComponent = AcquireCameraComponentFromObjectGuid(ObjectBindings[0]);
	if (CamComponent)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddCameraShake", "Camera Shake"), NSLOCTEXT("Sequencer", "AddCameraShakeTooltip", "Adds an additive camera shake track."),
			FNewMenuDelegate::CreateRaw(this, &FCameraShakeTrackEditor::AddCameraShakeSubMenu, ObjectBindings)
			);
	}
}

TSharedRef<SWidget> FCameraShakeTrackEditor::BuildCameraShakeSubMenu(FGuid ObjectBinding)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	AddCameraShakeSubMenu(MenuBuilder, ObjectBindings);

	return MenuBuilder.MakeWidget();
}

void FCameraShakeTrackEditor::AddCameraShakeSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FCameraShakeTrackEditor::OnCameraShakeAssetSelected, ObjectBindings);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FCameraShakeTrackEditor::OnCameraShakeAssetEnterPressed, ObjectBindings);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));

		IAssetRegistry & AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		TArray<FTopLevelAssetPath> ClassNames;
		TSet<FTopLevelAssetPath> DerivedClassNames;
		ClassNames.Add(UCameraShakeBase::StaticClass()->GetClassPathName());
		AssetRegistry.GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);
						
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([DerivedClassNames](const FAssetData& AssetData)
		{
			const FString ParentClassFromData = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
			if (!ParentClassFromData.IsEmpty())
			{
				const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(ParentClassFromData));
				if (DerivedClassNames.Contains(ClassObjectPath))
				{
					return false;
				}
			}
			return true;
		});
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}

TSharedPtr<SWidget> FCameraShakeTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return UE::Sequencer::MakeAddButton(LOCTEXT("AddCameraShake", "Camera Shake"), FOnGetContent::CreateSP(this, &FCameraShakeTrackEditor::BuildCameraShakeSubMenu, ObjectBinding), Params.ViewModel);
}


void FCameraShakeTrackEditor::OnCameraShakeAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings)
{
	FSlateApplication::Get().DismissAllMenus();

	UBlueprint* const SelectedObject = dynamic_cast<UBlueprint*>(AssetData.GetAsset());
	if (SelectedObject && SelectedObject->GeneratedClass && SelectedObject->GeneratedClass->IsChildOf(UCameraShakeBase::StaticClass()))
	{
		TSubclassOf<UCameraShakeBase> const ShakeClass = *(SelectedObject->GeneratedClass);

		TArray<TWeakObjectPtr<>> OutObjects;
		for (FGuid ObjectBinding : ObjectBindings)
		{
			for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectBinding))
			{
				OutObjects.Add(Object);
			}
		}
		
		const FScopedTransaction Transaction(LOCTEXT("AddCameraShake_Transaction", "Add Camera Shake"));

		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCameraShakeTrackEditor::AddKeyInternal, OutObjects, ShakeClass));
	}
}


FKeyPropertyResult FCameraShakeTrackEditor::AddKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> ShakeClass)
{
	FKeyPropertyResult KeyPropertyResult;

	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		UObject* Object = Objects[ObjectIndex].Get();

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneCameraShakeTrack::StaticClass());
			UMovieSceneTrack* Track = TrackResult.Track;
			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(Track))
			{
				UMovieSceneSection* NewSection = Cast<UMovieSceneCameraShakeTrack>(Track)->AddNewCameraShake(KeyTime, ShakeClass);
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);
				
				GetSequencer()->EmptySelection();
				GetSequencer()->SelectSection(NewSection);
				GetSequencer()->ThrobSectionSelection();
			}
		}
	}

	return KeyPropertyResult;
}

void FCameraShakeTrackEditor::OnCameraShakeAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings)
{
	if (AssetData.Num() > 0)
	{
		OnCameraShakeAssetSelected(AssetData[0].GetAsset(), ObjectBindings);
	}
}


UCameraComponent* FCameraShakeTrackEditor::AcquireCameraComponentFromObjectGuid(const FGuid& Guid)
{
	USkeleton* Skeleton = nullptr;
	for (TWeakObjectPtr<> WeakObject : GetSequencer()->FindObjectsInCurrentSequence(Guid))
	{
		UObject* const Obj = WeakObject.Get();
		if (AActor* const Actor = Cast<AActor>(Obj))
		{
			UCameraComponent* const CameraComp = MovieSceneHelpers::CameraComponentFromActor(Actor);
			if (CameraComp)
			{
				return CameraComp;
			}
		}
		else if (UCameraComponent* const CameraComp = Cast<UCameraComponent>(Obj))
		{
			if (CameraComp->IsActive())
			{
				return CameraComp;
			}
		}
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
