// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FMenuBuilder;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class SWidget;
class UCameraShakeBase;
class UClass;
class UMovieSceneSection;
class UMovieSceneTrack;
class UObject;
struct FAssetData;
struct FBuildEditWidgetParams;
struct FFrameNumber;
struct FGuid;

/**
* Tools for playing a camera shake
*/
class FCameraShakeTrackEditor : public FMovieSceneTrackEditor
{
public:

	/**
	* Constructor
	*
	* @param InSequencer The sequencer instance to be used by this tool
	*/
	FCameraShakeTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCameraShakeTrackEditor() = default;

	/**
	* Creates an instance of this class.  Called by a sequencer
	*
	* @param OwningSequencer The sequencer instance to be used by this tool
	* @return The new instance of this class
	*/
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;

private:

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, TSubclassOf<UCameraShakeBase> ShakeClass);

	/** Animation sub menu */
	TSharedRef<SWidget> BuildCameraShakeSubMenu(FGuid ObjectBinding);
	void AddCameraShakeSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);

	/** Animation asset selected */
	void OnCameraShakeAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings);

	/** Animation asset enter pressed */
	void OnCameraShakeAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings);

	class UCameraComponent* AcquireCameraComponentFromObjectGuid(const FGuid& Guid);
};


