// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CameraShakeTrackEditorBase.h"

#include "Camera/CameraShakeBase.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "TimeToPixel.h"
#include "UObject/Class.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CameraShakeTrackEditorBase"

FCameraShakeSectionBase::FCameraShakeSectionBase(const TSharedPtr<ISequencer> InSequencer, UMovieSceneSection& InSection, const FGuid& InObjectBindingId)
	: SequencerPtr(InSequencer)
	, SectionPtr(&InSection)
	, ObjectBindingId(InObjectBindingId)
{
}

FText FCameraShakeSectionBase::GetSectionTitle() const
{
	const TSubclassOf<UCameraShakeBase> ShakeClass = GetCameraShakeClass();
	if (const UClass* ShakeClassPtr = ShakeClass.Get())
	{
		FCameraShakeDuration ShakeDuration;
		const bool bHasDuration = UCameraShakeBase::GetCameraShakeDuration(ShakeClass, ShakeDuration);
		if (bHasDuration)
		{
			if (ShakeDuration.IsFixed())
			{
				if (ShakeDuration.Get() > SMALL_NUMBER)
				{
					return FText::FromString(ShakeClassPtr->GetName());
				}
				else
				{
					return FText::Format(LOCTEXT("ShakeHasNoDurationWarning", "{0} (warning: shake has no duration)"), FText::FromString(ShakeClassPtr->GetName()));
				}
			}
			else if (ShakeDuration.IsCustom())
			{
				if (ShakeDuration.Get() > SMALL_NUMBER)
				{
					return FText::FromString(ShakeClassPtr->GetName());
				}
				else
				{
					return FText::Format(LOCTEXT("ShakeHasCustomDurationWarning", "{0} (warning: shake has undefined custom duration)"), FText::FromString(ShakeClassPtr->GetName()));
				}
			}
			else if (ShakeDuration.IsInfinite())
			{
				return FText::FromString(ShakeClassPtr->GetName());
			}
		}
		else
		{
			return FText::Format(LOCTEXT("ShakeIsInvalidWarning", "{0} (warning: shake is invalid)"), FText::FromString(ShakeClassPtr->GetName()));
		}
	}
	return LOCTEXT("NoCameraShake", "No Camera Shake");
}

UMovieSceneSection* FCameraShakeSectionBase::GetSectionObject()
{
	return SectionPtr.Get();
}

TSharedPtr<ISequencer> FCameraShakeSectionBase::GetSequencer() const
{
	return SequencerPtr.Pin();
}

FGuid FCameraShakeSectionBase::GetObjectBinding() const
{
	return ObjectBindingId;
}

bool FCameraShakeSectionBase::IsReadOnly() const
{ 
	return SectionPtr.IsValid() ? SectionPtr.Get()->IsReadOnly() : false; 
}

int32 FCameraShakeSectionBase::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	Painter.LayerId = Painter.PaintSectionBackground();

	const UMovieSceneSection* SectionObject = SectionPtr.Get();
	const TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (SectionObject == nullptr || !Sequencer.IsValid())
	{
		return Painter.LayerId;
	}

	const UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (FocusedSequence == nullptr)
	{
		return Painter.LayerId;
	}

	const FTimeToPixel& TimeConverter = Painter.GetTimeConverter();
	const FFrameRate TickResolution = FocusedSequence->GetMovieScene()->GetTickResolution();

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const TRange<FFrameNumber> SectionRange = SectionObject->GetRange();
	const int32 SectionSize = UE::MovieScene::DiscreteSize(SectionRange);
	const float SectionDuration = FFrameNumber(SectionSize) / TickResolution;

	const float SectionStartTime = SectionObject->GetInclusiveStartFrame() / TickResolution;
	const float SectionStartTimeInPixels = TimeConverter.SecondsToPixel(SectionStartTime);
	const float SectionEndTime = SectionObject->GetExclusiveEndFrame() / TickResolution;
	const float SectionEndTimeInPixels = TimeConverter.SecondsToPixel(SectionEndTime);

	const TSubclassOf<UCameraShakeBase> CameraShakeClass = GetCameraShakeClass();
	const bool bHasValidCameraShake = (CameraShakeClass.Get() != nullptr);
	if (bHasValidCameraShake)
	{
		FCameraShakeDuration ShakeDuration;
		UCameraShakeBase::GetCameraShakeDuration(CameraShakeClass, ShakeDuration);

		const float ShakeEndInPixels = (ShakeDuration.IsFixed() || ShakeDuration.IsCustomWithHint()) ? 
			TimeConverter.SecondsToPixel(FMath::Min(SectionStartTime + ShakeDuration.Get(), SectionEndTime)) :
			SectionEndTimeInPixels;
		const bool bSectionContainsEntireShake = ((ShakeDuration.IsFixed() || ShakeDuration.IsCustomWithHint()) && SectionDuration > ShakeDuration.Get());

		if (ShakeDuration.IsFixed() || ShakeDuration.IsCustomWithHint())
		{
			if (bSectionContainsEntireShake && SectionRange.HasLowerBound())
			{
				// Add some separator where the shake ends.
				float OffsetPixel = ShakeEndInPixels - SectionStartTimeInPixels;

				FSlateDrawElement::MakeBox(
						Painter.DrawElements,
						Painter.LayerId++,
						Painter.SectionGeometry.MakeChild(
							FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
							FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
						).ToPaintGeometry(),
						GenericDivider,
						DrawEffects
				);

				// Draw the rest in a "muted" color.
				const float OverflowSizeInPixels = SectionEndTimeInPixels - ShakeEndInPixels;

				FSlateDrawElement::MakeBox(
						Painter.DrawElements,
						Painter.LayerId++,
						Painter.SectionGeometry.MakeChild(
							FVector2D(OverflowSizeInPixels, Painter.SectionGeometry.Size.Y),
							FSlateLayoutTransform(FVector2D(OffsetPixel, 0))
						).ToPaintGeometry(),
						FAppStyle::GetBrush("WhiteBrush"),
						ESlateDrawEffect::None,
						FLinearColor::Black.CopyWithNewOpacity(0.5f)
				);
			}

			float ShakeBlendIn = 0.f, ShakeBlendOut = 0.f;
			UCameraShakeBase::GetCameraShakeBlendTimes(CameraShakeClass, ShakeBlendIn, ShakeBlendOut);
			{
				// Draw the shake "intensity" as a line that goes up and down according to
				// blend in and out times.
				const FLinearColor LineColor(0.25f, 0.25f, 1.f, 0.75f);

				const bool bHasBlendIn = (ShakeBlendIn > SMALL_NUMBER);
				const bool bHasBlendOut = (ShakeBlendOut > SMALL_NUMBER);

				float ShakeBlendInEndInPixels = TimeConverter.SecondsToPixel(SectionStartTime + ShakeBlendIn);
				float ShakeBlendOutStartInPixels = ShakeEndInPixels - TimeConverter.SecondsDeltaToPixel(ShakeBlendOut);
				if (ShakeBlendInEndInPixels > ShakeBlendOutStartInPixels)
				{
					// If we have to blend out before we're done blending in, let's switch over
					// at the half mark.
					ShakeBlendInEndInPixels = ShakeBlendOutStartInPixels = (ShakeBlendInEndInPixels + ShakeBlendOutStartInPixels) / 2.f;
				}

				TArray<FVector2D> LinePoints;

				if (bHasBlendIn)
				{
					LinePoints.Add(FVector2D(SectionStartTimeInPixels, Painter.SectionGeometry.Size.Y - 2.f));
					LinePoints.Add(FVector2D(ShakeBlendInEndInPixels, 2.f));
				}
				else
				{
					LinePoints.Add(FVector2D(SectionStartTimeInPixels, 2.f));
				}

				if (bHasBlendOut)
				{
					LinePoints.Add(FVector2D(ShakeBlendOutStartInPixels, 2.f));
					LinePoints.Add(FVector2D(ShakeEndInPixels, Painter.SectionGeometry.Size.Y - 2.f));
				}
				else
				{
					LinePoints.Add(FVector2D(ShakeEndInPixels, 2.f));
				}

				FSlateDrawElement::MakeLines(
						Painter.DrawElements,
						Painter.LayerId++,
						Painter.SectionGeometry.ToPaintGeometry(),
						LinePoints,
						DrawEffects,
						LineColor);
			}
		}
		else
		{
			// Draw the shake in a "warning" orange colour.
			const float SectionDurationInPixels = TimeConverter.SecondsDeltaToPixel(SectionDuration);
			FSlateDrawElement::MakeBox(
					Painter.DrawElements,
					Painter.LayerId++,
					Painter.SectionGeometry.MakeChild(
						FVector2D(SectionDurationInPixels, Painter.SectionGeometry.Size.Y),
						FSlateLayoutTransform(FVector2D(SectionStartTimeInPixels, 0))
					).ToPaintGeometry(),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					FLinearColor(1.f, 0.5f, 0.f, 0.5f)
			);
		}
	}
	else
	{
		const float SectionDurationInPixels = TimeConverter.SecondsDeltaToPixel(SectionDuration);
		FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				Painter.LayerId++,
				Painter.SectionGeometry.MakeChild(
					FVector2D(SectionDurationInPixels, Painter.SectionGeometry.Size.Y),
					FSlateLayoutTransform(FVector2D(SectionStartTimeInPixels, 0))
					).ToPaintGeometry(),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Red.CopyWithNewOpacity(0.5f)
				);
	}

	return Painter.LayerId;
}

const UCameraShakeBase* FCameraShakeSectionBase::GetCameraShakeDefaultObject() const
{
	const TSubclassOf<UCameraShakeBase> ShakeClass = GetCameraShakeClass();
	if (const UClass* ShakeClassPtr = ShakeClass.Get())
	{
		return ShakeClassPtr->GetDefaultObject<UCameraShakeBase>();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE

