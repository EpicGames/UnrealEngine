// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ISequencerSection.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FSequencerSectionPainter;
class ISequencer;
class UCameraShakeBase;

/**
 * Section interface for shake sections
 */
class FCameraShakeSectionBase : public ISequencerSection
{
public:
	FCameraShakeSectionBase(const TSharedPtr<ISequencer> InSequencer, UMovieSceneSection& InSection, const FGuid& InObjectBindingId);

	virtual FText GetSectionTitle() const override;
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual bool IsReadOnly() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

protected:
	virtual TSubclassOf<UCameraShakeBase> GetCameraShakeClass() const = 0;

	TSharedPtr<ISequencer> GetSequencer() const;
	FGuid GetObjectBinding() const;

	template<typename SectionClass>
	SectionClass* GetSectionObjectAs() const
	{
		return Cast<SectionClass>(SectionPtr.Get());
	}

private:
	const UCameraShakeBase* GetCameraShakeDefaultObject() const;

private:
	TWeakPtr<ISequencer> SequencerPtr;
	TWeakObjectPtr<UMovieSceneSection> SectionPtr;
	FGuid ObjectBindingId;
};

