// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CinematicShotTrackEditor.h"

#include "AutomatedLevelSequenceCapture.h"
#include "LevelSequence.h"
#include "MovieSceneCaptureModule.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneToolsProjectSettings.h"
#include "Sections/CinematicShotSection.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

#include "Application/ThrottleManager.h"
#include "Editor.h"
#include "FCPXML/FCPXMLMovieSceneTranslator.h"
#include "Factories/Factory.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "LevelEditorViewport.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FCinematicShotTrackEditor"

/* FCinematicShotTrackEditor structors
 *****************************************************************************/

FCinematicShotTrackEditor::FCinematicShotTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FSubTrackEditor(InSequencer) 
{
	ThumbnailPool = MakeShareable(new FTrackEditorThumbnailPool(InSequencer));
}


TSharedRef<ISequencerTrackEditor> FCinematicShotTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FCinematicShotTrackEditor(InSequencer));
}

TWeakObjectPtr<AActor> FCinematicShotTrackEditor::GetCinematicShotCamera() const
{
	const UCameraComponent* Camera = GetSequencer()->GetLastEvaluatedCameraCut().Get();
	return Camera ? Camera->GetOwner() : nullptr;
}


/* ISequencerTrackEditor interface
 *****************************************************************************/

FText FCinematicShotTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ShotTrackEditor_DisplayName", "Shot");
}

TSharedPtr<SWidget> FCinematicShotTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	if (ColumnName == FCommonOutlinerNames::Add)
	{
		return UE::Sequencer::MakeAddButton(
			LOCTEXT("CinematicShotText", "Shot"),
			FOnGetContent::CreateSP(this, &FCinematicShotTrackEditor::HandleAddSubSequenceComboButtonGetMenuContent, Params.ViewModel.AsWeak()),
			Params.ViewModel);
	}

	// Only show camera lock on track-level models, not on rows or layers
	ITrackRowExtension* TrackRowExt = Params.ViewModel->CastThis<ITrackRowExtension>();
	if (!TrackRowExt)
	{
		bool bAddCameraLock = false;
		if (ColumnName == FCommonOutlinerNames::Nav)
		{
			bAddCameraLock = true;
		}
		else if (ColumnName == FCommonOutlinerNames::KeyFrame)
		{
			// Add the camera lock button to the keyframe column if Nav is disabled
			bAddCameraLock = Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false;
		}
		else if (ColumnName == FCommonOutlinerNames::Edit)
		{
			// Add the camera lock button to the edit column if both Nav and KeyFrame are disabled
			bAddCameraLock = Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false &&
				Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame) == false;
		}

		if (bAddCameraLock)
		{
			TSharedRef<SWidget> Button = SNew(SCheckBox)
			.Style(FAppStyle::Get(), "Sequencer.Outliner.ToggleButton")
			.Type(ESlateCheckBoxType::ToggleButton)
			.IsFocusable(false)
			.IsChecked(this, &FCinematicShotTrackEditor::AreShotsLocked)
			.OnCheckStateChanged(this, &FCinematicShotTrackEditor::OnLockShotsClicked)
			.ToolTipText(this, &FCinematicShotTrackEditor::GetLockShotsToolTip)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Sequencer.Outliner.CameraLock"))
			];

			if (ColumnName == FCommonOutlinerNames::Edit)
			{
				// Needs to be left aligned in the edit column because this column slot is set to fill
				return SNew(SBox)
				.HAlign(HAlign_Left)
				.Padding(4.f, 0.f)
				[
					Button
				];
			}
			else
			{
				return Button;
			}
		}
	}

	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

TSharedRef<ISequencerSection> FCinematicShotTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	
	UMovieSceneCinematicShotSection& SectionObjectImpl = *CastChecked<UMovieSceneCinematicShotSection>(&SectionObject);
	return MakeShareable(new FCinematicShotSection(GetSequencer(), SectionObjectImpl, SharedThis(this), ThumbnailPool));
}

bool FCinematicShotTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneCinematicShotTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

void FCinematicShotTrackEditor::Tick(float DeltaTime)
{
	TSharedPtr<ISequencer> SequencerPin = GetSequencer();
	if (!SequencerPin.IsValid())
	{
		return;
	}

	EMovieScenePlayerStatus::Type PlaybackState = SequencerPin->GetPlaybackStatus();

	UObject* PlaybackContext = SequencerPin->GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;

	// Render thumbnails if allow expensive tasks and playback state is not playing or scrubbing and if in a PIE world, the sequence must be bound to it
	if (FSlateThrottleManager::Get().IsAllowingExpensiveTasks()
		&& PlaybackState != EMovieScenePlayerStatus::Playing
		&& PlaybackState != EMovieScenePlayerStatus::Scrubbing
		&& (!bIsInPIEOrSimulate || (bIsInPIEOrSimulate && GEditor->PlayWorld == World)) )
	{
		SequencerPin->EnterSilentMode();

		FFrameTime SavedTime = SequencerPin->GetGlobalTime().Time;

		if (DeltaTime > 0.f && ThumbnailPool->DrawThumbnails())
		{
			SequencerPin->SetGlobalTime(SavedTime);
		}

		SequencerPin->ExitSilentMode();
	}
}

void FCinematicShotTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	MenuBuilder.BeginSection("Import/Export", NSLOCTEXT("Sequencer", "ImportExportMenuSectionName", "Import/Export"));

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT( "Sequencer", "ImportEDL", "Import EDL..." ),
		NSLOCTEXT( "Sequencer", "ImportEDLTooltip", "Import Edit Decision List (EDL) for non-linear editors." ),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateRaw(this, &FCinematicShotTrackEditor::ImportEDL )));

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT( "Sequencer", "ExportEDL", "Export EDL..." ),
		NSLOCTEXT( "Sequencer", "ExportEDLTooltip", "Export Edit Decision List (EDL) for non-linear editors." ),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateRaw(this, &FCinematicShotTrackEditor::ExportEDL )));

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("Sequencer", "ImportFCPXML", "Import Final Cut Pro 7 XML..."),
		NSLOCTEXT("Sequencer", "ImportFCPXMLTooltip", "Import Final Cut Pro 7 XML file for non-linear editors."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FCinematicShotTrackEditor::ImportFCPXML )));

	MenuBuilder.AddMenuEntry(
		NSLOCTEXT("Sequencer", "ExportFCPXML", "Export Final Cut Pro 7 XML..."),
		NSLOCTEXT("Sequencer", "ExportFCPXMLTooltip", "Export Final Cut Pro 7 XML file for non-linear editors."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FCinematicShotTrackEditor::ExportFCPXML )));

	MenuBuilder.EndSection();
}

void FCinematicShotTrackEditor::RenderShots(const TArray<UMovieSceneCinematicShotSection*>& Sections)
{
	GetSequencer()->RenderMovie(Sections);
}

/* FSubTrackEditor
 *****************************************************************************/

FText FCinematicShotTrackEditor::GetSubTrackName() const
{
	return LOCTEXT("ShotTrackName", "Shot Track");
}

FText FCinematicShotTrackEditor::GetSubTrackToolTip() const
{
	return LOCTEXT("ShotTrackToolTip", "A cinematic shot track.");
}

FName FCinematicShotTrackEditor::GetSubTrackBrushName() const
{
	return TEXT("Sequencer.Tracks.CinematicShot");
}

FString FCinematicShotTrackEditor::GetSubSectionDisplayName(const UMovieSceneSubSection* Section) const
{
	return Cast<UMovieSceneCinematicShotSection>(Section)->GetShotDisplayName();
}

FString FCinematicShotTrackEditor::GetDefaultSubsequenceName() const
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	return ProjectSettings->ShotPrefix;
}

FString FCinematicShotTrackEditor::GetDefaultSubsequenceDirectory() const
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	return ProjectSettings->ShotDirectory;
}

TSubclassOf<UMovieSceneSubTrack> FCinematicShotTrackEditor::GetSubTrackClass() const
{
	return UMovieSceneCinematicShotTrack::StaticClass();
}

bool FCinematicShotTrackEditor::HandleAddSubTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->FindTrack<UMovieSceneCinematicShotTrack>() == nullptr));
}

bool FCinematicShotTrackEditor::CanHandleAssetAdded(UMovieSceneSequence* Sequence) const
{
	// Only allow sequences with a camera cut track to be dropped as a shot. Otherwise, it'll be dropped as a subsequence.
	return Sequence->GetMovieScene()->GetCameraCutTrack() != nullptr;
}

ECheckBoxState FCinematicShotTrackEditor::AreShotsLocked() const
{
	if (GetSequencer()->IsPerspectiveViewportCameraCutEnabled())
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}


void FCinematicShotTrackEditor::OnLockShotsClicked(ECheckBoxState CheckBoxState)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	const bool bEnableCameraCuts = (CheckBoxState == ECheckBoxState::Checked);
	SequencerPtr->SetPerspectiveViewportCameraCutEnabled(bEnableCameraCuts);

	bool bNeedsRestoreViewport = true;
	if (const USequencerSettings* SequencerSettings = SequencerPtr->GetSequencerSettings())
	{
		bNeedsRestoreViewport = SequencerSettings->GetRestoreOriginalViewportOnCameraCutUnlock();
	}

	UMovieSceneEntitySystemLinker* Linker = SequencerPtr->GetEvaluationTemplate().GetEntitySystemLinker();
	UMovieSceneCameraCutTrackInstance::ToggleCameraCutLock(Linker, bEnableCameraCuts, bNeedsRestoreViewport);

	SequencerPtr->ForceEvaluate();
}

FText FCinematicShotTrackEditor::GetLockShotsToolTip() const
{
	return AreShotsLocked() == ECheckBoxState::Checked ?
		LOCTEXT("UnlockShots", "Unlock Viewport from Shots") :
		LOCTEXT("LockShots", "Lock Viewport to Shots");
}

UAutomatedLevelSequenceCapture* GetMovieSceneCapture()
{
	UAutomatedLevelSequenceCapture* MovieSceneCapture = Cast<UAutomatedLevelSequenceCapture>(IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture());
	if (!MovieSceneCapture)
	{
		MovieSceneCapture = FindObject<UAutomatedLevelSequenceCapture>(GetTransientPackage(), *UAutomatedLevelSequenceCapture::AutomatedLevelSequenceCaptureUIName.ToString());
	}
	
	if (!MovieSceneCapture)
	{
		MovieSceneCapture = NewObject<UAutomatedLevelSequenceCapture>(GetTransientPackage(), UAutomatedLevelSequenceCapture::StaticClass(), UMovieSceneCapture::MovieSceneCaptureUIName, RF_Transient);
		MovieSceneCapture->LoadFromConfig();
	}

	return MovieSceneCapture;
}

void FCinematicShotTrackEditor::ImportEDL()
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UAutomatedLevelSequenceCapture* MovieSceneCapture = GetMovieSceneCapture();
	if (!MovieSceneCapture)
	{
		return;
	}

	const FMovieSceneCaptureSettings& Settings = MovieSceneCapture->GetSettings();
	FString SaveDirectory = FPaths::ConvertRelativePathToFull(Settings.OutputDirectory.Path);

	if (MovieSceneToolHelpers::ShowImportEDLDialog(MovieScene, MovieScene->GetDisplayRate(), SaveDirectory))
	{
		GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemsChanged );
	}
}


void FCinematicShotTrackEditor::ExportEDL()
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}
		
	UAutomatedLevelSequenceCapture* MovieSceneCapture = GetMovieSceneCapture();
	if (!MovieSceneCapture)
	{
		return;
	}

	const FMovieSceneCaptureSettings& Settings = MovieSceneCapture->GetSettings();
	FString SaveDirectory = FPaths::ConvertRelativePathToFull(Settings.OutputDirectory.Path);
	int32 HandleFrames = Settings.HandleFrames;
	FString MovieExtension = Settings.MovieExtension;

	MovieSceneToolHelpers::ShowExportEDLDialog(MovieScene, MovieScene->GetDisplayRate(), SaveDirectory, HandleFrames, MovieExtension);
}


void FCinematicShotTrackEditor::ImportFCPXML()
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UAutomatedLevelSequenceCapture* MovieSceneCapture = GetMovieSceneCapture();
	if (!MovieSceneCapture)
	{
		return;
	}

	const FMovieSceneCaptureSettings& Settings = MovieSceneCapture->GetSettings();
	FString SaveDirectory = FPaths::ConvertRelativePathToFull(Settings.OutputDirectory.Path);

	FFCPXMLImporter *Importer = new FFCPXMLImporter;

	if (MovieSceneToolHelpers::MovieSceneTranslatorImport(Importer, MovieScene, MovieScene->GetDisplayRate(), SaveDirectory))
	{
		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}

	delete Importer;
}


void FCinematicShotTrackEditor::ExportFCPXML()
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	const UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UAutomatedLevelSequenceCapture* MovieSceneCapture = GetMovieSceneCapture();
	if (!MovieSceneCapture)
	{
		return;
	}

	const FMovieSceneCaptureSettings& Settings = MovieSceneCapture->GetSettings();

	FFCPXMLExporter *Exporter = new FFCPXMLExporter;

	MovieSceneToolHelpers::MovieSceneTranslatorExport(Exporter, MovieScene, Settings);

	delete Exporter;
}




#undef LOCTEXT_NAMESPACE
