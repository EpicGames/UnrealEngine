// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "TrackEditors/DefaultLinkedAnimTrackProvider.h"
#include "EditModes/SkeletalAnimationTrackEditMode.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Animation/AnimSequenceBase.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "SequencerSectionPainter.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SequencerBakingSetupRestore.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "ISectionLayoutBuilder.h"
#include "ISequencer.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"
#include "Animation/MirrorDataTable.h"
#include "Styling/AppStyle.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "MovieSceneTimeHelpers.h"
#include "SequencerTimeSliderController.h"
#include "FrameNumberDisplayFormat.h"
#include "FrameNumberNumericInterface.h"
#include "AnimationBlueprintLibrary.h"
#include "MovieSceneTransformTypes.h"
#include "AnimationEditorUtils.h"
#include "Factories/PoseAssetFactory.h"
#include "Misc/MessageDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "SequencerSettings.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/Blueprint.h"

#include "Subsystems/AssetEditorSubsystem.h"

#include "Baker/SequencerBakerSubsystem.h"
#include "Baker/AnimSequenceBaker.h"
#include "Editor.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"

#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "PropertyEditorModule.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "IPropertyUtilities.h"
#include "Factories/AnimSequenceFactory.h"
#include "MovieSceneToolHelpers.h"

#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Images/SImage.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/Selection/Selection.h"
#include "Exporters/AnimSeqExportOption.h"

#include "EditModes/SkeletalAnimationTrackEditMode.h"
#include "EditorModeManager.h"

#include "LevelSequence.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "AnimSequenceLevelSequenceLink.h"
#include "UObject/SavePackage.h"
#include "AnimSequencerInstanceProxy.h"
#include "IDetailGroup.h"
#include "TimeToPixel.h"
#include "SequencerUtilities.h"
#include "SequencerAnimationOverride.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Features/IModularFeatures.h"
#include "IMovieSceneAnimSequenceBakeScope.h"
#include "Misc/ScopeExit.h"

#define LOCTEXT_NAMESPACE "FCommonAnimationTrackEditor"

namespace UE::Sequencer
{

int32 FCommonAnimationTrackEditor::NumberActive = 0;

static FDefaultLinkedAnimTrackProvider GDefaultLinkedAnimTrackProvider;

namespace CommonAnimationEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 AnimationTrackHeight = 28;
}


class SAnimSequenceOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimSequenceOptionsWindow)
		: _ExportOptions(nullptr)
		, _WidgetWindow()
		, _FullPath()
	{}

	SLATE_ARGUMENT(UAnimSeqExportOption*, ExportOptions)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)
		SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnExport()
	{
		bShouldExport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}


	FReply OnCancel()
	{
		bShouldExport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldExport() const
	{
		return bShouldExport;
	}

	
	SAnimSequenceOptionsWindow()
		: ExportOptions(nullptr)
		, bShouldExport(false)
	{}

private:

	FReply OnResetToDefaultClick() const;

private:
	UAnimSeqExportOption* ExportOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr< SWindow > WidgetWindow;
	bool			bShouldExport;
};


void SAnimSequenceOptionsWindow::Construct(const FArguments& InArgs)
{
	ExportOptions = InArgs._ExportOptions;
	WidgetWindow = InArgs._WidgetWindow;

	check(ExportOptions);

	FText CancelText =  LOCTEXT("AnimSequenceOptions_Cancel", "Cancel");
	FText CancelTooltipText = LOCTEXT("AnimSequenceOptions_Cancel_ToolTip", "Cancel the current Anim Sequence Creation.");

	TSharedPtr<SBox> HeaderToolBox;
	TSharedPtr<SHorizontalBox> AnimHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(HeaderToolBox, SBox)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
		.Text(LOCTEXT("Export_CurrentFileTitle", "Current File: "))
		]
	+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
		.Text(InArgs._FullPath)
		]
		]
		]
	+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2)
		[
			SAssignNew(InspectorBox, SBox)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
	   + SUniformGridPanel::Slot(1, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("AnimExportOptionsWindow_Export", "Export"))
			.OnClicked(this, &SAnimSequenceOptionsWindow::OnExport)
		]
	   + SUniformGridPanel::Slot(2, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.Text(CancelText)
		.ToolTipText(CancelTooltipText)
		.OnClicked(this, &SAnimSequenceOptionsWindow::OnCancel)
		]
		]
			]
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());

	HeaderToolBox->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
		[
			SAssignNew(AnimHeaderButtons, SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SButton)
			.Text(LOCTEXT("AnimSequenceOptions_ResetOptions", "Reset to Default"))
		.OnClicked(this, &SAnimSequenceOptionsWindow::OnResetToDefaultClick)
		]
		]
		]
		]
	);

	DetailsView->SetObject(ExportOptions);
}

FReply SAnimSequenceOptionsWindow::OnResetToDefaultClick() const
{
	ExportOptions->ResetToDefault();
	//Refresh the view to make sure the custom UI are updating correctly
	DetailsView->SetObject(ExportOptions, true);
	return FReply::Handled();
}




USkeleton* GetSkeletonFromComponent(UActorComponent* InComponent)
{
	USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
	{
		// @todo Multiple actors, multiple components
		return SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
	}

	return nullptr;
}

// Get the skeletal mesh components from the guid
// If bGetSingleRootComponent - return only the root component if it is a skeletal mesh component. 
// This allows the root object binding to have an animation track without needing a skeletal mesh component binding
//
TArray<USkeletalMeshComponent*> AcquireSkeletalMeshComponentsFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr, const bool bGetSingleRootComponent = true)
{
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;

	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	AActor* Actor = Cast<AActor>(BoundObject);

	if (!Actor)
	{
		if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(BoundObject))
		{
			Actor = ChildActorComponent->GetChildActor();
		}
	}

	if (Actor)
	{
		if (bGetSingleRootComponent)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
			{
				SkeletalMeshComponents.Add(SkeletalMeshComponent);
				return SkeletalMeshComponents;
			}
		}

		Actor->GetComponents(SkeletalMeshComponents);
		if (SkeletalMeshComponents.Num())
		{
			return SkeletalMeshComponents;
		}

		AActor* ActorCDO = Cast<AActor>(Actor->GetClass()->GetDefaultObject());
		if (ActorCDO)
		{
			if (bGetSingleRootComponent)
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ActorCDO->GetRootComponent()))
				{
					SkeletalMeshComponents.Add(SkeletalMeshComponent);
					return SkeletalMeshComponents;
				}
			}

			ActorCDO->GetComponents(SkeletalMeshComponents);
			if (SkeletalMeshComponents.Num())
			{
				return SkeletalMeshComponents;
			}
		}

		UBlueprintGeneratedClass* ActorBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Actor->GetClass());
		if (ActorBlueprintGeneratedClass && ActorBlueprintGeneratedClass->SimpleConstructionScript)
		{
			const TArray<USCS_Node*>& ActorBlueprintNodes = ActorBlueprintGeneratedClass->SimpleConstructionScript->GetAllNodes();

			for (USCS_Node* Node : ActorBlueprintNodes)
			{
				if (Node->ComponentClass->IsChildOf(USkeletalMeshComponent::StaticClass()))
				{
					if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Node->GetActualComponentTemplate(ActorBlueprintGeneratedClass)))
					{
						SkeletalMeshComponents.Add(SkeletalMeshComponent);
					}
				}
			}

			if (SkeletalMeshComponents.Num())
			{
				return SkeletalMeshComponents;
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		SkeletalMeshComponents.Add(SkeletalMeshComponent);
		return SkeletalMeshComponents;
	}
	
	return SkeletalMeshComponents;
}


class FMovieSceneSkeletalAnimationParamsDetailCustomization : public IPropertyTypeCustomization
{
public:
	FMovieSceneSkeletalAnimationParamsDetailCustomization(const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams)
		: Params(InParams)
	{
		if (Params.ParentObjectBindingGuid.IsValid())
		{
			if (USkeletalMeshComponent* SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(Params.ParentObjectBindingGuid, Params.SequencerWeak.Pin()))
			{
				TScriptInterface<ISequencerAnimationOverride> SequencerAnimOverride = ISequencerAnimationOverride::GetSequencerAnimOverride(SkelMeshComp);
				if (SequencerAnimOverride.GetObject())
				{
					bAllowsCinematicOverride = ISequencerAnimationOverride::Execute_AllowsCinematicOverride(SequencerAnimOverride.GetObject());
					SlotNameOptions = ISequencerAnimationOverride::Execute_GetSequencerAnimSlotNames(SequencerAnimOverride.GetObject());
					bShowSlotNameOptions = SlotNameOptions.Num() > 0 && !bAllowsCinematicOverride;
				}
			}
		}
	}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		SlotNameProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneSkeletalAnimationParams, SlotName));
	}

	IDetailPropertyRow& AddPropertyRow(IDetailChildrenBuilder& ChildBuilder, const FString& GroupName, TMap<FString, IDetailGroup*>& NameToGroupMap, const TSharedPtr<IPropertyHandle>& ChildPropertyHandle)
	{
		if (GroupName.IsEmpty())
		{
			return ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
		
		IDetailGroup* const* DetailGroup = NameToGroupMap.Find(GroupName);
		if (!DetailGroup)
		{
			IDetailGroup& NewGroup = ChildBuilder.AddGroup(*GroupName, FText::FromString(GroupName));
			DetailGroup = &NameToGroupMap.Add(GroupName, &NewGroup);
		}
		return (*DetailGroup)->AddPropertyRow(ChildPropertyHandle.ToSharedRef());
	
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		const FName AnimationPropertyName = GET_MEMBER_NAME_CHECKED(FMovieSceneSkeletalAnimationParams, Animation);
		const FName MirrorDataTableName = GET_MEMBER_NAME_CHECKED(FMovieSceneSkeletalAnimationParams, MirrorDataTable);
		const FName SlotNamePropertyName = GET_MEMBER_NAME_CHECKED(FMovieSceneSkeletalAnimationParams, SlotName);

		TMap<FString, IDetailGroup*> NameToGroupMap;

		uint32 NumChildren;
		PropertyHandle->GetNumChildren(NumChildren);
		for (uint32 i = 0; i < NumChildren; ++i)
		{
			TSharedPtr<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(i);
			FName ChildPropertyName = ChildPropertyHandle->GetProperty()->GetFName();
			
			FString GroupName;
			FString CategoryName;
			ChildPropertyHandle->GetDefaultCategoryName().ToString().Split("|", &CategoryName, &GroupName);

			IDetailPropertyRow& ChildPropertyRow = AddPropertyRow(ChildBuilder, GroupName, NameToGroupMap, ChildPropertyHandle);
			
			// Let most properties be whatever they want to be... we just want to customize the `Animation` and `MirrorDataTable` properties
			// by making it look like a normal asset reference property, but with some custom filtering.
			if (ChildPropertyName == AnimationPropertyName || ChildPropertyName == MirrorDataTableName)
			{
				FDetailWidgetRow& Row = ChildPropertyRow.CustomWidget();

				if (Params.ParentObjectBindingGuid.IsValid())
				{
					// Store the compatible skeleton's name, and create a property widget with a filter that will check
					// for animations that match that skeleton.
					Skeleton = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(Params.ParentObjectBindingGuid, Params.SequencerWeak.Pin());
					SkeletonName = FAssetData(Skeleton).GetExportTextName();

					TSharedPtr<IPropertyUtilities> PropertyUtilities = CustomizationUtils.GetPropertyUtilities();
					UClass* AllowedStaticClass = ChildPropertyName == AnimationPropertyName ? UAnimSequenceBase::StaticClass() : UMirrorDataTable::StaticClass(); 

					TSharedRef<SObjectPropertyEntryBox> ContentWidget = SNew(SObjectPropertyEntryBox)
						.PropertyHandle(ChildPropertyHandle)
						.AllowedClass(AllowedStaticClass)
						.DisplayThumbnail(true)
						.ThumbnailPool(PropertyUtilities.IsValid() ? PropertyUtilities->GetThumbnailPool() : nullptr)
						.OnShouldFilterAsset(FOnShouldFilterAsset::CreateRaw(this, &FMovieSceneSkeletalAnimationParamsDetailCustomization::ShouldFilterAsset));

					Row.NameContent()[ChildPropertyHandle->CreatePropertyNameWidget()];
					Row.ValueContent()[ContentWidget];

					float MinDesiredWidth, MaxDesiredWidth;
					ContentWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
					Row.ValueContent().MinWidth = MinDesiredWidth;
					Row.ValueContent().MaxWidth = MaxDesiredWidth;

					// The content widget already contains a "reset to default" button, so we don't want the details view row
					// to make another one. We add this metadata on the property handle instance to suppress it.
					ChildPropertyHandle->SetInstanceMetaData(TEXT("NoResetToDefault"), TEXT("true"));
				}
			}
			else if (ChildPropertyName == SlotNamePropertyName)
			{
				// If the anim instance implements the ISequencerAnimationOverride interface, and has defined slots to used, override this row with a drop-down menu
				// Otherwise the default row will be created, which uses a text input field.
				if (bShowSlotNameOptions)
				{
					ChildPropertyRow.IsEnabled(TAttribute<bool>::CreateSP(this, &FMovieSceneSkeletalAnimationParamsDetailCustomization::GetCanEditSlotName));
					FDetailWidgetRow& Row = ChildPropertyRow.CustomWidget();
					Row.NameContent()[ChildPropertyHandle->CreatePropertyNameWidget()];

					Row.ValueContent()
					[
						SNew(SComboBox<FName>)
						.OptionsSource(&SlotNameOptions)
						.OnSelectionChanged(this, &FMovieSceneSkeletalAnimationParamsDetailCustomization::OnSlotNameChanged)
						.OnGenerateWidget_Lambda([](FName InSlotName) { return SNew(STextBlock).Text(FText::FromName(InSlotName)); })
						[
							SNew(STextBlock)
							.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
							.Text(this, &FMovieSceneSkeletalAnimationParamsDetailCustomization::GetSlotNameDesc)
						]
					];
				}
			}
		}
	}

	bool ShouldFilterAsset(const FAssetData& AssetData)
	{
		// Since the `SObjectPropertyEntryBox` doesn't support passing some `Filter` properties for the asset picker, 
		// we just combine the tag value filtering we want (i.e. checking the skeleton compatibility) along with the
		// other filtering we already get from the track editor's filter callback.
		FCommonAnimationTrackEditor& TrackEditor = static_cast<FCommonAnimationTrackEditor&>(Params.TrackEditor);
		if (TrackEditor.ShouldFilterAsset(AssetData))
		{
			return true;
		}

		return !(Skeleton && Skeleton->IsCompatibleForEditor(AssetData));
	}


	FText GetSlotNameDesc() const
	{
		FName NameValue;
		SlotNameProperty->GetValue(NameValue);

		return FText::FromString(NameValue.ToString());
	}

	bool GetCanEditSlotName() const
	{
		if (bShowSlotNameOptions)
		{
			FName NameValue;
			SlotNameProperty->GetValue(NameValue);
			// If we're allowing cinematic override, then the slot names are irrelevant, don't allow edit.
			// If we have less than 2 slot name options, then changing them is irrelevant, don't allow edit.
			// Always allow an edit if the current slot name isn't currently set to one of the provided ones.
			if (bAllowsCinematicOverride || (SlotNameOptions.Num() < 2 && SlotNameOptions.Contains(NameValue)))
			{
				return false;
			}
		}
		return true;
	}

	void OnSlotNameChanged(FName InSlotName, ESelectInfo::Type InInfo)
	{
		SlotNameProperty->SetValue(InSlotName);
	}

private:
	FSequencerSectionPropertyDetailsViewCustomizationParams Params;
	FString SkeletonName;
	USkeleton* Skeleton = nullptr;
	TSharedPtr<IPropertyHandle> SlotNameProperty;
	TArray<FName> SlotNameOptions;
	bool bShowSlotName = true;
	bool bShowSlotNameOptions = false;
	bool bAllowsCinematicOverride = false;
};


FCommonAnimationSection::FCommonAnimationSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: WeakSection(CastChecked<UMovieSceneSkeletalAnimationSection>(&InSection))
	, Sequencer(InSequencer)
	, PreDilatePlayRate(1.0)
{
}

FCommonAnimationSection::~FCommonAnimationSection()
{
}

void FCommonAnimationSection::BeginDilateSection()
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		PreDilatePlayRate = Section->Params.PlayRate.AsFixedPlayRate(); //make sure to cache the play rate
	}
	else if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		FMovieSceneTimeWarpChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneTimeWarpChannel>(0);
		if (Channel)
		{
			Section->Params.PlayRate.AsCustom()->Modify();
			PreDilateChannel = MakeUnique<FMovieSceneTimeWarpChannel>(*Channel);
		}
	}
}

void FCommonAnimationSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		Section->Params.PlayRate.Set(PreDilatePlayRate / DilationFactor);
	}
	else if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		FMovieSceneTimeWarpChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneTimeWarpChannel>(0);
		if (Channel)
		{
			*Channel = *PreDilateChannel;

			// Dilate the times
			Dilate(Channel, FFrameNumber(0), DilationFactor);

			Section->Params.PlayRate.AsCustom()->MarkAsChanged();
		}
	}
	Section->SetRange(NewRange);
}

UMovieSceneSection* FCommonAnimationSection::GetSectionObject()
{ 
	return WeakSection.Get();
}

FText FCommonAnimationSection::GetSectionTitle() const
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return FText();
	}

	if (Section->Params.Animation != nullptr)
	{
		if (!Section->Params.MirrorDataTable)
		{
			return FText::FromString( Section->Params.Animation->GetName() );
		}
		else
		{
			return FText::Format(LOCTEXT("SectionTitleContentFormat", "{0} mirrored with {1}"), FText::FromString(Section->Params.Animation->GetName()), FText::FromString(Section->Params.MirrorDataTable->GetName()));
		}
	}
	return LOCTEXT("NoAnimationSection", "No Animation");
}

FText FCommonAnimationSection::GetSectionToolTip() const
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (Section && Section->Params.Animation != nullptr && Section->HasStartFrame() && Section->HasEndFrame())
	{
		UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		FMovieSceneSequenceTransform Transform = Section->Params.MakeTransform(TickResolution, Section->GetRange());

		const double StartTime     = Transform.TransformTime(0).AsDecimal();
		const float  SectionLength = Section->GetRange().Size<FFrameTime>() / TickResolution;

		if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
		{
			const double PlayRate = Section->Params.PlayRate.AsFixedPlayRate();
			return FText::Format(LOCTEXT("ToolTipContentFormat_FixedPlayRate", "Start: {0}s\nDuration: {1}s\nPlay Rate: {2}x"), StartTime, SectionLength, PlayRate);
		}
		else if (Section->Params.PlayRate.GetType() == EMovieSceneTimeWarpType::Custom)
		{
			return FText::Format(LOCTEXT("ToolTipContentFormat_TimwWarp", "Start: {0}s\nDuration: {1}s\nPlay Rate: Variable"), StartTime, SectionLength);
		}
	}
	return FText::GetEmpty();
}

TOptional<FFrameTime> FCommonAnimationSection::GetSectionTime(FSequencerSectionPainter& InPainter) const
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();

	if (!Section || !InPainter.bIsSelected || !Sequencer.Pin() || Section->Params.Animation == nullptr)
	{
		return TOptional<FFrameTime>();
	}

	FFrameTime CurrentTime = Sequencer.Pin()->GetLocalTime().Time;
	if (!Section->GetRange().Contains(CurrentTime.FrameNumber))
	{
		return TOptional<FFrameTime>();
	}

	const FTimeToPixel& TimeToPixelConverter = InPainter.GetTimeConverter();
	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Draw the current time next to the scrub handle
	const double AnimTime = Section->MapTimeToAnimation(CurrentTime, TickResolution);
	const FFrameRate SamplingFrameRate = Section->Params.Animation->GetSamplingFrameRate();

	FQualifiedFrameTime HintFrameTime;
	if (!UAnimationBlueprintLibrary::EvaluateRootBoneTimecodeAttributesAtTime(Section->Params.Animation, static_cast<float>(AnimTime), HintFrameTime))
	{
		const FFrameTime FrameTime = SamplingFrameRate.AsFrameTime(AnimTime);
		HintFrameTime = FQualifiedFrameTime(FrameTime, SamplingFrameRate);
	}

	// Convert to tick resolution
	HintFrameTime = FQualifiedFrameTime(ConvertFrameTime(HintFrameTime.Time, HintFrameTime.Rate, TickResolution), TickResolution);

	// Get the desired frame display format and zero padding from
	// the sequencer settings, if possible.
	TAttribute<EFrameNumberDisplayFormats> DisplayFormatAttr(EFrameNumberDisplayFormats::Frames);
	TAttribute<uint8> ZeroPadFrameNumbersAttr(0u);
	if (const USequencerSettings* SequencerSettings = Sequencer.Pin()->GetSequencerSettings())
	{
		DisplayFormatAttr.Set(SequencerSettings->GetTimeDisplayFormat());
		ZeroPadFrameNumbersAttr.Set(SequencerSettings->GetZeroPadFrames());
	}

	// No frame rate conversion necessary since we're displaying
	// the source frame time/rate.
	const TAttribute<FFrameRate> TickResolutionAttr(HintFrameTime.Rate);
	const TAttribute<FFrameRate> DisplayRateAttr(HintFrameTime.Rate);

	FFrameNumberInterface FrameNumberInterface(DisplayFormatAttr, ZeroPadFrameNumbersAttr, TickResolutionAttr, DisplayRateAttr);

	float Subframe = 0.0f;
	if (UAnimationBlueprintLibrary::EvaluateRootBoneTimecodeSubframeAttributeAtTime(Section->Params.Animation, static_cast<float>(AnimTime), Subframe))
	{
		if (FMath::IsNearlyEqual(Subframe, FMath::RoundToFloat(Subframe)))
		{
			FrameNumberInterface.SetSubframeIndicator(FString::Printf(TEXT(" (%d)"), FMath::RoundToInt(Subframe)));
		}
		else
		{
			FrameNumberInterface.SetSubframeIndicator(FString::Printf(TEXT(" (%s)"), *LexToSanitizedString(Subframe)));
		}
	}

	return HintFrameTime.Time;
}


float FCommonAnimationSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(CommonAnimationEditorConstants::AnimationTrackHeight);
}


FMargin FCommonAnimationSection::GetContentPadding() const
{
	return FMargin(8.0f, 8.0f);
}


int32 FCommonAnimationSection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	using namespace UE::MovieScene;

	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return Painter.LayerId;
	}

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	if (!Section->HasStartFrame() || !Section->HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	const FFrameNumber StartFrame = Section->GetInclusiveStartFrame();
	const FFrameNumber EndFrame   = Section->GetExclusiveEndFrame();

	if (UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>())
	{
		FFrameRate FrameRate = MovieScene->GetTickResolution();
		FMovieSceneSequenceTransform OuterToInnerTransform = Section->Params.MakeTransform(FrameRate, Section->GetRange());

		// As seconds represented as a FFrameTime
		FFrameTime LoopStart = FFrameTime::FromDecimal(Section->Params.StartFrameOffset / FrameRate);

		FLinearColor SectionTint = Painter.GetSectionColor().LinearRGBToHSV();
		SectionTint.B *= 0.1f;
		SectionTint = SectionTint.HSVToLinearRGB();

		auto PaintTime = [&](FFrameTime Time)
		{
			float OffsetPixel = TimeToPixelConverter.FrameToPixel(Time);

			TArray<FVector2f> NewVector;
			NewVector.Reserve(2);

			NewVector.Add(FVector2f(OffsetPixel, 1.f));
			NewVector.Add(FVector2f(OffsetPixel, Painter.SectionGeometry.Size.Y-2.f));

			constexpr float Thickness = 1.f;
			constexpr float DashLengthPx = 3.f;
			FSlateDrawElement::MakeDashedLines(
				Painter.DrawElements,
				Painter.LayerId++,
				Painter.SectionGeometry.ToPaintGeometry(),
				MoveTemp(NewVector),
				DrawEffects,
				SectionTint,
				Thickness,
				DashLengthPx
			);
			return true;
		};

		OuterToInnerTransform.ExtractBoundariesWithinRange(StartFrame, EndFrame, [&PaintTime](FFrameTime StartTime){
			PaintTime(StartTime);
			return true;
		});
	}

	return LayerId;
}

void FCommonAnimationSection::BeginResizeSection()
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	const bool bClampToOuterRange = false; // When resizing, we want to be able to go outside of the section range.
	const bool bForceLoop = true; // Similarly, when resizing, we want to be able to start looping even if the section isn't looping right now.
	const FFrameRate FrameRate = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
	InitialDragTransform = MakeUnique<FMovieSceneSequenceTransform>(Section->Params.MakeTransform(FrameRate, Section->GetRange(), nullptr, bClampToOuterRange, bForceLoop));
}

void FCommonAnimationSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();

	if (Section && ResizeMode == SSRM_LeadingEdge && Section->Params.PlayRate.GetType() != EMovieSceneTimeWarpType::Custom)
	{
		const FFrameRate FrameRate = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		Section->Params.FirstLoopStartFrameOffset = (InitialDragTransform->TransformTime(ResizeTime).AsDecimal() * FrameRate).RoundToFrame();
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FCommonAnimationSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FCommonAnimationSection::SlipSection(FFrameNumber SlipTime)
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (Section)
	{
		const FFrameRate FrameRate = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		Section->Params.FirstLoopStartFrameOffset = (InitialDragTransform->TransformTime(SlipTime).AsDecimal() * FrameRate).RoundToFrame();
	}
	ISequencerSection::SlipSection(SlipTime);
}

bool FCommonAnimationSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath)
{
	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (Section)
	{
		Section->Modify();
		Section->DeleteChannels(KeyAreaNamePath);
	}
	return true;
}

void FCommonAnimationSection::CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const
{
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(
		TEXT("MovieSceneSkeletalAnimationParams"),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]() { return MakeShared<FMovieSceneSkeletalAnimationParamsDetailCustomization>(InParams); }));
}

void FCommonAnimationSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return;
	}

	UMovieSceneSkeletalAnimationSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	UMovieSceneCommonAnimationTrack* const Track = Section->GetTypedOuter<UMovieSceneCommonAnimationTrack>();
	if (!IsValid(Track))
	{
		return;
	}

	// Can't pick the object that this track binds
	USkeleton* const Skeleton = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(ObjectBinding, SequencerPtr);
	if (!IsValid(Skeleton))
	{
		return;
	}

	const int32 NumBones = Skeleton->GetReferenceSkeleton().GetNum();
	TArray<FName> BoneNames;
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		BoneNames.Add(Skeleton->GetReferenceSkeleton().GetBoneName(BoneIndex));
	}

	auto MatchToBone = [this, ObjectBinding, Skeleton, Section](bool bMatchPrevious, int32 Index)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, ObjectBinding, Skeleton, bMatchPrevious, Index, Section]
				{
					const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
					if (!SequencerPtr.IsValid())
					{
						return;
					}

					UMovieSceneCommonAnimationTrack* const Track = Section->GetTypedOuter<UMovieSceneCommonAnimationTrack>();
					if (!IsValid(Track))
					{
						return;
					}

					FScopedTransaction MatchSection(LOCTEXT("MatchSectionByBone_Transaction", "Match Section By Bone"));
					Section->Modify();	
					Section->bMatchWithPrevious = bMatchPrevious;
					if (Index >= 0)
					{
						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FName Name = Skeleton->GetReferenceSkeleton().GetBoneName(Index);
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Name);
					}
					else
					{
						Section->ClearMatchedOffsetTransforms();
					}
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);

				}
			),
			FCanExecuteAction::CreateLambda([this]() -> bool 
				{ 
					return Sequencer.IsValid(); 
				}),
			FIsActionChecked::CreateLambda([this, ObjectBinding, Index, Section]() -> bool
				{
					const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
					if (!SequencerPtr.IsValid())
					{
						return false;
					}

					USkeleton* const Skeleton = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(ObjectBinding, SequencerPtr);
					if (!IsValid(Skeleton))
					{
						return false;
					}

					if (Index >= 0)
					{
						FName Name = Skeleton->GetReferenceSkeleton().GetBoneName(Index);
						return (Section->MatchedBoneName == Name);
					}

					return (Section->MatchedBoneName == NAME_None);
				})
			);
	};

	if (!ShouldShowLegacyBoneMatching())
	{
		return;
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("MotionBlendingOptions", "Motion Blending Options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("MatchWithThisBoneInPreviousClip", "Match With This Bone In Previous Clip"), LOCTEXT("MatchWithThisBoneInPreviousClip_Tooltip", "Match This Bone With Previous Clip At Current Frame"),
			FNewMenuDelegate::CreateLambda([MatchToBone, BoneNames](FMenuBuilder& SubMenuBuilder)
				{
					int32 Index = -1;
					FText NoNameText = LOCTEXT("TurnOffBoneMatching", "Turn Off Matching");
					FText NoNameTooltipText = LOCTEXT("TurnOffMatchingTooltip", "Turn Off Any Bone Matching");
					SubMenuBuilder.AddMenuEntry(
						NoNameText, NoNameTooltipText,
						FSlateIcon(), MatchToBone(true, Index++), NAME_None, EUserInterfaceActionType::RadioButton);

					for (const FName& BoneName : BoneNames)
					{
						FText Name = FText::FromName(BoneName);
						FText Text = FText::Format(LOCTEXT("BoneNameSelect", "{0}"), Name);
						FText TooltipText = FText::Format(LOCTEXT("BoneNameSelectTooltip", "Match To This Bone {0}"), Name);
						SubMenuBuilder.AddMenuEntry(
							Text, TooltipText,
							FSlateIcon(), MatchToBone(true, Index++), NAME_None, EUserInterfaceActionType::RadioButton);
					}
				}));

		MenuBuilder.AddSubMenu(
			LOCTEXT("MatchWithThisBoneInNextClip", "Match With This Bone In Next Clip"), LOCTEXT("MatchWithThisBoneInNextClip_Tooltip", "Match This Bone With Next Clip At Current Frame"),
			FNewMenuDelegate::CreateLambda([MatchToBone, BoneNames](FMenuBuilder& SubMenuBuilder)
				{
					int32 Index = -1;
					
					FText NoNameText = LOCTEXT("TurnOffBoneMatching", "Turn Off Matching");
					FText NoNameTooltipText = LOCTEXT("TurnOffMatchingTooltip", "Turn Off Any Bone Matching");
					SubMenuBuilder.AddMenuEntry(
						NoNameText, NoNameTooltipText,
						FSlateIcon(), MatchToBone(false, Index++), NAME_None, EUserInterfaceActionType::RadioButton);

					for (const FName& BoneName : BoneNames)
					{
						FText Name = FText::FromName(BoneName);
						FText Text = FText::Format(LOCTEXT("BoneNameSelect", "{0}"), Name);
						FText TooltipText = FText::Format(LOCTEXT("BoneNameSelectTooltip", "Match To This Bone {0}"), Name);
						SubMenuBuilder.AddMenuEntry(
							Text, TooltipText,
							FSlateIcon(), MatchToBone(false, Index++), NAME_None, EUserInterfaceActionType::RadioButton);
					}
				}));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MatchTranslation", "Match X and Y Translation"),
			LOCTEXT("MatchTranslationTooltip", "Match the Translation to the Specified Bone"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ObjectBinding, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FScopedTransaction MatchTransaction(LOCTEXT("MatchTranslation_Transaction", "Match Translation"));
						Section->Modify();	
						Section->ToggleMatchTranslation();
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Section->MatchedBoneName);
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}),
				FCanExecuteAction::CreateLambda([]() -> bool
					{
						return true;
					}),
				FIsActionChecked::CreateLambda([Section]() -> bool
					{
						return Section->bMatchTranslation;
					})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MatchZHeight", "Match Z Height"),
			LOCTEXT("MatchZHeightTooltip", "Match the Z Height, may want this off for better matching"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ObjectBinding, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FScopedTransaction MatchTransaction(LOCTEXT("MatchZHeight_Transaction", "Match Z Height"));
						Section->Modify();
						Section->ToggleMatchIncludeZHeight(); 
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Section->MatchedBoneName);
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}),
					FCanExecuteAction::CreateLambda([]() -> bool
						{
							return true;
						}),
					FIsActionChecked::CreateLambda([Section]() -> bool
						{
							return Section->bMatchIncludeZHeight;
						})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MatchYawRotation", "Match Yaw Rotation"),
			LOCTEXT("MatchYawRotationTooltip", "Match the Yaw Rotation, may want this off for better matching"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ObjectBinding, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FScopedTransaction MatchTransaction(LOCTEXT("MatchYawRotation_Transaction", "Match Yaw Rotation"));
						Section->Modify();
						Section->ToggleMatchIncludeYawRotation();
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Section->MatchedBoneName);
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged); }),
					FCanExecuteAction::CreateLambda([]() -> bool
						{
							return true;
						}),
					FIsActionChecked::CreateLambda([Section]() -> bool
						{
							return Section->bMatchRotationYaw;
						})),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MatchPitchRotation", "Match Pitch Rotation"),
			LOCTEXT("MatchPitchRotationTooltip", "Match the Pitch Rotation, may want this off for better matching"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ObjectBinding, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FScopedTransaction MatchTransaction(LOCTEXT("MatchPitchRotation_Transaction", "Match Pitch Rotation"));
						Section->Modify();
						Section->ToggleMatchIncludePitchRotation();
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Section->MatchedBoneName);
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}),
					FCanExecuteAction::CreateLambda([]() -> bool
						{
							return true;
						}),
					FIsActionChecked::CreateLambda([Section]() -> bool
						{
							return Section->bMatchRotationPitch;
						})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MatchRollRotation", "Match Roll Rotation"),
			LOCTEXT("MatchRollRotationTooltip", "Match the Roll Rotation, may want this off for better matching"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ObjectBinding, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						USkeletalMeshComponent* const SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SequencerPtr);

						FScopedTransaction MatchTransaction(LOCTEXT("MatchRollRotation_Transaction", "Match Roll Rotation"));
						Section->Modify();
						Section->ToggleMatchIncludeRollRotation();
						Section->MatchSectionByBoneTransform(SkelMeshComp, SequencerPtr->GetLocalTime().Time, SequencerPtr->GetLocalTime().Rate, Section->MatchedBoneName);
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}),
				FCanExecuteAction::CreateLambda([]() -> bool
					{
						return true;
					}),
				FIsActionChecked::CreateLambda([Section]() -> bool
					{
						return Section->bMatchRotationRoll;
					})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("SkelAnimSectionDisplay", "Display"));
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "ShowSkeletons", "Show Skeleton"),
			NSLOCTEXT("Sequencer", "ShowSkeletonsTooltip", "Show A Skeleton for this Section."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, Section]()
					{
						const TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
						if (!SequencerPtr.IsValid())
						{
							return;
						}

						Section->ToggleShowSkeleton();
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}),
				FCanExecuteAction::CreateLambda([this]() -> bool
					{
						return Sequencer.IsValid();
					}),
				FIsActionChecked::CreateLambda([Section]() -> bool
					{
						return Section->bShowSkeleton;
					})),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

bool FCommonAnimationTrackEditor::AcquireAllBoundSkeletalMeshObjectsFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr, TArray<USkeletalMeshComponent*>& OutSkelMeshComponents,
	TArray<FGuid>& OutBindingIds)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		TArray<USkeletalMeshComponent*> ActorSkelMeshComps;
		Actor->GetComponents(ActorSkelMeshComps);

		if (ActorSkelMeshComps.Num() == 1)
		{
			OutSkelMeshComponents.Add(ActorSkelMeshComps[0]);
			OutBindingIds.Add(Guid);
		}
		else
		{
			constexpr bool bCreateHandleIfMissing = false;
			for (USkeletalMeshComponent* Component : ActorSkelMeshComps)
			{
				if (Component)
				{
					FGuid ObjectHandle = SequencerPtr->GetHandleToObject(Component, bCreateHandleIfMissing);
					if (ObjectHandle.IsValid())
					{
						OutSkelMeshComponents.Add(Component);
						OutBindingIds.Add(ObjectHandle);
					}
				}
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			OutSkelMeshComponents.Add(SkeletalMeshComponent);
			OutBindingIds.Add(Guid);
		}
	}
	return OutSkelMeshComponents.Num() > 0;
}

USkeletalMeshComponent* FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
		{
			return SkeletalMeshComponent;
		}

		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		Actor->GetComponents(SkeletalMeshComponents);
		
		if (SkeletalMeshComponents.Num() == 1)
		{
			return SkeletalMeshComponents[0];
		}
	}
	else if(USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return SkeletalMeshComponent;
		}
	}

	return nullptr;
}

USkeleton* FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(Guid, SequencerPtr);

	if (SkeletalMeshComponents.Num() == 1)
	{
		return GetSkeletonFromComponent(SkeletalMeshComponents[0]);
	}

	return nullptr;
}

bool FCommonAnimationTrackEditor::CreatePoseAsset(const TArray<UObject*> NewAssets, FGuid InObjectBinding)
{
	USkeletalMeshComponent* SkeletalMeshComponent = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(InObjectBinding, GetSequencer());

	bool bResult = false;
	if (NewAssets.Num() > 0)
	{
		for (auto NewAsset : NewAssets)
		{
			UPoseAsset* NewPoseAsset = Cast<UPoseAsset>(NewAsset);
			if (NewPoseAsset)
			{
				NewPoseAsset->AddPoseWithUniqueName(SkeletalMeshComponent);
				bResult = true;
			}
		}

		// if it contains error, warn them
		if (bResult)
		{				
			FText NotificationText;
			if (NewAssets.Num() == 1)
			{
				NotificationText = FText::Format(LOCTEXT("NumPoseAssetsCreated", "{0} Pose assets created."), NewAssets.Num());
			}
			else
			{
				NotificationText = FText::Format(LOCTEXT("PoseAssetsCreated", "Pose asset created: '{0}'."), FText::FromString(NewAssets[0]->GetName()));
			}
						
			FNotificationInfo Info(NotificationText);	
			Info.ExpireDuration = 8.0f;	
			Info.bUseLargeFont = false;
			Info.Hyperlink = FSimpleDelegate::CreateLambda([NewAssets]()
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(NewAssets);
			});
			Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewPoseAssetHyperlink", "Open {0}"), FText::FromString(NewAssets[0]->GetName()));
				
			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if ( Notification.IsValid() )
			{
				Notification->SetCompletionState( SNotificationItem::CS_Success );
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToCreateAsset", "Failed to create asset"));
		}
	}
	return bResult;
}


void FCommonAnimationTrackEditor::HandleCreatePoseAsset(FGuid InObjectBinding)
{
	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(InObjectBinding, GetSequencer());
	if (Skeleton)
	{
		TArray<TSoftObjectPtr<UObject>> Skeletons;
		Skeletons.Add(Skeleton);
		AnimationEditorUtils::ExecuteNewAnimAsset<UPoseAssetFactory, UPoseAsset>(Skeletons, FString("_PoseAsset"), FAnimAssetCreated::CreateSP(this, &FCommonAnimationTrackEditor::CreatePoseAsset, InObjectBinding), false, false);
	}
}

bool FCommonAnimationTrackEditor::CanCreatePoseAsset(FGuid InObjectBinding) const
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
	TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
	FClassViewerInitializationOptions ClassViewerOptions = {};

	if (GlobalClassFilter.IsValid())
	{
		return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, UPoseAsset::StaticClass(), ClassFilterFuncs);
	}

	return true;
}

FCommonAnimationTrackEditor::FCommonAnimationTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer ) 
{ 
	//We use the FGCObject pattern to keep the anim export option alive during the editor session

	AnimSeqExportOption = NewObject<UAnimSeqExportOption>();

}

void FCommonAnimationTrackEditor::OnInitialize()
{
	SequencerSavedHandle = GetSequencer()->OnPostSave().AddRaw(this, &FCommonAnimationTrackEditor::OnSequencerSaved);
	SequencerChangedHandle = GetSequencer()->OnMovieSceneDataChanged().AddRaw(this, &FCommonAnimationTrackEditor::OnSequencerDataChanged);

	++FCommonAnimationTrackEditor::NumberActive;

	if (FCommonAnimationTrackEditor::NumberActive == 1)
	{
		IModularFeatures::Get().RegisterModularFeature(
			IMovieSceneLinkedAnimTrackProvider::GetModularFeatureName(), &GDefaultLinkedAnimTrackProvider);
	}

	if (ShouldActivateSkeletalAnimEditMode())
	{
		// Activate the default mode in case FEditorModeTools::Tick isn't run before here.
		// This can be removed once a general fix for UE-143791 has been implemented.
		GLevelEditorModeTools().ActivateDefaultMode();

		GLevelEditorModeTools().ActivateMode(FSkeletalAnimationTrackEditMode::ModeName);
		FSkeletalAnimationTrackEditMode* EditMode = static_cast<FSkeletalAnimationTrackEditMode*>(GLevelEditorModeTools().GetActiveMode(FSkeletalAnimationTrackEditMode::ModeName));
		if (EditMode)
		{
			EditMode->SetSequencer(GetSequencer());
		}
	}
}

void FCommonAnimationTrackEditor::OnRelease()
{
	--FCommonAnimationTrackEditor::NumberActive;

	if (GetSequencer().IsValid())
	{
		if (SequencerSavedHandle.IsValid())
		{
			GetSequencer()->OnPostSave().Remove(SequencerSavedHandle);
			SequencerSavedHandle.Reset();
		}
		if (SequencerChangedHandle.IsValid())
		{
			GetSequencer()->OnMovieSceneDataChanged().Remove(SequencerChangedHandle);
			SequencerChangedHandle.Reset();
		}
	}
	if (FCommonAnimationTrackEditor::NumberActive == 0)
	{
		IModularFeatures::Get().UnregisterModularFeature(
			IMovieSceneLinkedAnimTrackProvider::GetModularFeatureName(), &GDefaultLinkedAnimTrackProvider);

		GLevelEditorModeTools().DeactivateMode(FSkeletalAnimationTrackEditMode::ModeName);
	}
}

void FCommonAnimationTrackEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (AnimSeqExportOption != nullptr)
	{
		Collector.AddReferencedObject(AnimSeqExportOption);
	}
}


TSharedRef<ISequencerSection> FCommonAnimationTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	
	return MakeShareable( new FCommonAnimationSection(SectionObject, GetSequencer()) );
}


bool FCommonAnimationTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (Asset->IsA<UAnimSequenceBase>() && SequencerPtr.IsValid())
	{
		UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(Asset);
		
		if (TargetObjectGuid.IsValid() && AnimSequence->CanBeUsedInComposition())
		{
			USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(TargetObjectGuid, GetSequencer());

			if (Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(TargetObjectGuid);
				
				UMovieSceneTrack* Track = nullptr;

				const FScopedTransaction Transaction(LOCTEXT("AddAnimation_Transaction", "Add Animation"));

				int32 RowIndex = INDEX_NONE;
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCommonAnimationTrackEditor::AddKeyInternal, Object, AnimSequence, Track, RowIndex));

				return true;
			}
		}
	}
	return false;
}

FText FCommonAnimationTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CommonAnimationTrackEditor_DisplayName", "Skeletal Animation");
}

void FCommonAnimationTrackEditor::BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()) || ObjectClass->IsChildOf(UChildActorComponent::StaticClass()))
	{
		ConstructObjectBindingTrackMenu(MenuBuilder, ObjectBindings);
	}
}

void FCommonAnimationTrackEditor::SaveAnimSequenceLink(FLevelSequenceAnimSequenceLinkItem& Item)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
	if (AnimSequence == nullptr)
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	UMovieSceneSequence* MovieSceneSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	UMovieSceneSequence* RootMovieSceneSequence = SequencerPtr->GetRootMovieSceneSequence();
	FMovieSceneSequenceIDRef Template = SequencerPtr->GetFocusedTemplateID();
	FMovieSceneSequenceTransform RootToLocalTransform = SequencerPtr->GetFocusedMovieSceneSequenceTransformUnwarped();

	if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(AnimSequence))
	{
		UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
		if (!AnimLevelLink)
		{
			AnimLevelLink = NewObject<UAnimSequenceLevelSequenceLink>(AnimSequence, NAME_None, RF_Public | RF_Transactional);
			AnimAssetUserData->AddAssetUserData(AnimLevelLink);
		}
		AnimLevelLink->SetLevelSequence(LevelSequence);
		AnimLevelLink->SkelTrackGuid = Item.SkelTrackGuid;
	}
	USkeletalMeshComponent* SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(Item.SkelTrackGuid, GetSequencer());
	if (AnimSequence && SkelMeshComp && AnimSeqExportOption)
	{
		const bool bSavedSetRetargetSourceAsset = AnimSeqExportOption->bSetRetargetSourceAsset;
		const bool bSavedExportMorphTargets = AnimSeqExportOption->bExportMorphTargets;
		const bool bSavedExportAttributeCurves = AnimSeqExportOption->bExportAttributeCurves;
		const bool bSavedExportMaterialCurves = AnimSeqExportOption->bExportMaterialCurves;
		const bool bSavedSkipCurvesWithZeroValue = AnimSeqExportOption->bSkipCurvesWithZeroValue;
		const bool bSavedExportTransforms = AnimSeqExportOption->bExportTransforms;
		const bool bSavedIncludeComponentTransform = AnimSeqExportOption->bRecordInWorldSpace;
		const bool bSavedEvaluateAllSkeletalMeshComponents = AnimSeqExportOption->bEvaluateAllSkeletalMeshComponents;
		const EAnimInterpolationType SavedInterpolationType = AnimSeqExportOption->Interpolation;
		const ERichCurveInterpMode SavedCurveInterpolationType = AnimSeqExportOption->CurveInterpolation;
		const TArray<FString> SavedIncludeAnimationNames = AnimSeqExportOption->IncludeAnimationNames;
		const TArray<FString> SavedExcludeAnimationNames = AnimSeqExportOption->ExcludeAnimationNames;
		const bool bSavedRemoveExcludedCurves = AnimSeqExportOption->bRemoveExcludedCurves;
		const FFrameNumber SavedWarmUpFrames = AnimSeqExportOption->WarmUpFrames;
		const FFrameNumber SavedDelayBeforeStart = AnimSeqExportOption->DelayBeforeStart;
		const bool bSavedUseCustomTimeRange = AnimSeqExportOption->bUseCustomTimeRange;
		const FFrameNumber SavedCustomStartFrame = AnimSeqExportOption->CustomStartFrame;
		const FFrameNumber SavedCustomEndFrame = AnimSeqExportOption->CustomEndFrame;
		const FFrameRate SavedCustomDisplayRate = AnimSeqExportOption->CustomDisplayRate;
		const bool bSavedUseCustomFrameRate = AnimSeqExportOption->bUseCustomFrameRate;
		const FFrameRate SavedCustomFrameRate = AnimSeqExportOption->CustomFrameRate;

		//If the asset cleared the retarget source asset we don't want to set it!
		if (Item.bSetRetargetSourceAsset)
		{
			if (AnimSequence->GetRetargetSourceAsset().IsNull() == true)
			{
				AnimSeqExportOption->bSetRetargetSourceAsset = false;
			}
			else
			{
				AnimSeqExportOption->bSetRetargetSourceAsset = true;
			}
		}
		else
		{
			AnimSeqExportOption->bSetRetargetSourceAsset = false;
		}
		AnimSeqExportOption->bExportMorphTargets = Item.bExportMorphTargets;
		AnimSeqExportOption->bExportAttributeCurves = Item.bExportAttributeCurves;
		AnimSeqExportOption->bExportMaterialCurves = Item.bExportMaterialCurves;
		AnimSeqExportOption->bSkipCurvesWithZeroValue = Item.bSkipCurvesWithZeroValue;

		AnimSeqExportOption->bExportTransforms = Item.bExportTransforms;
		AnimSeqExportOption->bRecordInWorldSpace = Item.bRecordInWorldSpace;
		AnimSeqExportOption->bEvaluateAllSkeletalMeshComponents = Item.bEvaluateAllSkeletalMeshComponents;
		AnimSeqExportOption->Interpolation = Item.Interpolation;
		AnimSeqExportOption->CurveInterpolation = Item.CurveInterpolation;

		AnimSeqExportOption->IncludeAnimationNames = Item.IncludeAnimationNames;
		AnimSeqExportOption->ExcludeAnimationNames = Item.ExcludeAnimationNames;
		AnimSeqExportOption->bRemoveExcludedCurves = Item.bRemoveExcludedCurves;
		AnimSeqExportOption->WarmUpFrames = Item.WarmUpFrames;
		AnimSeqExportOption->DelayBeforeStart = Item.DelayBeforeStart;
		AnimSeqExportOption->bUseCustomTimeRange = Item.bUseCustomTimeRange;
		AnimSeqExportOption->CustomStartFrame = Item.CustomStartFrame;
		AnimSeqExportOption->CustomEndFrame = Item.CustomEndFrame;
		AnimSeqExportOption->CustomDisplayRate = Item.CustomDisplayRate;
		AnimSeqExportOption->bUseCustomFrameRate = Item.bUseCustomFrameRate;
		AnimSeqExportOption->CustomFrameRate = Item.CustomFrameRate;

		TGuardValue<bool> TransactionOptionGuard(AnimSeqExportOption->bTransactRecording, false);
		FAnimExportSequenceParameters AESP;
		AESP.Player = SequencerPtr.Get();
		AESP.RootToLocalTransform = RootToLocalTransform;
		AESP.MovieSceneSequence = MovieSceneSequence;
		AESP.RootMovieSceneSequence = RootMovieSceneSequence;
		AESP.bForceUseOfMovieScenePlaybackRange = SequencerPtr.Get()->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
		//make sure all spanwables are present
		SequencerPtr->ForceEvaluate();

		// Let plugins bracket the recorder playback (e.g. AnimMixer forces mixer root
		// motion onto the root bone so it's captured by the skel-mesh recorder instead
		// of being applied to the actor).
		TArray<IMovieSceneAnimSequenceBakeScope*> BakeScopes =
			IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimSequenceBakeScope>(
				IMovieSceneAnimSequenceBakeScope::GetModularFeatureName());
		for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->BeginBakeScope(); } }
		ON_SCOPE_EXIT { for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->EndBakeScope(); } } };

		bool bResult = MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, AnimSeqExportOption, AESP, SkelMeshComp);

		AnimSeqExportOption->bSetRetargetSourceAsset = bSavedSetRetargetSourceAsset;
		AnimSeqExportOption->bExportMorphTargets = bSavedExportMorphTargets;
		AnimSeqExportOption->bExportAttributeCurves = bSavedExportAttributeCurves;
		AnimSeqExportOption->bExportMaterialCurves = bSavedExportMaterialCurves;
		AnimSeqExportOption->bSkipCurvesWithZeroValue = bSavedSkipCurvesWithZeroValue;
		AnimSeqExportOption->bExportTransforms = bSavedExportTransforms;
		AnimSeqExportOption->bRecordInWorldSpace = bSavedIncludeComponentTransform;
		AnimSeqExportOption->bEvaluateAllSkeletalMeshComponents = bSavedEvaluateAllSkeletalMeshComponents;
		AnimSeqExportOption->Interpolation = SavedInterpolationType;
		AnimSeqExportOption->CurveInterpolation = SavedCurveInterpolationType;

		AnimSeqExportOption->IncludeAnimationNames = SavedIncludeAnimationNames;
		AnimSeqExportOption->ExcludeAnimationNames = SavedExcludeAnimationNames;
		AnimSeqExportOption->bRemoveExcludedCurves = bSavedRemoveExcludedCurves;
		AnimSeqExportOption->WarmUpFrames = SavedWarmUpFrames;
		AnimSeqExportOption->DelayBeforeStart = SavedDelayBeforeStart;
		AnimSeqExportOption->bUseCustomTimeRange = bSavedUseCustomTimeRange;
		AnimSeqExportOption->CustomStartFrame = SavedCustomStartFrame;
		AnimSeqExportOption->CustomEndFrame = SavedCustomEndFrame;
		AnimSeqExportOption->CustomDisplayRate = SavedCustomDisplayRate;
		AnimSeqExportOption->bUseCustomFrameRate = bSavedUseCustomFrameRate;
		AnimSeqExportOption->CustomFrameRate = SavedCustomFrameRate;
		//save the anim sequence to disk to make sure they are in sync
		UPackage* const Package = AnimSequence->GetOutermost();
		FString const PackageName = Package->GetName();
		FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.SaveFlags = SAVE_NoError;
		UPackage::SavePackage(Package, NULL, *PackageFileName, SaveArgs);
	}
}

void FCommonAnimationTrackEditor::UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID)
{
	if (IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(InSequencer, BindingID))
	{
		Provider->UpdateLinkedAnimSectionRange(InSequencer, BindingID);
	}
}

TAutoConsoleVariable<bool> CVarUseOldSaveLinkedAnims(TEXT("Sequencer.UseOldSaveLinkedAnimsWhenSaving"), false, TEXT("When true use old system when saving linked anims, if false use new, faster system (default false) "));

void FCommonAnimationTrackEditor::OnSequencerSaved(ISequencer& )
{
	
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	if (CVarUseOldSaveLinkedAnims->GetBool())
	{

		ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
		if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
		{
			if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
			{
				ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
				if (LevelAnimLink && LevelAnimLink->AnimSequenceLinks.Num() > 0)
				{

					//if in subsquence and we want to, turn on ShouldEvaluateSubSequencesInIsolation, use the following object for that
					{
						UE::Sequencer::FSequencerBakingSetupRestore RestoreBaking(SequencerPtr);
						for (int32 Index = LevelAnimLink->AnimSequenceLinks.Num() - 1; Index >= 0; --Index)
						{
							FLevelSequenceAnimSequenceLinkItem& Item = LevelAnimLink->AnimSequenceLinks[Index];
							UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
							if (AnimSequence == nullptr)
							{
								LevelAnimLink->AnimSequenceLinks.RemoveAt(Index);
								continue;
							}
							SaveAnimSequenceLink(Item);
						}
					}//restore bake settings

					//re-evaluate at current frame
					SequencerPtr->ForceEvaluate();
				}
			}
		}
	}
	else
	{

		ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
		if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
		{
			USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>();
			TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem ? SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr) : nullptr;
			//ones not linked up to do smart baking, still add them to system
			TArray<TPair<UAnimSequence*, FGuid>> TempsToSave;
			UE::Sequencer::FSequencerBakingSetupRestore RestoreBaking(SequencerPtr);

			if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
			{
				ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
				if (LevelAnimLink && LevelAnimLink->AnimSequenceLinks.Num() > 0)
				{
					for (int32 Index = LevelAnimLink->AnimSequenceLinks.Num() - 1; Index >= 0; --Index)
					{
						FLevelSequenceAnimSequenceLinkItem& Item = LevelAnimLink->AnimSequenceLinks[Index];
						UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
						if (AnimSequence == nullptr)
						{
							LevelAnimLink->AnimSequenceLinks.RemoveAt(Index);
							continue;
						}
						if (Item.bAutoBake && SequencerBaker.IsValid())
						{
							if (SequencerBaker->FindRecorder(Item.SkelTrackGuid, AnimSequence) == nullptr)
							{
								TPair<UAnimSequence*, FGuid> Pair;
								Pair.Key = AnimSequence;
								Pair.Value = Item.SkelTrackGuid;
								TempsToSave.Add(Pair);
							}
						}
						else
						{
							//mz todo and test
							//	TSharedPtr<ISequencerBakeRecorder> Recorder = MakeShared<UE::Sequencer::FAnimSequenceBakeRecorder>(*SequencerBaker, Item.SkelTrackGuid, AnimSequence, Item);
							//Add to extra recordes and use that system.
							SaveAnimSequenceLink(Item);
						}
					}
					if (SequencerBaker)
					{
						//bake it all out
						SequencerBaker->AddAllRecordersToBake();
						SequencerBaker->BakeFullRange();

						//now make sure it's all good
						for (TPair<UAnimSequence*, FGuid>& Pair : TempsToSave)
						{
							UAnimSequence* AnimSequence = Pair.Key;

							if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(AnimSequence))
							{
								UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
								if (!AnimLevelLink)
								{
									AnimLevelLink = NewObject<UAnimSequenceLevelSequenceLink>(AnimSequence, NAME_None, RF_Public | RF_Transactional);
									AnimAssetUserData->AddAssetUserData(AnimLevelLink);
								}
								AnimLevelLink->SetLevelSequence(LevelSequence);
								AnimLevelLink->SkelTrackGuid = Pair.Value;
							}

							//save the anim sequence to disk to make sure they are in sync
							UPackage* const Package = AnimSequence->GetOutermost();
							FString const PackageName = Package->GetName();
							FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

							FSavePackageArgs SaveArgs;
							SaveArgs.TopLevelFlags = RF_Standalone;
							SaveArgs.SaveFlags = SAVE_NoError;
							UPackage::SavePackage(Package, NULL, *PackageFileName, SaveArgs);
						}
					}
				}
			}
		}
	}
}

//dirty anim sequence when the sequencer changes, to make sure it get's checked out etc..
void FCommonAnimationTrackEditor::OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	//only return if data really changed
	if(DataChangeType ==  EMovieSceneDataChangeType::RefreshTree ||
		DataChangeType == EMovieSceneDataChangeType::ActiveMovieSceneChanged ||
		DataChangeType == EMovieSceneDataChangeType::RefreshAllImmediately)
	{
		return;
	}
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				for (int32 Index = LevelAnimLink->AnimSequenceLinks.Num() - 1; Index >= 0; --Index)
				{
					FLevelSequenceAnimSequenceLinkItem& Item = LevelAnimLink->AnimSequenceLinks[Index];
					UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
					if (AnimSequence)
					{
						AnimSequence->Modify();
					}
				}
			}
		}
	}
}

bool FCommonAnimationTrackEditor::CreateAnimationSequence(const TArray<UObject*> NewAssets, USkeletalMeshComponent* SkelMeshComp, FGuid Binding, bool bCreateSoftLink)
{
	bool bResult = false;
	if (NewAssets.Num() > 0)
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(NewAssets[0]);
		if (AnimSequence && GetSequencer().IsValid())
		{
			UObject* NewAsset = NewAssets[0];
			TSharedPtr<SWindow> ParentWindow;
			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}

			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(NSLOCTEXT("UnrealEd", "AnimSeqOpionsTitle", "Animation Sequence Options"))
				.SizingRule(ESizingRule::UserSized)
				.AutoCenter(EAutoCenter::PrimaryWorkArea)
				.ClientSize(FVector2D(500, 445));

			TSharedPtr<SAnimSequenceOptionsWindow> OptionWindow;
			Window->SetContent
			(
				SAssignNew(OptionWindow, SAnimSequenceOptionsWindow)
				.ExportOptions(AnimSeqExportOption)
				.WidgetWindow(Window)
				.FullPath(FText::FromString(NewAssets[0]->GetName()))
			);

			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			if (OptionWindow->ShouldExport())
			{
				TSharedPtr<ISequencer> ParentSequencer = GetSequencer();
				UMovieSceneSequence* MovieSceneSequence = ParentSequencer->GetFocusedMovieSceneSequence();
				UMovieSceneSequence* RootMovieSceneSequence = ParentSequencer->GetRootMovieSceneSequence();
				FMovieSceneSequenceTransform RootToLocalTransform = ParentSequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
				//if in subsquence and we want to, turn on ShouldEvaluateSubSequencesInIsolation, use the following object for that
				{
					UE::Sequencer::FSequencerBakingSetupRestore RestoreBaking(ParentSequencer);
					//reacquire the above function may force spawnables to get rebound.
					SkelMeshComp = UE::Sequencer::FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(Binding,ParentSequencer);

					FAnimExportSequenceParameters AESP;
					AESP.Player = ParentSequencer.Get();
					AESP.RootToLocalTransform = RootToLocalTransform;
					AESP.MovieSceneSequence = MovieSceneSequence;
					AESP.RootMovieSceneSequence = RootMovieSceneSequence;
					AESP.bForceUseOfMovieScenePlaybackRange = ParentSequencer->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
					AnimSeqExportOption->CustomDisplayRate = ParentSequencer->GetFocusedDisplayRate();

					// Let plugins bracket the recorder playback (see other site for details).
					TArray<IMovieSceneAnimSequenceBakeScope*> BakeScopes =
						IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimSequenceBakeScope>(
							IMovieSceneAnimSequenceBakeScope::GetModularFeatureName());
					for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->BeginBakeScope(); } }
					ON_SCOPE_EXIT { for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->EndBakeScope(); } } };

					bResult = MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, AnimSeqExportOption, AESP, SkelMeshComp);
				}
				//re-evaluate at current frame
				ParentSequencer->ForceEvaluate();
			}
		}

		if (bResult && bCreateSoftLink)
		{
			FScopedTransaction Transaction(LOCTEXT("SaveLinkedAnimation_Transaction", "Save Link Animation"));
			TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
			ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
			if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass())
				&& AnimSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
			{
				LevelSequence->Modify();
				if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(AnimSequence))
				{
					UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
					if (!AnimLevelLink)
					{
						AnimLevelLink = NewObject<UAnimSequenceLevelSequenceLink>(AnimSequence, NAME_None, RF_Public | RF_Transactional);
						AnimAssetUserData->AddAssetUserData(AnimLevelLink);
					}
					
					AnimLevelLink->SetLevelSequence(LevelSequence);
					AnimLevelLink->SkelTrackGuid = Binding;
				}
				if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
				{
					bool bAddItem = true;
					ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
					if (LevelAnimLink)
					{
						for (FLevelSequenceAnimSequenceLinkItem& LevelAnimLinkItem : LevelAnimLink->AnimSequenceLinks)
						{
							if (LevelAnimLinkItem.IsEqual(Binding, AnimSeqExportOption->bUseCustomTimeRange,
								AnimSeqExportOption->CustomStartFrame, AnimSeqExportOption->CustomEndFrame, AnimSeqExportOption->CustomDisplayRate,
								AnimSeqExportOption->bUseCustomFrameRate, AnimSeqExportOption->CustomFrameRate
								))
							{
								bAddItem = false;
								UAnimSequence* OtherAnimSequence = LevelAnimLinkItem.ResolveAnimSequence();
								
								if (OtherAnimSequence != AnimSequence)
								{
									if (IInterface_AssetUserData* OtherAnimAssetUserData = Cast< IInterface_AssetUserData >(OtherAnimSequence))
									{
										UAnimSequenceLevelSequenceLink* OtherAnimLevelLink = OtherAnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
										if (OtherAnimLevelLink)
										{
											OtherAnimAssetUserData->RemoveUserDataOfClass(UAnimSequenceLevelSequenceLink::StaticClass());
										}
									}
								}
								LevelAnimLinkItem.PathToAnimSequence = FSoftObjectPath(AnimSequence);
								LevelAnimLinkItem.bSetRetargetSourceAsset = AnimSeqExportOption->bSetRetargetSourceAsset;
								LevelAnimLinkItem.bExportMorphTargets = AnimSeqExportOption->bExportMorphTargets;
								LevelAnimLinkItem.bExportAttributeCurves = AnimSeqExportOption->bExportAttributeCurves;
								LevelAnimLinkItem.bExportMaterialCurves = AnimSeqExportOption->bExportMaterialCurves;
								LevelAnimLinkItem.bSkipCurvesWithZeroValue = AnimSeqExportOption->bSkipCurvesWithZeroValue;
								LevelAnimLinkItem.bExportTransforms = AnimSeqExportOption->bExportTransforms;
								LevelAnimLinkItem.bRecordInWorldSpace = AnimSeqExportOption->bRecordInWorldSpace;
								LevelAnimLinkItem.bEvaluateAllSkeletalMeshComponents = AnimSeqExportOption->bEvaluateAllSkeletalMeshComponents;
								LevelAnimLinkItem.Interpolation = AnimSeqExportOption->Interpolation;
								LevelAnimLinkItem.CurveInterpolation = AnimSeqExportOption->CurveInterpolation;
								LevelAnimLinkItem.IncludeAnimationNames = AnimSeqExportOption->IncludeAnimationNames;
								LevelAnimLinkItem.ExcludeAnimationNames = AnimSeqExportOption->ExcludeAnimationNames;
								LevelAnimLinkItem.bRemoveExcludedCurves = AnimSeqExportOption->bRemoveExcludedCurves;
								LevelAnimLinkItem.WarmUpFrames = AnimSeqExportOption->WarmUpFrames;
								LevelAnimLinkItem.DelayBeforeStart = AnimSeqExportOption->DelayBeforeStart;
								LevelAnimLinkItem.bUseCustomTimeRange = AnimSeqExportOption->bUseCustomTimeRange;
								LevelAnimLinkItem.CustomStartFrame = AnimSeqExportOption->CustomStartFrame;
								LevelAnimLinkItem.CustomEndFrame = AnimSeqExportOption->CustomEndFrame;
								LevelAnimLinkItem.CustomDisplayRate = AnimSeqExportOption->CustomDisplayRate;
								LevelAnimLinkItem.bUseCustomFrameRate = AnimSeqExportOption->bUseCustomFrameRate;
								LevelAnimLinkItem.CustomFrameRate = AnimSeqExportOption->CustomFrameRate;
								break;
							}
						}
					}
					else
					{
						LevelAnimLink = NewObject<ULevelSequenceAnimSequenceLink>(LevelSequence, NAME_None, RF_Public | RF_Transactional);
						
					}
					if (bAddItem == true)
					{
						FLevelSequenceAnimSequenceLinkItem LevelAnimLinkItem;
						LevelAnimLinkItem.SkelTrackGuid = Binding;
						LevelAnimLinkItem.PathToAnimSequence = FSoftObjectPath(AnimSequence);
						LevelAnimLinkItem.bSetRetargetSourceAsset = AnimSeqExportOption->bSetRetargetSourceAsset;
						LevelAnimLinkItem.bExportMorphTargets = AnimSeqExportOption->bExportMorphTargets;
						LevelAnimLinkItem.bExportAttributeCurves = AnimSeqExportOption->bExportAttributeCurves;
						LevelAnimLinkItem.bExportMaterialCurves = AnimSeqExportOption->bExportMaterialCurves;
						LevelAnimLinkItem.bSkipCurvesWithZeroValue = AnimSeqExportOption->bSkipCurvesWithZeroValue;
						LevelAnimLinkItem.bExportTransforms = AnimSeqExportOption->bExportTransforms;
						LevelAnimLinkItem.bRecordInWorldSpace = AnimSeqExportOption->bRecordInWorldSpace;
						LevelAnimLinkItem.bEvaluateAllSkeletalMeshComponents = AnimSeqExportOption->bEvaluateAllSkeletalMeshComponents;
						LevelAnimLinkItem.Interpolation = AnimSeqExportOption->Interpolation;
						LevelAnimLinkItem.CurveInterpolation = AnimSeqExportOption->CurveInterpolation;
						LevelAnimLinkItem.IncludeAnimationNames = AnimSeqExportOption->IncludeAnimationNames;
						LevelAnimLinkItem.ExcludeAnimationNames = AnimSeqExportOption->ExcludeAnimationNames;
						LevelAnimLinkItem.bRemoveExcludedCurves = AnimSeqExportOption->bRemoveExcludedCurves;
						LevelAnimLinkItem.WarmUpFrames = AnimSeqExportOption->WarmUpFrames;
						LevelAnimLinkItem.DelayBeforeStart = AnimSeqExportOption->DelayBeforeStart;
						LevelAnimLinkItem.bUseCustomTimeRange = AnimSeqExportOption->bUseCustomTimeRange;
						LevelAnimLinkItem.CustomStartFrame = AnimSeqExportOption->CustomStartFrame;
						LevelAnimLinkItem.CustomEndFrame = AnimSeqExportOption->CustomEndFrame;
						LevelAnimLinkItem.CustomDisplayRate = AnimSeqExportOption->CustomDisplayRate;
						LevelAnimLinkItem.bUseCustomFrameRate = AnimSeqExportOption->bUseCustomFrameRate;
						LevelAnimLinkItem.CustomFrameRate = AnimSeqExportOption->CustomFrameRate;

						LevelAnimLink->AnimSequenceLinks.Add(LevelAnimLinkItem);
						AssetUserDataInterface->AddAssetUserData(LevelAnimLink);
					}

					/*
					FText Name = SequencerPtr->GetDisplayName(Binding);
					FString StringName = Name.ToString();
					if (bAddItem == false)
					{
						//ok already had a name added so need to remove the old one..
						TArray<FString> Strings;
						StringName.ParseIntoArray(Strings,TEXT(" --> "));
						if (Strings.Num() > 0)
						{
							StringName = Strings[0];
						}
					}
	
					FString AnimName = AnimSequence->GetName();
					StringName = StringName + FString(TEXT(" --> ")) + AnimName;
					SequencerPtr->SetDisplayName(Binding, FText::FromString(StringName));
					*/

				}
			}
		}
		// if it contains error, warn them
		if (bResult)
		{
			FText NotificationText;
			if (NewAssets.Num() == 1)
			{
				NotificationText = FText::Format(LOCTEXT("NumAnimSequenceAssetsCreated", "{0} Anim Sequence  assets created."), NewAssets.Num());
			}
			else
			{
				NotificationText = FText::Format(LOCTEXT("AnimSequenceAssetsCreated", "Anim Sequence asset created: '{0}'."), FText::FromString(NewAssets[0]->GetName()));
			}

			FNotificationInfo Info(NotificationText);
			Info.ExpireDuration = 8.0f;
			Info.bUseLargeFont = false;
			Info.Hyperlink = FSimpleDelegate::CreateLambda([NewAssets]()
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(NewAssets);
			});
			Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewPoseAssetHyperlink", "Open {0}"), FText::FromString(NewAssets[0]->GetName()));

			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if (Notification.IsValid())
			{
				Notification->SetCompletionState(SNotificationItem::CS_Success);
			}
			if (const TSharedPtr<ISequencer> ParentSequencer = GetSequencer())
			{
				ParentSequencer->RequestEvaluate(); 
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToCreateAsset", "Failed to create asset"));
		}
	}
	return bResult;
}

void FCommonAnimationTrackEditor::HandleCreateAnimationSequence(USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton, FGuid Binding, bool bCreateSoftLink)
{
	if (SkelMeshComp)
	{
		TArray<TSoftObjectPtr<UObject>> Skels;
		if (SkelMeshComp->GetSkeletalMeshAsset())
		{
			Skels.Add(SkelMeshComp->GetSkeletalMeshAsset());
		}
		else
		{
			Skels.Add(Skeleton);
		}
	
		const bool bDoNotShowNameDialog = false;
		const bool bAllowReplaceExisting = true;
		AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>(Skels, FString("_Sequence"), FAnimAssetCreated::CreateSP(this, &FCommonAnimationTrackEditor::CreateAnimationSequence, SkelMeshComp, Binding, bCreateSoftLink), bDoNotShowNameDialog, bAllowReplaceExisting);
	}
}

void FCommonAnimationTrackEditor::OpenLinkedAnimSequence(FGuid Binding)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				
				for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
				{
					if (Item.SkelTrackGuid == Binding)
					{
						UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
						if (AnimSequence)
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimSequence);
						}
					}
				}
			}
		}
	}
}

IMovieSceneLinkedAnimTrackProvider* FCommonAnimationTrackEditor::FindProviderForBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID)
{
	TArray<IMovieSceneLinkedAnimTrackProvider*> Providers =
		IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneLinkedAnimTrackProvider>(
			IMovieSceneLinkedAnimTrackProvider::GetModularFeatureName());

	Providers.Sort([](const IMovieSceneLinkedAnimTrackProvider& A, const IMovieSceneLinkedAnimTrackProvider& B)
	{
		return A.GetLinkedAnimTrackPriority() < B.GetLinkedAnimTrackPriority();
	});

	for (IMovieSceneLinkedAnimTrackProvider* Provider : Providers)
	{
		if (Provider->CanHandleBinding(InSequencer, BindingID))
		{
			return Provider;
		}
	}
	return nullptr;
}

UMovieSceneSkeletalAnimationTrack* FCommonAnimationTrackEditor::FindEmptyAnimationTrack(FGuid BindingID)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid() || SequencerPtr->GetFocusedMovieSceneSequence() == nullptr ||
		SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
	{
		return nullptr;
	}
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	//todo handle multiples
	TArray<UMovieSceneTrack*> AllTracks = MovieScene->FindTracks(UMovieSceneSkeletalAnimationTrack::StaticClass(), BindingID, NAME_None);
	for (UMovieSceneTrack* Track : AllTracks)
	{
		if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
		{
			if (AnimTrack->GetAllSections().Num() == 0)
			{
				return AnimTrack;
			}
		}
	}
	return nullptr;
}

void FCommonAnimationTrackEditor::CreateLinkedAnimationSequence(USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton, FGuid Binding, bool bCreateSoftLink)
{
	HandleCreateAnimationSequence(SkelMeshComp, Skeleton, Binding, bCreateSoftLink);
}

TArray<FLevelSequenceAnimSequenceLinkItem*> FCommonAnimationTrackEditor::GetLinkedAnimSequences(const TWeakPtr<ISequencer>& InSequencer, FGuid Binding)
{
	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems;
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return LinkedItems;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
				{
					if (Item.SkelTrackGuid == Binding)
					{
						if (UAnimSequence* AnimSequence = Item.ResolveAnimSequence())
						{
							LinkedItems.Add(&Item);
						}
					}
				}
			}
		}
	}
	return LinkedItems;

}


static bool IsObjectEnabled(UMovieSceneSignedObject* Object)
{
	if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Object))
	{
		if (Track->GetMaxRowIndex() == 0)
		{
			return !Track->IsEvalDisabled();
		}
		for (int32 RowIndex = 0; RowIndex <= Track->GetMaxRowIndex(); ++RowIndex)
		{
			if (!Track->IsRowEvalDisabled(RowIndex))
			{
				return true;
			}
		}

	}
	else if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(Object))
	{
		if (UMovieSceneTrack* ParentTrack = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			if (ParentTrack->GetMaxRowIndex() == 0)
			{
				return !ParentTrack->IsEvalDisabled();
			}
			else
			{
				const int32 SectionRowIndex = Section->GetRowIndex();
				for (int32 RowIndex = 0; RowIndex <= ParentTrack->GetMaxRowIndex(); ++RowIndex)
				{
					if (SectionRowIndex == RowIndex && !ParentTrack->IsRowEvalDisabled(RowIndex))
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

namespace
{
	// Look up the real FCommonAnimationTrackEditor instance owned by the Sequencer. The provider's
	// create-track flow calls AsShared() via CreateSP, so the TrackEditor reference passed in must
	// be held by a TSharedPtr (which the editors in FSequencer::TrackEditors are).
	TSharedPtr<FCommonAnimationTrackEditor> FindSharedCommonAnimationTrackEditor(const TSharedRef<ISequencer>& InSequencer)
	{
		UMovieSceneSkeletalAnimationTrack* LookupTrack = NewObject<UMovieSceneSkeletalAnimationTrack>(GetTransientPackage());
		TSharedPtr<ISequencerTrackEditor> AbstractEditor = InSequencer->GetTrackEditor(LookupTrack);
		return StaticCastSharedPtr<FCommonAnimationTrackEditor>(AbstractEditor);
	}
}

UMovieSceneTrack* FCommonAnimationTrackEditor::GetOrCreateLinkedAnimTrack(
	const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return nullptr;
	}
	IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(InSequencer, BindingID);
	if (!Provider)
	{
		return nullptr;
	}
	TSharedPtr<FCommonAnimationTrackEditor> RealTrackEditor = FindSharedCommonAnimationTrackEditor(SequencerPtr.ToSharedRef());
	if (!RealTrackEditor.IsValid())
	{
		return nullptr;
	}
	return Provider->GetOrCreateLinkedAnimTrack(InSequencer, BindingID, *RealTrackEditor);
}

bool FCommonAnimationTrackEditor::IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bIsolate)
{
	if (IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(InSequencer, BindingID))
	{
		if (UMovieSceneTrack* Track = Provider->GetLinkedAnimTrack(InSequencer, BindingID))
		{
			Track->MarkAsChanged();
		}
		return Provider->IsolateLinkedAnimTrack(InSequencer, BindingID, bIsolate);
	}
	return false;
}

UMovieSceneTrack* FCommonAnimationTrackEditor::GetLinkedAnimTrack(FGuid Binding, bool bCreateIfMissing)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return nullptr;
	}
	if (IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(SequencerPtr, Binding))
	{
		if (bCreateIfMissing == false)
		{
			return Provider->GetLinkedAnimTrack(SequencerPtr,Binding);
		}
		else
		{
			return Provider->GetOrCreateLinkedAnimTrack(SequencerPtr, Binding, *this);
		}
	}
	return nullptr;
}

void FCommonAnimationTrackEditor::ConfigureAutoBake(const TWeakPtr<ISequencer>& InSequencer, TArray<FGuid> Bindings)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	FScopedTransaction MatchSection(LOCTEXT("StartAutoBake_Transaction", "Set Up LinkedAutoBakes"));
	for (FGuid& Binding : Bindings)
	{
		FCommonAnimationTrackEditor::GetOrCreateLinkedAnimTrack(SequencerPtr, Binding);
		ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
		if (LevelSequence && LevelSequence->GetMovieScene() && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
		{
			UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
			if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
			{
				ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
				if (LevelAnimLink)
				{
					for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
					{
						if (Item.SkelTrackGuid == Binding && Item.bAutoBake == true)
						{
							UAnimSequence* AnimSequence = Item.ResolveAnimSequence();

							USkeletalMeshComponent* SkelMeshComp = AcquireSkeletalMeshFromObjectGuid(Binding, SequencerPtr);
							if (AnimSequence && SkelMeshComp)
							{
								if (USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>())
								{
									if (TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr))
									{
										if (SequencerBaker->FindRecorder(Binding, AnimSequence) == nullptr)
										{
											LevelSequence->Modify();
											TSharedPtr<ISequencerBakeRecorder> Recorder = FAnimSequenceBakeRecorder::CreateRecorderAndAddToBaker(SequencerBaker, Binding, AnimSequence, Item);
											SequencerBaker->AddRecorderToBake(Recorder);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

bool FCommonAnimationTrackEditor::IsolateLinkedAnimTracks(const TWeakPtr<ISequencer>& InSequencer, TArray<FGuid> Bindings, bool bIsolate)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return false;
	}
	if (bIsolate)
	{
		if (USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>())
		{
			if (TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr))
			{
				SequencerBaker->BakeFullRange();
			}
		}
	}
	FScopedTransaction MatchSection(LOCTEXT("IsolateLinkedAnimTracks", "Isolate Linked Anim Tracks"));

	bool bAnyChanged = false;
	for (FGuid& Binding : Bindings)
	{
		if (FCommonAnimationTrackEditor::IsolateLinkedAnimTrack(SequencerPtr, Binding, bIsolate))
		{
			bAnyChanged = true;
		}
	}

	if (bAnyChanged)
	{
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	return bAnyChanged;
}

bool FCommonAnimationTrackEditor::HasNoLinkedTracks(TArray<FGuid> Bindings)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return true;
	}

	for (FGuid& Binding : Bindings)
	{
		if (GetLinkedAnimTrack(Binding, false))
		{
			return false;
		}
	}
	return true;
}

bool FCommonAnimationTrackEditor::HasAutoBakedLinkedTracks(TArray<FGuid> Bindings)
{
	return !HasNoLinkedTracks(Bindings);
}

ECheckBoxState FCommonAnimationTrackEditor::IsLinkedAnimTrackIsolated(FGuid ObjectGuid) const
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}
	if (IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(SequencerPtr, ObjectGuid))
	{
		if (UMovieSceneTrack* Track = Provider->GetLinkedAnimTrack(SequencerPtr, ObjectGuid))
		{
			return IsObjectEnabled(Track) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		}
	}
	return ECheckBoxState::Unchecked;
}

void FCommonAnimationTrackEditor::OnIsolateAnimTrackClicked(ECheckBoxState NewState, FGuid ObjectGuid)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	const bool bIsolate = (NewState == ECheckBoxState::Checked);
	TArray<FGuid> SelectedObjects;

	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : SequencerPtr->GetViewModel()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		FGuid Guid = ObjectBindingNode->GetObjectGuid();
		SelectedObjects.Add(Guid);
	}
	if (SelectedObjects.Contains(ObjectGuid) == false)
	{
		SelectedObjects.Add(ObjectGuid);
	}
	IsolateLinkedAnimTracks(SequencerPtr,SelectedObjects, bIsolate);
}

void FCommonAnimationTrackEditor::BuildObjectBindingColumnWidgets(
	TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox,
	const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding,
	const UE::Sequencer::FCreateOutlinerViewParams& InParams,
	const FName& InColumnName)
{
	using namespace UE::Sequencer;

	FGuid ObjectGuid = ObjectBinding->GetObjectGuid();
	if (HasNoLinkedTracks({ObjectGuid}))
	{
		return;
	}

	bool bIsolateAnim = false;
	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		bIsolateAnim = true;
	}
	else if (InColumnName == FCommonOutlinerNames::KeyFrame)
	{
		bIsolateAnim = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false;
	}
	else if (InColumnName == FCommonOutlinerNames::Edit)
	{
		bIsolateAnim = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false &&
			InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame) == false;
	}

	if (bIsolateAnim)
	{
		const bool bEditColumn = InColumnName == FCommonOutlinerNames::Edit;
		GetEditBox()->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(bEditColumn ? HAlign_Left : HAlign_Center)
		.Padding(bEditColumn ? FMargin(4.f, 0.f) : FMargin(0.f))
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "Sequencer.Outliner.ToggleButton")
			.Type(ESlateCheckBoxType::ToggleButton)
			.IsFocusable(false)
			.IsChecked(this, &FCommonAnimationTrackEditor::IsLinkedAnimTrackIsolated, ObjectGuid)
			.OnCheckStateChanged(this, &FCommonAnimationTrackEditor::OnIsolateAnimTrackClicked, ObjectGuid)
			.ToolTipText(LOCTEXT("IsolateAnimTrack", "Isolate the Baked Anim Track or not"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Sequencer.Bake"))
			]
		];
	}
}

bool FCommonAnimationTrackEditor::CanOpenLinkedAnimSequences(TArray<FGuid> Bindings)
{
	for (FGuid& Binding : Bindings)
	{
		if (CanOpenLinkedAnimSequence(Binding))
		{
			return true;
		}
	}
	return false;
}

bool FCommonAnimationTrackEditor::CanOpenLinkedAnimSequence(FGuid Binding)
{

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return false;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{

				for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
				{
					if (Item.SkelTrackGuid == Binding)
					{
						UAnimSequence* AnimSequence = Item.ResolveAnimSequence();
						if (AnimSequence)
						{
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

bool FCommonAnimationTrackEditor::CanDeleteLinkedAnimSequence(FGuid Binding)
{
	return CanOpenLinkedAnimSequence(Binding);
}

void FCommonAnimationTrackEditor::HandleDeleteLinkedAnimSequence(FGuid Binding)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData<ULevelSequenceAnimSequenceLink>();
			if (LevelAnimLink)
			{
				for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
				{
					if (Item.SkelTrackGuid == Binding)
					{
						if (UAnimSequence* AnimSequence = Item.ResolveAnimSequence())
						{
							UnlinkLinkedAnimSequence(AnimSequence);
						}
						break;
					}
				}
			}
		}
	}
}

bool FCommonAnimationTrackEditor::UnlinkLinkedAnimSequence(UAnimSequence* AnimSequence)
{
	bool bDidUnlink = false;
	if (IInterface_AssetUserData* AnimAssetUserData = Cast<IInterface_AssetUserData>(AnimSequence))
	{
		UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData<UAnimSequenceLevelSequenceLink>();
		if (AnimLevelLink)
		{
			ULevelSequence* LevelSequence = AnimLevelLink->ResolveLevelSequence();
			if (LevelSequence)
			{
				if (IInterface_AssetUserData* LevelSequenceUserData = Cast<IInterface_AssetUserData>(LevelSequence))
				{
					ULevelSequenceAnimSequenceLink* LevelAnimLink = LevelSequenceUserData->GetAssetUserData<ULevelSequenceAnimSequenceLink>();
					if (LevelAnimLink)
					{
						LevelAnimLink->Modify();
						for (int32 Index = 0; Index < LevelAnimLink->AnimSequenceLinks.Num(); ++Index)
						{
							if (LevelAnimLink->AnimSequenceLinks[Index].ResolveAnimSequence() == AnimSequence)
							{
								LevelAnimLink->AnimSequenceLinks.RemoveAtSwap(Index);
								bDidUnlink = true;
								break;
							}
						}
					}
				}
			}
			AnimAssetUserData->RemoveUserDataOfClass(UAnimSequenceLevelSequenceLink::StaticClass());
		}
	}
	return bDidUnlink;
}

void FCommonAnimationTrackEditor::HandleRemoveAutoBake(TArray<FGuid> Bindings)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	if(ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence()))
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveAutoBake_Transaction", "Remove AutoBake"));
		LevelSequence->Modify();
		for (const FGuid& Binding : Bindings)
		{
			DeleteLinkedAnimTrackForBinding(SequencerPtr, Binding, /*bOnlyAutoBaked=*/true);
		}
	}
}

bool FCommonAnimationTrackEditor::DeleteLinkedAnimTrackForBinding(
	const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bOnlyAutoBaked)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return false;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (!LevelSequence)
	{
		return false;
	}
	USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>();
	TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem ? SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr) : nullptr;

	// Collect matching anim sequences before unlinking to avoid mutating the array during iteration
	TArray<UAnimSequence*> SequencesToUnlink;
	if (IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(LevelSequence))
	{
		ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData<ULevelSequenceAnimSequenceLink>();
		if (LevelAnimLink)
		{
			for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
			{
				if (Item.SkelTrackGuid == BindingID && (!bOnlyAutoBaked || Item.bAutoBake))
				{
					if (UAnimSequence* AnimSequence = Item.ResolveAnimSequence())
					{
						SequencesToUnlink.Add(AnimSequence);
						//we can remove the bakers here though
						if(SequencerBaker.IsValid())
						{
							if (TSharedPtr<ISequencerBakeRecorder>* Recorder = SequencerBaker->FindRecorder(BindingID, AnimSequence))
							{
								SequencerBaker->RemoveRecorder(*Recorder);
							}
						}
					}
				}
			}
		}
	}

	bool bRemovedTrack = false;
	if (IMovieSceneLinkedAnimTrackProvider* Provider = FindProviderForBinding(InSequencer, BindingID))
	{
		bRemovedTrack = Provider->DeleteLinkedAnimTrack(InSequencer, BindingID);
	}

	if(SequencesToUnlink.Num() >0)
	{
		for (UAnimSequence* AnimSequence : SequencesToUnlink)
		{
			UnlinkLinkedAnimSequence(AnimSequence);
		}
	}

	return bRemovedTrack;
}

void FCommonAnimationTrackEditor::ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{

	auto AddAutoBakedMenus = [&MenuBuilder,this](const TArray<FGuid>& InObjectBindings)
	{
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(
			LOCTEXT("StartAutoBake", "Add AutoBake"),
			LOCTEXT("StartAutoBakeTooltip", "Setup linked and automatic baked Anim Sequences"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakSequencer = TWeakPtr<ISequencer>(GetSequencer()), InObjectBindings]()
				{
					FCommonAnimationTrackEditor::ConfigureAutoBake(WeakSequencer, InObjectBindings);
				}),
				FCanExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HasNoLinkedTracks, InObjectBindings)
			),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveAutoBake", "Remove AutoBake"),
			LOCTEXT("RemoveAutoBakeTooltip", "Remove the auto-baked Animation Sequence link and delete the linked animation track for this binding."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HandleRemoveAutoBake, InObjectBindings),
				FCanExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HasAutoBakedLinkedTracks, InObjectBindings)
			),
			NAME_None,
			EUserInterfaceActionType::Button);
	};

	if(ObjectBindings.Num() > 0)
	{
		USkeletalMeshComponent* SkelMeshComp = AcquireSkeletalMeshFromObjectGuid(ObjectBindings[0], GetSequencer());

		if (SkelMeshComp)
		{

			MenuBuilder.BeginSection("Create Animation Assets", LOCTEXT("CreateAnimationAssetsName", "Create Animation Assets"));
			USkeleton* Skeleton = GetSkeletonFromComponent(SkelMeshComp);
			//todo do we not link if alreadhy linked???

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CreateLinkAnimSequence", "Create Linked Animation Sequence"),
				LOCTEXT("CreateLinkAnimSequenceTooltip", "Create Animation Sequence for this Skeletal Mesh and have this Track Own that Anim Sequence. Note it will create it based upon the Sequencer Display Range and Display Frame Rate"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HandleCreateAnimationSequence, SkelMeshComp, Skeleton, ObjectBindings[0], true)),
				NAME_None,
				EUserInterfaceActionType::Button);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteLinkAnimSequence", "Delete Linked Animation Sequence"),
				LOCTEXT("DeleteLinkAnimSequenceTooltip", "Remove the link between this track and its owned Animation Sequence."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HandleDeleteLinkedAnimSequence, ObjectBindings[0]),
					FCanExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::CanDeleteLinkedAnimSequence, ObjectBindings[0])
				),
				NAME_None,
				EUserInterfaceActionType::Button);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenAnimSequence", "Open Linked Animation Sequence"),
				LOCTEXT("OpenAnimSequenceTooltip", "Open Animation Sequence that this Animation Track is Driving."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::OpenLinkedAnimSequence, ObjectBindings[0]),
					FCanExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::CanOpenLinkedAnimSequence, ObjectBindings[0])
				),
				NAME_None,
				EUserInterfaceActionType::Button);
			
			MenuBuilder.AddSeparator();
		
				AddAutoBakedMenus(ObjectBindings);
	
			MenuBuilder.AddSeparator();
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CreateAnimSequence", "Bake Animation Sequence"),
				LOCTEXT("PasteCreateAnimSequenceTooltip", "Bake an Animation Sequence for this Skeletal Mesh. Note it will create it based upon the Sequencer Display Range and Display Frame Rate"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HandleCreateAnimationSequence, SkelMeshComp,Skeleton, ObjectBindings[0], false)),
				NAME_None,
				EUserInterfaceActionType::Button);

			if (CanCreatePoseAsset(ObjectBindings[0]))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("CreatePoseAsset", "Bake Pose Asset"),
					LOCTEXT("CreatePoseAsset_ToolTip", "Bake Animation from current Pose"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateRaw(this, &FCommonAnimationTrackEditor::HandleCreatePoseAsset, ObjectBindings[0])),
					NAME_None,
					EUserInterfaceActionType::Button);
			}

			MenuBuilder.EndSection();
		}
		else
		{
			//maybe topmost actor that has multiple skel mesh components so still do stuff
			TArray<USkeletalMeshComponent*> SkelMeshComponents;
			TArray<FGuid> BindingIds;
			if(AcquireAllBoundSkeletalMeshObjectsFromObjectGuid(ObjectBindings[0], GetSequencer(),SkelMeshComponents, BindingIds))
			{
				AddAutoBakedMenus(BindingIds);
			}
		}
	}
}

void FCommonAnimationTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (!ObjectClass || (!ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) && !ObjectClass->IsChildOf(AActor::StaticClass()) && !ObjectClass->IsChildOf(UChildActorComponent::StaticClass())))
	{
		return;
	}

	// Collect all animation menu providers and sort by priority
	TArray<IMovieSceneAnimMixerItemMenuProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimMixerItemMenuProvider>(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName());
	Providers.Sort([](IMovieSceneAnimMixerItemMenuProvider& A, IMovieSceneAnimMixerItemMenuProvider& B)
	{
		return A.GetObjectBindingAnimationMenuPriority() < B.GetObjectBindingAnimationMenuPriority();
	});

	// Check if the Animation Mixer plugin is enabled
	const bool bAnimationMixerEnabled = FModuleManager::Get().IsModuleLoaded(TEXT("MovieSceneAnimMixerEditor"));

	if (bAnimationMixerEnabled)
	{
		// Create an "Animation" submenu containing all animation-related track options
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddAnimationSubmenu", "Animation"),
			NSLOCTEXT("Sequencer", "AddAnimationSubmenuTooltip", "Add animation-related tracks."),
			FNewMenuDelegate::CreateLambda([Providers, ObjectBindings, ObjectClass](FMenuBuilder& SubMenuBuilder)
			{
				for (IMovieSceneAnimMixerItemMenuProvider* Provider : Providers)
				{
					Provider->PopulateObjectBindingAnimationMenu(SubMenuBuilder, ObjectBindings, ObjectClass, /*bIsInsideSubmenu=*/ true);
				}
			})
		);
	}
	else
	{
		// Add animation menu entries directly to the top-level menu
		for (IMovieSceneAnimMixerItemMenuProvider* Provider : Providers)
		{
			Provider->PopulateObjectBindingAnimationMenu(MenuBuilder, ObjectBindings, ObjectClass, /*bIsInsideSubmenu=*/ false);
		}
	}
}

TSharedRef<SWidget> FCommonAnimationTrackEditor::BuildAddAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel)
{
	TViewModelPtr<FViewModel> ViewModel = WeakViewModel.Pin();
	if (!ViewModel)
	{
		return SNullWidget::NullWidget;
	}

	TViewModelPtr<ITrackExtension> TrackModel = ViewModel->FindAncestorOfType<ITrackExtension>(true);

	if (!TrackModel)
	{
		return SNullWidget::NullWidget;
	}

	UMovieSceneTrack* Track = TrackModel->GetTrack();

	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);


	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarp_Label", "Time Warp"));
	{
		FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, WeakViewModel);
	}
	MenuBuilder.EndSection();


	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AddAnimation_Label", "Add Animation"));
	{
		AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FCommonAnimationTrackEditor::BuildAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneTrack* Track)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track);

	return MenuBuilder.MakeWidget();
}

bool FCommonAnimationTrackEditor::ShouldFilterAsset(const FAssetData& AssetData)
{
	// we don't want montage
	if (AssetData.AssetClassPath == UAnimMontage::StaticClass()->GetClassPathName())
	{
		return true;
	}

	const FString EnumString = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
	if (EnumString.IsEmpty())
	{
		return false;
	}

	UEnum* AdditiveTypeEnum = StaticEnum<EAdditiveAnimationType>();
	return ((EAdditiveAnimationType)AdditiveTypeEnum->GetValueByName(*EnumString) == AAT_RotationOffsetMeshSpace);
}

void FCommonAnimationTrackEditor::AddAnimationSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, USkeleton* Skeleton, UMovieSceneTrack* Track, int32 RowIndex)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FCommonAnimationTrackEditor::OnAnimationAssetSelected, ObjectBindings, Track, RowIndex);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FCommonAnimationTrackEditor::OnAnimationAssetEnterPressed, ObjectBindings, Track, RowIndex);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequenceBase::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnShouldFilterAsset.BindRaw(this, &FCommonAnimationTrackEditor::FilterAnimSequences, Skeleton);
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}

bool FCommonAnimationTrackEditor::FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton)
{
	if (ShouldFilterAsset(AssetData))
	{
		return true;
	}

	if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData) == false)
	{
		return true;
	}

	return false;
}

void FCommonAnimationTrackEditor::OnAnimationAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, int32 RowIndex)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SelectedObject && SelectedObject->IsA(UAnimSequenceBase::StaticClass()) && SequencerPtr.IsValid())
	{
		UAnimSequenceBase* AnimSequence = CastChecked<UAnimSequenceBase>(AssetData.GetAsset());

		const FScopedTransaction Transaction(LOCTEXT("AddAnimation_Transaction", "Add Animation"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FCommonAnimationTrackEditor::AddKeyInternal, Object, AnimSequence, Track, RowIndex));
		}
	}
}

void FCommonAnimationTrackEditor::OnAnimationAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, int32 RowIndex)
{
	if (AssetData.Num() > 0)
	{
		OnAnimationAssetSelected(AssetData[0].GetAsset(), ObjectBindings, Track, RowIndex);
	}
}


FKeyPropertyResult FCommonAnimationTrackEditor::AddKeyInternal( FFrameNumber KeyTime, UObject* Object, class UAnimSequenceBase* AnimSequence, UMovieSceneTrack* Track, int32 RowIndex )
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
		UMovieSceneCommonAnimationTrack* SkelAnimTrack = Cast<UMovieSceneCommonAnimationTrack>(Track);
		FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectHandle);

		// Add a track if no track was specified or if the track specified doesn't belong to the tracks of the targeted guid
		if (!SkelAnimTrack || (Binding && !Binding->GetTracks().Contains(SkelAnimTrack)))
		{
			SkelAnimTrack = CastChecked<UMovieSceneCommonAnimationTrack>(AddTrack(MovieScene, ObjectHandle, GetTrackClass().Get(), NAME_None), ECastCheckedType::NullAllowed);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(SkelAnimTrack))
		{
			SkelAnimTrack->Modify();

			UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(SkelAnimTrack->AddNewAnimationOnRow(KeyTime, AnimSequence, RowIndex));
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			// Init the slot name on the new section if necessary
			if (USkeletalMeshComponent* SkeletalMeshComponent = AcquireSkeletalMeshFromObjectGuid(ObjectHandle, GetSequencer()))
			{
				if (TSubclassOf<UAnimInstance> AnimInstanceClass = SkeletalMeshComponent->GetAnimClass())
				{
					if (UAnimInstance* AnimInstance = AnimInstanceClass->GetDefaultObject<UAnimInstance>())
					{
						if (AnimInstance->Implements<USequencerAnimationOverride>())
						{
							TScriptInterface<ISequencerAnimationOverride> SequencerAnimOverride = AnimInstance;
							if (SequencerAnimOverride.GetObject())
							{
								TArray<FName> SlotNameOptions = ISequencerAnimationOverride::Execute_GetSequencerAnimSlotNames(SequencerAnimOverride.GetObject());
								if (SlotNameOptions.Num() > 0)
								{
									NewSection->Params.SlotName = SlotNameOptions[0];
								}
							}
						}
					}
				}
			}

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}

TSharedPtr<SWidget> FCommonAnimationTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBinding, GetSequencer());

	if (Skeleton)
	{
		FOnGetContent HandleGetAddButtonContent = FOnGetContent::CreateSP(this, &FCommonAnimationTrackEditor::BuildAddAnimationSubMenu, ObjectBinding, Skeleton, Params.ViewModel.AsWeak());
		return UE::Sequencer::MakeAddButton(LOCTEXT("AnimationText", "Animation"), HandleGetAddButtonContent, Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}
}

bool FCommonAnimationTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	  
	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return false;
	}
	
	if (!DragDropParams.TargetObjectGuid.IsValid())
	{
		return false;
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return false;
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return false;
	}

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(DragDropParams.TargetObjectGuid, SequencerPtr, false);

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(AssetData.GetAsset());

		const bool bValidAnimSequence = AnimSequence && AnimSequence->CanBeUsedInComposition();

		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			USkeleton* Skeleton = GetSkeletonFromComponent(SkeletalMeshComponent);
			if (bValidAnimSequence && Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();
				FFrameNumber LengthInFrames = TickResolution.AsFrameNumber(AnimSequence->GetPlayLength());
				DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, DragDropParams.FrameNumber + LengthInFrames);
				return true;
			}
		}
	}

	return false;
}


FReply FCommonAnimationTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return FReply::Unhandled();
	}
	
	if (!DragDropParams.TargetObjectGuid.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return FReply::Unhandled();
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return FReply::Unhandled();
	}

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(DragDropParams.TargetObjectGuid, SequencerPtr, false);

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(AssetData.GetAsset());
		const bool bValidAnimSequence = AnimSequence && AnimSequence->CanBeUsedInComposition();

		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			USkeleton* Skeleton = GetSkeletonFromComponent(SkeletalMeshComponent);

			if (bValidAnimSequence && Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(DragDropParams.TargetObjectGuid) : nullptr;

				AnimatablePropertyChanged( FOnKeyProperty::CreateRaw(this, &FCommonAnimationTrackEditor::AddKeyInternal, BoundObject, AnimSequence, DragDropParams.Track.Get(), DragDropParams.RowIndex));

				bAnyDropped = true;
			}
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
