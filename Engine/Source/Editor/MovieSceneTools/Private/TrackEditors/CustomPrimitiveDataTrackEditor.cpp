// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/CustomPrimitiveDataTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Components/DecalComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Sections/MovieSceneCustomPrimitiveDataSection.h"
#include "Tracks/MovieSceneCustomPrimitiveDataTrack.h"
#include "Sections/ParameterSection.h"
#include "SequencerUtilities.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Modules/ModuleManager.h"
#include "MaterialEditorModule.h"
#include "Engine/Selection.h"
#include "ISequencerModule.h"
#include "Components/MeshComponent.h"
#include "Materials/MaterialParameters.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSpacer.h"


#define LOCTEXT_NAMESPACE "CustomPrimitiveDataTrackEditor"


FCustomPrimitiveDataTrackEditor::FCustomPrimitiveDataTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerSection> FCustomPrimitiveDataTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneCustomPrimitiveDataSection* CustomPrimitiveDataSection = Cast<UMovieSceneCustomPrimitiveDataSection>(&SectionObject);
	checkf(CustomPrimitiveDataSection != nullptr, TEXT("Unsupported section type."));

	return MakeShareable(new FParameterSection(*CustomPrimitiveDataSection, GetSequencer()));
}

FText FCustomPrimitiveDataTrackEditor::GetDisplayName() const
{
	return LOCTEXT("CustomPrimitiveDataTrackEditor_DisplayName", "Custom Primitive Data");
}

TSharedPtr<SWidget> FCustomPrimitiveDataTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneCustomPrimitiveDataTrack* CPDTrack = Cast<UMovieSceneCustomPrimitiveDataTrack>(Track);
	FOnGetContent MenuContent = FOnGetContent::CreateSP(this, &FCustomPrimitiveDataTrackEditor::OnGetAddMenuContent, ObjectBinding, CPDTrack, Params.TrackInsertRowIndex);

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddParameterButton", "Parameter"), MenuContent, Params.ViewModel);
}


TSharedRef<SWidget> FCustomPrimitiveDataTrackEditor::OnGetAddMenuContent(FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, int32 TrackInsertRowIndex)
{
	// IF this is supported, allow creating other sections with different blend types, and put
	// the material parameters after a separator. Otherwise, just show the parameters menu.
	const FMovieSceneBlendTypeField SupportedBlendTypes = CPDTrack->GetSupportedBlendTypes();

	if (SupportedBlendTypes.Num() > 1)
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();
		FSequencerUtilities::PopulateMenu_CreateNewSection(MenuBuilder, TrackInsertRowIndex, CPDTrack, WeakSequencer);

		MenuBuilder.AddSeparator();

		OnBuildAddParameterMenu(MenuBuilder, ObjectBinding, CPDTrack);

		return MenuBuilder.MakeWidget();
	}
	else
	{
		return OnGetAddParameterMenuContent(ObjectBinding, CPDTrack);
	}
}

TSharedRef<SWidget> FCustomPrimitiveDataTrackEditor::OnGetAddParameterMenuContent(FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack)
{
	FMenuBuilder AddParameterMenuBuilder(true, nullptr);

	OnBuildAddParameterMenu(AddParameterMenuBuilder, ObjectBinding, CPDTrack);
	return AddParameterMenuBuilder.MakeWidget();
}

void FCustomPrimitiveDataTrackEditor::OnBuildAddParameterMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack)
{
	check(CPDTrack);

	UObject* Object = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBinding);
	if (!Object)
	{
		return;
	}

	UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object);
	if (!Component)
	{
		return;
	}

	auto OnStartIndexChanged = [this](int32 NewValue) 
	{ 
		StartIndex = NewValue; 
	};
	FGetStartIndexDelegate GetStartIndex = [this]() { return StartIndex; };

	TSortedMap<uint8, TArray<FCustomPrimitiveDataMaterialParametersData>> CPDData;
	CPDTrack->GetCPDMaterialData(*GetSequencer(), ObjectBinding, GetSequencer()->GetFocusedTemplateID(), CPDData);
	if (!CPDData.IsEmpty())
	{
		auto GetMaterialParameterLayerName = [](const FMaterialLayersFunctions& Layers, const FMaterialParameterInfo& InParameterInfo)
		{
			FString LayerName;
			if (Layers.EditorOnly.LayerNames.IsValidIndex(InParameterInfo.Index))
			{
				LayerName = Layers.GetLayerName(InParameterInfo.Index).ToString();
			}
			return LayerName;
		};
		auto GetMaterialParameterAssetName = [](const FMaterialLayersFunctions& Layers, const FMaterialParameterInfo& InParameterInfo)
		{
			FString AssetName;
			if (InParameterInfo.Association == EMaterialParameterAssociation::LayerParameter && Layers.Layers.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Layers[InParameterInfo.Index]->GetName();
			}
			else if (InParameterInfo.Association == EMaterialParameterAssociation::BlendParameter && Layers.Blends.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Blends[InParameterInfo.Index]->GetName();
			}
			return AssetName;
		};

		auto GetMaterialParameterDisplayName = [](const FMaterialParameterInfo& InParameterInfo, const FString& InMaterialName, const FString& InLayerName, const FString& InAssetName, uint8 CPDIndex)
		{
			FText DisplayName = FText::Format(LOCTEXT("MaterialAndParameterDisplayName", "{0} (Index: {1}, Asset: {2})"), FText::FromName(InParameterInfo.Name), FText::AsNumber(CPDIndex), FText::FromString(InMaterialName));
			if (!InLayerName.IsEmpty() && !InAssetName.IsEmpty())
			{
				DisplayName = FText::Format(LOCTEXT("MaterialParameterDisplayName", "{0} (Index: {1}, Asset: {2}.{3}.{4})"), 
					FText::FromName(InParameterInfo.Name),
					FText::AsNumber(CPDIndex),
					FText::FromString(InMaterialName),
					FText::FromString(InLayerName), 
					FText::FromString(InAssetName));
			}
			return DisplayName;
		};

		MenuBuilder.BeginSection("UsedInMaterials", LOCTEXT("CustomPrimitiveDataFromMaterialParameters", "Used In Material Parameters"));
		{
			for (const auto& CPDDataMapEntry : CPDData)
			{
				uint8 CPDIndex = CPDDataMapEntry.Key;

				FGetStartIndexDelegate GetCPDIndex = [CPDIndex] { return CPDIndex; };
				for (const FCustomPrimitiveDataMaterialParametersData& Data : CPDDataMapEntry.Value)
				{
					FString MaterialName = Data.MaterialAsset.IsValid() ? Data.MaterialAsset->GetName() : FString();
					FMaterialLayersFunctions Layers;
					if (Data.MaterialAsset.IsValid())
					{
						Data.MaterialAsset->GetMaterialLayers(Layers);
					}
					FString LayerName = GetMaterialParameterLayerName(Layers, Data.ParameterInfo);
					FString AssetName = GetMaterialParameterAssetName(Layers, Data.ParameterInfo);
					FText DisplayName = GetMaterialParameterDisplayName(Data.ParameterInfo, MaterialName, LayerName, AssetName, CPDIndex);

					if (Data.MaterialParameterType == EMaterialParameterType::Vector)
					{
						MenuBuilder.AddMenuEntry(
							DisplayName,
							TAttribute<FText>::CreateLambda([this, CPDTrack, GetCPDIndex]()
								{
									if (CanAddParameter(CPDTrack, GetCPDIndex, 4))
									{
										return FText::Format(LOCTEXT("CustomPrimitiveDataMaterialTooltipVector", "Add a track for this vector/color parameter starting at index {0}"), FText::AsNumber(GetCPDIndex()));
									}
									else
									{
										return FText::Format(LOCTEXT("CustomPrimitiveDataMaterialTooltipVectorInvalid", "Can't add a vector/color parameter starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(GetCPDIndex()));
									}
								}),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddColorParameter, Component, CPDTrack, GetCPDIndex),
								FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetCPDIndex, 4))
						);
					}
					else
					{
						MenuBuilder.AddMenuEntry(
							DisplayName,
							TAttribute<FText>::CreateLambda([this, CPDTrack, GetCPDIndex]()
								{
									if (CanAddParameter(CPDTrack, GetCPDIndex, 1))
									{
										return FText::Format(LOCTEXT("CustomPrimitiveDataMaterialTooltipScalar", "Add a track for this scalar parameter starting at index {0}"), FText::AsNumber(GetCPDIndex()));
									}
									else
									{
										return FText::Format(LOCTEXT("CustomPrimitiveDataMaterialTooltipScalarInvalid", "Can't add a scalar starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(GetCPDIndex()));
									}
								}),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddScalarParameter, Component, CPDTrack, GetCPDIndex),
								FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetCPDIndex, 1))
						);
					}
				}
			}
		}
		MenuBuilder.EndSection();
	}
	MenuBuilder.BeginSection("FromStartIndex", LOCTEXT("CustomPrimitiveDataFromStartIndex", "From Start Index"));
	{
		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(50.f)
				[
					SNew(SSpinBox<int32>)
					.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.OnValueCommitted_Lambda([=](int32 Value, ETextCommit::Type CommitType) { OnStartIndexChanged(Value); })
					.OnValueChanged_Lambda(OnStartIndexChanged)
					.MinValue(0)
					.MaxValue(35)
					.Value_Lambda([=]() -> int32 { return GetStartIndex(); })
				]
			],
			LOCTEXT("StartIndexText", "Custom Primitive Data Start Index")
			);


		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomPrimitiveDataScalar", "Scalar Parameter"),
			TAttribute<FText>::CreateLambda([this, CPDTrack, GetStartIndex]()
				{
					if (CanAddParameter(CPDTrack, GetStartIndex, 1))
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipScalar", "Add a track for a new scalar parameter starting at index {0}"), FText::AsNumber(StartIndex));
					}
					else
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipScalarInvalid", "Can't add a scalar parameter starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(StartIndex));
					}
				}), 
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddScalarParameter, Component, CPDTrack, GetStartIndex),
				FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetStartIndex, 1))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomPrimitiveDataVector2D", "Vector2D Parameter"),
			TAttribute<FText>::CreateLambda([this, CPDTrack, GetStartIndex]()
				{
					if (CanAddParameter(CPDTrack, GetStartIndex, 2))
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipVector2D", "Add a track for a new Vector2D parameter starting at index {0}"), FText::AsNumber(StartIndex));
					}
					else
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipVector2DInvalid", "Can't add a Vector2D parameter starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(StartIndex));
					}
				}), 
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddVector2DParameter, Component, CPDTrack, GetStartIndex),
				FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetStartIndex, 2))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomPrimitiveDataVector", "Vector Parameter"),
			TAttribute<FText>::CreateLambda([this, CPDTrack, GetStartIndex]()
				{
					if (CanAddParameter(CPDTrack, GetStartIndex, 3))
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipVector", "Add a track for a new Vector parameter starting at index {0}"), FText::AsNumber(StartIndex));
					}
					else
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipVectorInvalid", "Can't add a Vector parameter starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(StartIndex));
					}
				}),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddVectorParameter, Component, CPDTrack, GetStartIndex),
				FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetStartIndex, 3))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomPrimitiveDataColor", "Color Parameter"),
			TAttribute<FText>::CreateLambda([this, CPDTrack, GetStartIndex]()
				{
					if (CanAddParameter(CPDTrack, GetStartIndex, 4))
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipColor", "Add a track for a new color parameter starting at index {0}"), FText::AsNumber(StartIndex));
					}
					else
					{
						return FText::Format(LOCTEXT("CustomPrimitiveDataNewMaterialTooltipVectorColorInvalid", "Can't add a color parameter starting at index {0} as it overlaps with other parameters using that index."), FText::AsNumber(StartIndex));
					}
				}), 
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::AddColorParameter, Component, CPDTrack, GetStartIndex),
				FCanExecuteAction::CreateRaw(this, &FCustomPrimitiveDataTrackEditor::CanAddParameter, CPDTrack, GetStartIndex, 4))
		);
	}
	MenuBuilder.EndSection();
}


void FCustomPrimitiveDataTrackEditor::AddScalarParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate)
{
	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddScalarParameter", "Add scalar parameter"));
	float ParameterValue = 0.0f;
	const TArray<float>& CustomPrimitiveData = Component->GetCustomPrimitiveData().Data;
	uint8 PrimitiveDataIndex = GetStartIndexDelegate();
	if (CustomPrimitiveData.IsValidIndex(PrimitiveDataIndex))
	{
		ParameterValue = CustomPrimitiveData[PrimitiveDataIndex];
	}
	CPDTrack->Modify();
	CPDTrack->AddScalarParameterKey(PrimitiveDataIndex, KeyTime, ParameterValue, GetSequencer()->GetKeyInterpolation());

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FCustomPrimitiveDataTrackEditor::AddVector2DParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate)
{
	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddVector2DParameter", "Add Vector2D parameter"));
	FVector2D ParameterValue = FVector2D(0.0f, 0.0f);
	const TArray<float>& CustomPrimitiveData = Component->GetCustomPrimitiveData().Data;
	uint8 PrimitiveDataStartIndex = GetStartIndexDelegate();
	if (CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex) && CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex + 1))
	{
		ParameterValue.X = CustomPrimitiveData[PrimitiveDataStartIndex];
		ParameterValue.Y = CustomPrimitiveData[PrimitiveDataStartIndex + 1];
	}
	CPDTrack->Modify();
	CPDTrack->AddVector2DParameterKey(PrimitiveDataStartIndex, KeyTime, ParameterValue, GetSequencer()->GetKeyInterpolation());

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FCustomPrimitiveDataTrackEditor::AddVectorParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate)
{
	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddVectorParameter", "Add Vector parameter"));
	FVector ParameterValue = FVector(0.0f, 0.0f, 0.0f);
	const TArray<float>& CustomPrimitiveData = Component->GetCustomPrimitiveData().Data;
	uint8 PrimitiveDataStartIndex = GetStartIndexDelegate();
	if (CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex) && CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex + 2))
	{
		ParameterValue.X = CustomPrimitiveData[PrimitiveDataStartIndex];
		ParameterValue.Y = CustomPrimitiveData[PrimitiveDataStartIndex + 1];
		ParameterValue.Z = CustomPrimitiveData[PrimitiveDataStartIndex + 2];
	}
	CPDTrack->Modify();
	CPDTrack->AddVectorParameterKey(PrimitiveDataStartIndex, KeyTime, ParameterValue, GetSequencer()->GetKeyInterpolation());

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FCustomPrimitiveDataTrackEditor::AddColorParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate)
{
	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddColorParameter", "Add Color parameter"));
	FLinearColor ParameterValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	const TArray<float>& CustomPrimitiveData = Component->GetCustomPrimitiveData().Data;
	uint8 PrimitiveDataStartIndex = GetStartIndexDelegate();
	if (CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex) && CustomPrimitiveData.IsValidIndex(PrimitiveDataStartIndex + 3))
	{
		ParameterValue.R = CustomPrimitiveData[PrimitiveDataStartIndex];
		ParameterValue.G = CustomPrimitiveData[PrimitiveDataStartIndex + 1];
		ParameterValue.B = CustomPrimitiveData[PrimitiveDataStartIndex + 2];
		ParameterValue.A = CustomPrimitiveData[PrimitiveDataStartIndex + 3];
	}
	CPDTrack->Modify();
	CPDTrack->AddColorParameterKey(PrimitiveDataStartIndex, KeyTime, ParameterValue, GetSequencer()->GetKeyInterpolation());

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

bool FCustomPrimitiveDataTrackEditor::CanAddParameter(UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate, int ParameterSize)
{
	check(CPDTrack);
	check(ParameterSize >= 1 && ParameterSize <= 4);
	UMovieSceneCustomPrimitiveDataSection* NearestSection = Cast< UMovieSceneCustomPrimitiveDataSection>(CPDTrack->GetSectionToKey());
	if (NearestSection == nullptr)
	{
		NearestSection = Cast<UMovieSceneCustomPrimitiveDataSection>(MovieSceneHelpers::FindNearestSectionAtTime(CPDTrack->GetAllSections(), GetTimeForKey(), INDEX_NONE));
	}
	if (NearestSection)
	{
		uint8 CPDStartIndex = GetStartIndexDelegate();
		uint64 ChannelsUsedBitmap = NearestSection->GetChannelsUsed();
		uint8 TestBit = 0;
		for (int i = CPDStartIndex; i < ParameterSize + CPDStartIndex; ++i)
		{
			TestBit |= ((uint64)1 << i);
		}
		return (CPDStartIndex >= 0 && CPDStartIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats - ParameterSize + 1)
			&& ((ChannelsUsedBitmap & TestBit) == 0);
	}
	return true;
}

TSharedRef<ISequencerTrackEditor> FCustomPrimitiveDataTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShareable(new FCustomPrimitiveDataTrackEditor(OwningSequencer));
}

bool FCustomPrimitiveDataTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneCustomPrimitiveDataTrack::StaticClass();
}

bool FCustomPrimitiveDataTrackEditor::GetDefaultExpansionState(UMovieSceneTrack* InTrack) const
{
	return true;
}

void FCustomPrimitiveDataTrackEditor::ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UPrimitiveComponent::StaticClass()))
	{
		Extender->AddMenuExtension(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection, EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &FCustomPrimitiveDataTrackEditor::ConstructObjectBindingTrackMenu, ObjectBindings));
	}
}

void FCustomPrimitiveDataTrackEditor::ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	UObject* Object = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBindings[0]);
	if (!Object)
	{
		return;
	}

	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Object);
	if (!PrimitiveComponent)
	{
		return;
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();

	const bool bAlreadyExists = MovieScene->FindTrack<UMovieSceneCustomPrimitiveDataTrack>(ObjectBindings[0]) != nullptr;
	if (!bAlreadyExists)
	{
		MenuBuilder.BeginSection("CustomPrimitiveData", LOCTEXT("CustomPrimitiveDataSection", "Custom Primitive Data"));
		{
			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("Sequencer", "AddCustomPrimitiveDataTrack", "Custom Primitive Data"),
				NSLOCTEXT("Sequencer", "AddCustomPrimitiveDataTrackTooltip", "Adds a track to animate custom primitive data"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FCustomPrimitiveDataTrackEditor::HandleAddCustomPrimitiveDataTrackExecute, PrimitiveComponent)
				)
			);
		}
		MenuBuilder.EndSection();
	}
}

void FCustomPrimitiveDataTrackEditor::HandleAddCustomPrimitiveDataTrackExecute(UPrimitiveComponent* Component)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddCPDTrack", "Add custom primitive data track"));

	MovieScene->Modify();

	FString ComponentName = Component->GetName();

	TArray<UActorComponent*> ActorComponents;
	ActorComponents.Add(Component);

	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (SelectedActors && SelectedActors->Num() > 0)
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = CastChecked<AActor>(*Iter);

			TArray<UActorComponent*> OutActorComponents;
			Actor->GetComponents(OutActorComponents);
			for (UActorComponent* ActorComponent : OutActorComponents)
			{
				if (ActorComponent->GetName() == ComponentName)
				{
					ActorComponents.AddUnique(ActorComponent);
				}
			}
		}
	}

	for (UActorComponent* ActorComponent : ActorComponents)
	{
		FGuid ObjectHandle = SequencerPtr->GetHandleToObject(ActorComponent);
		const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectHandle);
		
		const bool bAlreadyExists = MovieScene->FindTrack<UMovieSceneCustomPrimitiveDataTrack>(ObjectHandle) != nullptr;
		if (!bAlreadyExists)
		{
			MovieScene->AddTrack<UMovieSceneCustomPrimitiveDataTrack>(ObjectHandle);
		}
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}



#undef LOCTEXT_NAMESPACE
