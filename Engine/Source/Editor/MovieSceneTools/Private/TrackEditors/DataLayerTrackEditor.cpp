// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/DataLayerTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "Sections/MovieSceneDataLayerSection.h"
#include "Tracks/MovieSceneDataLayerTrack.h"
#include "MVVM/Views/ViewUtilities.h"
#include "LevelUtils.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneToolHelpers.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "SequencerSectionPainter.h"
#include "DataLayer/DataLayerEditorSubsystem.h"

#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "DataLayer/DataLayerDragDropOp.h"
#include "SDropTarget.h"

#define LOCTEXT_NAMESPACE "DataLayerTrackEditor"

struct FDataLayerSection
	: public ISequencerSection
	, public TSharedFromThis<FDataLayerSection>
{
	FDataLayerSection(TSharedPtr<ISequencer> InSequencer, UMovieSceneDataLayerSection* InSection)
		: SequencerPtr(InSequencer)
		, WeakSection(InSection)
	{}

	/*~ ISequencerSection */
	virtual UMovieSceneSection* GetSectionObject() override
	{
		return WeakSection.Get();
	}
	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override
	{
		return InPainter.PaintSectionBackground();
	}
	virtual float GetSectionHeight() const override
	{
		return 30.f;
	}
	virtual TSharedRef<SWidget> GenerateSectionWidget() override
	{
		return 
			SNew(SDropTarget)
			.OnAllowDrop(this, &FDataLayerSection::OnAllowDrop)
			.OnDropped(this, &FDataLayerSection::OnDrop)
			.Content()
			[
				SNew(SBox)
				.Padding(FMargin(4.f))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(this, &FDataLayerSection::GetVisibilityText)
							.ColorAndOpacity(this, &FDataLayerSection::GetTextColor)
							.TextStyle(FAppStyle::Get(), "NormalText.Important")
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(this, &FDataLayerSection::GetPrerollText)
						]
					]
				]
			];
	}

	FText GetSectionToolTip() const override
	{
		UMovieSceneDataLayerSection* Section = WeakSection.Get();
		UDataLayerEditorSubsystem* SubSystem = UDataLayerEditorSubsystem::Get();

		if (SubSystem && Section)
		{
			FString LayerName;

			const TArray<UDataLayerAsset*>& DataLayerAssets = Section->GetDataLayerAssets();
			for (int32 Index = 0; Index < DataLayerAssets.Num(); ++Index)
			{
				UDataLayerInstance* DataLayerInstance = SubSystem->GetDataLayerInstance(DataLayerAssets[Index]);
				if (DataLayerInstance)
				{
					LayerName += DataLayerInstance->GetDataLayerFullName();
				}
				else
				{
					LayerName += FText::Format(LOCTEXT("UnknownDataLayer", "**invalid: {0}**"), FText::FromString(DataLayerAssets[Index]->GetFullName())).ToString();
				}

				if (Index < DataLayerAssets.Num() - 1)
				{
					LayerName += TEXT("\n");
				}
			}

			return FText::FromString(LayerName);
		}

		return FText();
	}

	FText GetVisibilityText() const
	{
		UMovieSceneDataLayerSection* Section = WeakSection.Get();
		if (Section)
		{
			switch (Section->GetDesiredState())
			{
			case EDataLayerRuntimeState::Unloaded:  return LOCTEXT("VisibilityText_Unloaded", "Unload");
			case EDataLayerRuntimeState::Loaded:    return LOCTEXT("VisibilityText_Loaded", "Load");
			default: break;
			}
		}

		return LOCTEXT("VisibilityText_Activated", "Activate");
	}

	FText GetPrerollText() const
	{
		UMovieSceneDataLayerSection* Section = WeakSection.Get();
		if (Section && Section->GetPreRollFrames() > 0)
		{
			switch (Section->GetPrerollState())
			{
			case EDataLayerRuntimeState::Unloaded:  return LOCTEXT("PrerollText_Unloaded", "(Unloaded over time in pre/post roll)");
			case EDataLayerRuntimeState::Loaded:    return LOCTEXT("PrerollText_Loaded", "(Loaded over time in preroll)");
			case EDataLayerRuntimeState::Activated:	return LOCTEXT("PrerollText_Activated", "(Activated over time in preroll)");
			}
		}

		return FText();
	}

	FSlateColor GetTextColor() const
	{
		UMovieSceneDataLayerSection* Section = WeakSection.Get();
		if (Section)
		{
			switch (Section->GetDesiredState())
			{
			case EDataLayerRuntimeState::Unloaded:  return FStyleColors::AccentRed;
			case EDataLayerRuntimeState::Loaded:    return FStyleColors::AccentBlue;
			case EDataLayerRuntimeState::Activated: return FStyleColors::AccentGreen;
			}
		}
		return FStyleColors::Foreground;
	}

	bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
	{
		if (DragDropOperation->IsOfType<FDataLayerDragDropOp>() && StaticCastSharedPtr<FDataLayerDragDropOp>(DragDropOperation)->DataLayerInstances.Num() > 0)
		{
			return true;
		}
		else if (DragDropOperation->IsOfType<FAssetDragDropOp>())
		{
			return true;
		}
		return false;
	}

	FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
	{
		UMovieSceneDataLayerSection* Section = WeakSection.Get();
		if (!Section)
		{
			return FReply::Handled();
		}

		UDataLayerEditorSubsystem* SubSystem = UDataLayerEditorSubsystem::Get();
		if (!SubSystem)
		{
			return FReply::Handled();
		}

		TArray<UDataLayerAsset*> DataLayerAssets = Section->GetDataLayerAssets();
		if (TSharedPtr<FDataLayerDragDropOp> DataLayerDragDropOp = InDragDropEvent.GetOperationAs<FDataLayerDragDropOp>())
		{
			if (DataLayerDragDropOp->DataLayerInstances.Num() > 0)
			{
				for (const TWeakObjectPtr<UDataLayerInstance>& DataLayerInstance : DataLayerDragDropOp->DataLayerInstances)
				{
					if (DataLayerInstance.IsValid())
					{
						UDataLayerInstanceWithAsset* DataLayerWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance.Get());
						UDataLayerAsset* DataLayerAsset = DataLayerWithAsset ? const_cast<UDataLayerAsset*>(DataLayerWithAsset->GetAsset()) : nullptr;

						if (DataLayerAsset)
						{
							DataLayerAssets.AddUnique(DataLayerAsset);
						}
					}
				}
			}
		}
		else if (TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{
			UMovieSceneSequence* FocusedSequence = SequencerPtr.IsValid() ? SequencerPtr.Pin()->GetFocusedMovieSceneSequence() : nullptr;

			for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
			{
				if (UDataLayerAsset* DataLayerAsset = Cast<UDataLayerAsset>(AssetData.GetAsset()))
				{	
					if (MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
					{
						DataLayerAssets.AddUnique(DataLayerAsset);
					}
				}
			}
		}
		else
		{	
			return FReply::Handled();
		}

		FScopedTransaction Transaction(LOCTEXT("TransactionText", "Add Data Layer(s) to Data Layer Section"));
		Section->Modify();
		Section->SetDataLayerAssets(DataLayerAssets);

		return FReply::Handled();
	}

private:

	TWeakPtr<ISequencer> SequencerPtr;
	TWeakObjectPtr<UMovieSceneDataLayerSection> WeakSection;
};

FDataLayerTrackEditor::FDataLayerTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{}

TSharedRef<ISequencerTrackEditor> FDataLayerTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShared<FDataLayerTrackEditor>(InSequencer);
}

bool FDataLayerTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneDataLayerTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FDataLayerTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneDataLayerTrack::StaticClass();
}

const FSlateBrush* FDataLayerTrackEditor::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("Sequencer.Tracks.DataLayer");
}

FText FDataLayerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("DataLayerTrackEditor_DisplayName", "Data Layer");
}

TSharedRef<ISequencerSection> FDataLayerTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneDataLayerSection* DataLayerSection = Cast<UMovieSceneDataLayerSection>(&SectionObject);
	check(SupportsType(SectionObject.GetOuter()->GetClass()) && DataLayerSection != nullptr);

	return MakeShared<FDataLayerSection>(GetSequencer(), DataLayerSection);
}

void FDataLayerTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Data Layer Track"),
		LOCTEXT("AddTrackToolTip", "Adds a new track that can load, activate or unload Data Layers in a World Partition world."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.DataLayer"),
		FUIAction(FExecuteAction::CreateRaw(this, &FDataLayerTrackEditor::HandleAddTrack)));
}

TSharedPtr<SWidget> FDataLayerTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return UE::Sequencer::MakeAddButton(
		LOCTEXT("AddDataLayer_ButtonLabel", "Data Layer"),
		FOnGetContent::CreateSP(this, &FDataLayerTrackEditor::BuildAddDataLayerMenu, Track),
		Params.ViewModel);
}

UMovieSceneDataLayerSection* FDataLayerTrackEditor::AddNewSection(UMovieScene* MovieScene, UMovieSceneTrack* DataLayerTrack, EDataLayerRuntimeState DesiredState)
{
	using namespace UE::MovieScene;

	const FScopedTransaction Transaction(LOCTEXT("AddDataLayerSection_Transaction", "Add Data Layer"));
	DataLayerTrack->Modify();

	UMovieSceneDataLayerSection* DataLayerSection = CastChecked<UMovieSceneDataLayerSection>(DataLayerTrack->CreateNewSection());
	DataLayerSection->SetDesiredState(DesiredState);

	// By default, activated states will preroll to loaded
	if (DesiredState == EDataLayerRuntimeState::Activated)
	{
		DataLayerSection->SetPrerollState(EDataLayerRuntimeState::Loaded);
	}
	else
	{
		DataLayerSection->SetPrerollState(DesiredState);
	}

	TRange<FFrameNumber> SectionRange = MovieScene->GetPlaybackRange();
	DataLayerSection->InitialPlacement(DataLayerTrack->GetAllSections(), MovieScene->GetPlaybackRange().GetLowerBoundValue(), DiscreteSize(MovieScene->GetPlaybackRange()), true);
	DataLayerTrack->AddSection(*DataLayerSection);

	// Set some default preroll for activated or loaded data layers
	if (DesiredState != EDataLayerRuntimeState::Unloaded)
	{
		DataLayerSection->SetPreRollFrames((2.f * MovieScene->GetTickResolution()).RoundToFrame().Value);
	}

	return DataLayerSection;
}

void FDataLayerTrackEditor::HandleAddTrack()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr || FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddDataLayerTrack_Transaction", "Add Data Layer Track"));
	FocusedMovieScene->Modify();

	UMovieSceneDataLayerTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneDataLayerTrack>();
	checkf(NewTrack, TEXT("Failed to create new data layer track."));

	UMovieSceneDataLayerSection* NewSection = AddNewSection(FocusedMovieScene, NewTrack, EDataLayerRuntimeState::Activated);
	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

TSharedRef<SWidget> FDataLayerTrackEditor::BuildAddDataLayerMenu(UMovieSceneTrack* DataLayerTrack)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddActivatedDataLayer", "Activated"),
		LOCTEXT("AddActivatedDataLayer_Tip", "Instruct a data layer to be loaded and active."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(
			this, &FDataLayerTrackEditor::HandleAddNewSection, DataLayerTrack, EDataLayerRuntimeState::Activated)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddLoadedDataLayer", "Loaded"),
		LOCTEXT("AddLoadedDataLayer_Tip", "Instruct a data layer to be loaded (but not active)."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(
			this, &FDataLayerTrackEditor::HandleAddNewSection, DataLayerTrack, EDataLayerRuntimeState::Loaded)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddUnloadedDataLayer", "Unloaded"),
		LOCTEXT("AddUnloadedDataLayer_Tip", "Instruct a data layer to be unloaded for a duration."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(
			this, &FDataLayerTrackEditor::HandleAddNewSection, DataLayerTrack, EDataLayerRuntimeState::Unloaded)));

	return MenuBuilder.MakeWidget();
}


void FDataLayerTrackEditor::HandleAddNewSection(UMovieSceneTrack* DataLayerTrack, EDataLayerRuntimeState DesiredState)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene)
	{
		UMovieSceneDataLayerSection* NewSection = AddNewSection(FocusedMovieScene, DataLayerTrack, DesiredState);

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		GetSequencer()->EmptySelection();
		GetSequencer()->SelectSection(NewSection);
		GetSequencer()->ThrobSectionSelection();
	}
}

#undef LOCTEXT_NAMESPACE
