// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "MovieSceneTrackEditor.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

class FMenuBuilder;
class UMovieSceneDataLayerSection;

/**
 * A sequencer track editor for data layer tracks.
 */
class FDataLayerTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	FDataLayerTrackEditor(TSharedRef<ISequencer> InSequencer);

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	//virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

private:
	/** Adds a new section which spans the length of the owning movie scene with the specified desired state. */
	UMovieSceneDataLayerSection* AddNewSection(UMovieScene* MovieScene, UMovieSceneTrack* DataLayerTrack, EDataLayerRuntimeState DesiredState);

	/** Handles when the add track menu item is activated. */
	void HandleAddTrack();

	/** Builds the add data layer menu which is displayed on the track. */
	TSharedRef<SWidget> BuildAddDataLayerMenu(UMovieSceneTrack* DataLayerTrack);

	/** Called to add a new section with a desired state. */
	void HandleAddNewSection(UMovieSceneTrack* DataLayerTrack, EDataLayerRuntimeState DesiredState);
};
