// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/DefaultLinkedAnimTrackProvider.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Animation/AnimSequence.h"
#include "Baker/SequencerBakerSubsystem.h"
#include "Baker/SequencerBaker.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "DefaultLinkedAnimTrackProvider"

using namespace UE::Sequencer;

static bool DisableTrackHelper(UMovieSceneTrack* Track, bool bInEvalDisabled)
{
	bool bWasChanged = false;
	if (Track->GetMaxRowIndex() == 0)
	{
		if (Track->IsEvalDisabled() != bInEvalDisabled)
		{
			bWasChanged = true;
			Track->Modify();
			Track->SetEvalDisabled(bInEvalDisabled);
		}
	}
	else
	{
		for (int32 RowIndex = 0; RowIndex <= Track->GetMaxRowIndex(); ++RowIndex)
		{
			if (Track->IsRowEvalDisabled(RowIndex) != bInEvalDisabled)
			{
				bWasChanged = true;
				Track->Modify();
				Track->SetRowEvalDisabled(bInEvalDisabled, RowIndex);
			}
		}
	}
	return bWasChanged;
}

bool FDefaultLinkedAnimTrackProvider::CanHandleBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const
{
	// Default provider is the fallback — always handles
	return true;
}

UMovieSceneTrack* FDefaultLinkedAnimTrackProvider::GetLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || SequencerPtr->GetFocusedMovieSceneSequence() == nullptr ||
		SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
	{
		return nullptr;
	}
	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, BindingID);
	if (LinkedItems.Num() > 0)
	{
		UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
		TArray<UMovieSceneTrack*> AllTracks = MovieScene->FindTracks(UMovieSceneSkeletalAnimationTrack::StaticClass(), BindingID, NAME_None);
		for (UMovieSceneTrack* Track : AllTracks)
		{
			if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
			{
				bool bItMatches = false;
				const TArray<UMovieSceneSection*>& Sections = AnimTrack->GetAllSections();
				for (int32 Index = 0; Index < LinkedItems.Num(); ++Index)
				{
					FLevelSequenceAnimSequenceLinkItem* Item = LinkedItems[Index];
					if (Item && Item->bAutoBake == true)
					{
						UAnimSequence* LinkedAnimSequence = Item->ResolveAnimSequence();
						for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
						{
							if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Sections[SectionIndex]))
							{
								if (LinkedAnimSequence && LinkedAnimSequence == AnimSection->Params.Animation)
								{
									bItMatches = true;
									break;
								}
							}
						}
					}
				}
				if (bItMatches)
				{
					return AnimTrack;
				}
			}
		}
	}
	return nullptr;
}

UMovieSceneTrack* FDefaultLinkedAnimTrackProvider::GetOrCreateLinkedAnimTrack(
	const TWeakPtr<ISequencer>& InSequencer,
	FGuid BindingID,
	FCommonAnimationTrackEditor& TrackEditor)
{
	// First try read-only lookup
	if (UMovieSceneTrack* Track = GetLinkedAnimTrack(InSequencer, BindingID))
	{
		return Track;
	}

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || SequencerPtr->GetFocusedMovieSceneSequence() == nullptr ||
		SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
	{
		return nullptr;
	}

	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	FScopedTransaction MatchSection(LOCTEXT("CreateLinkedAnimTrack_Transaction", "Create Linked Anim Track"));
	MovieScene->Modify();
	FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingID);
	if (!Binding)
	{
		return nullptr;
	}

	UMovieSceneSkeletalAnimationTrack* AnimTrack = TrackEditor.FindEmptyAnimationTrack(BindingID);
	if (AnimTrack == nullptr)
	{
		AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(TrackEditor.AddTrack(MovieScene, BindingID, UMovieSceneSkeletalAnimationTrack::StaticClass(), NAME_None));
		
	}
	if (!AnimTrack)
	{
		return nullptr;
	}

	const FString TrackName = TEXT("Animation (Linked}");
	AnimTrack->SetDisplayName(FText::FromString(TrackName));
	AnimTrack->SetColorTint(UMovieSceneSkeletalAnimationTrack::LinkedAnimSeqTrackColor);

	AnimTrack->Modify();
	USkeletalMeshComponent* SkelMeshComp = FCommonAnimationTrackEditor::AcquireSkeletalMeshFromObjectGuid(BindingID, SequencerPtr);
	USkeleton* Skeleton = FCommonAnimationTrackEditor::AcquireSkeletonFromObjectGuid(BindingID, SequencerPtr);
	bool bNeedToCreateLinkedAnim = true;
	int32 LinkedItemNum = INDEX_NONE;
	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(SequencerPtr, BindingID);
	if (LinkedItems.Num() > 0)
	{
		for (int32 LinkedIndex = 0; LinkedIndex < LinkedItems.Num(); ++LinkedIndex)
		{
			FLevelSequenceAnimSequenceLinkItem* LinkedItem = LinkedItems[LinkedIndex];
			if (LinkedItem && LinkedItem->bUseCustomTimeRange == false)
			{
				bNeedToCreateLinkedAnim = false;
				LinkedItemNum = LinkedIndex;
			}
		}
	}
	if (bNeedToCreateLinkedAnim)
	{
		TrackEditor.CreateLinkedAnimationSequence(SkelMeshComp, Skeleton, BindingID, bNeedToCreateLinkedAnim);
		LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(SequencerPtr, BindingID);
		LinkedItemNum = LinkedItems.Num() - 1;
	}
	if (LinkedItems.Num() > 0 && LinkedItemNum != INDEX_NONE)
	{
		if (FLevelSequenceAnimSequenceLinkItem* LinkedItem = LinkedItems[LinkedItemNum])
		{
			UAnimSequence* AnimSequence = LinkedItem->ResolveAnimSequence();
			TOptional<TRange<FFrameNumber>> OptionalRange = SequencerPtr->GetSubSequenceRange();
			TRange<FFrameNumber> Range = OptionalRange.IsSet() ? OptionalRange.GetValue() : SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
			FFrameNumber Frame = Range.GetLowerBoundValue();
			UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(AnimTrack->AddNewAnimationOnRow(Frame, AnimSequence, LinkedItems.Num() - 1));
			if (NewSection)
			{
				LinkedItem->bUseCustomTimeRange = false;
				LinkedItem->bAutoBake = true;
				NewSection->SetRange(Range);
				AnimTrack->SetEvalDisabled(true);
			}
			return AnimTrack;
		}
	}

	return nullptr;
}

bool FDefaultLinkedAnimTrackProvider::IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bIsolate)
{
	bool bWasChanged = false;

	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid() || SequencerPtr->GetFocusedMovieSceneSequence() == nullptr ||
		SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
	{
		return bWasChanged;
	}
	TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, BindingID);
	if (LinkedItems.Num() > 0)
	{
		UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
		TArray<UMovieSceneTrack*> AllTracks = MovieScene->FindTracks(UMovieSceneTrack::StaticClass(), BindingID, NAME_None);
		for (UMovieSceneTrack* Track : AllTracks)
		{
			if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
			{
				const TArray<UMovieSceneSection*>& Sections = AnimTrack->GetAllSections();
				bool bItMatches = false;
				for (int32 Index = 0; Index < LinkedItems.Num(); ++Index)
				{
					FLevelSequenceAnimSequenceLinkItem* Item = LinkedItems[Index];
					if (Item && Item->bAutoBake == true)
					{
						for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
						{
							if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Sections[SectionIndex]))
							{
								UAnimSequence* LinkedAnimSequence = Item->ResolveAnimSequence();
								if (LinkedAnimSequence && LinkedAnimSequence == AnimSection->Params.Animation)
								{
									bItMatches = true;
									break;
								}
							}
						}
					}
				}
				if (bItMatches)
				{
					if (DisableTrackHelper(AnimTrack, !bIsolate))
					{
						bWasChanged = true;
					}
				}
				else
				{
					if (DisableTrackHelper(AnimTrack, bIsolate))
					{
						bWasChanged = true;
					}
				}
			}
			else
			{
				FString TrackName = Track->GetClass()->GetName();
				if (TrackName == FString("MovieSceneControlRigParameterTrack"))
				{
					if (DisableTrackHelper(Track, bIsolate))
					{
						bWasChanged = true;
					}
				}
			}
		}
		if (USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>())
		{
			if (TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr))
			{
				SequencerBaker->IgnoreLastChange();
			}
		}
	}
	return bWasChanged;
}

void FDefaultLinkedAnimTrackProvider::UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>();
		TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem ? SequencerBakeSubsystem->GetSequencerBaker(SequencerPtr) : nullptr;
		if (SequencerBaker.IsValid() == false)
		{
			return;
		}

		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();

			UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
			TOptional<TRange<FFrameNumber>> OptionalRange = SequencerPtr->GetSubSequenceRange();
			TRange<FFrameNumber> Range = OptionalRange.IsSet() ? OptionalRange.GetValue() : SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();

			TArray<FLevelSequenceAnimSequenceLinkItem*> LinkedItems = FCommonAnimationTrackEditor::GetLinkedAnimSequences(InSequencer, BindingID);
			if (LinkedItems.Num() > 0)
			{
				TArray<UMovieSceneTrack*> AllTracks = MovieScene->FindTracks(UMovieSceneTrack::StaticClass(), BindingID, NAME_None);
				for (UMovieSceneTrack* Track : AllTracks)
				{
					if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track))
					{
						const TArray<UMovieSceneSection*>& Sections = AnimTrack->GetAllSections();
						for (int32 Index = 0; Index < LinkedItems.Num(); ++Index)
						{
							FLevelSequenceAnimSequenceLinkItem* Item = LinkedItems[Index];
							if (Item && Item->bAutoBake == true &&  Item->bUseCustomTimeRange == false)
							{
								for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
								{
									if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Sections[SectionIndex]))
									{
										UAnimSequence* LinkedAnimSequence = Item->ResolveAnimSequence();
										if (LinkedAnimSequence && LinkedAnimSequence == AnimSection->Params.Animation)
										{
											if (AnimSection->GetRange() != Range)
											{
												AnimSection->Modify();
												AnimSection->SetRange(Range);
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

bool FDefaultLinkedAnimTrackProvider::DeleteLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID)
{
	TSharedPtr<ISequencer> SequencerPtr = InSequencer.Pin();
	if (!SequencerPtr.IsValid())
	{
		return false;
	}
	if (UMovieSceneTrack* Track = GetLinkedAnimTrack(InSequencer, BindingID))
	{
		if (UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
		{
			MovieScene->Modify();
			MovieScene->RemoveTrack(*Track);
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
