// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneLinkedAnimTrackProvider.h"

/**
 * Default implementation of IMovieSceneLinkedAnimTrackProvider.
 * Handles the standard case where the linked animation track is a standalone
 * UMovieSceneSkeletalAnimationTrack on the binding (not inside a mixer).
 */
class FDefaultLinkedAnimTrackProvider : public IMovieSceneLinkedAnimTrackProvider
{
public:

	//~ IMovieSceneLinkedAnimTrackProvider interface
	virtual int32 GetLinkedAnimTrackPriority() const override { return 100; }
	virtual bool CanHandleBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const override;
	virtual UMovieSceneTrack* GetLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const override;
	virtual UMovieSceneTrack* GetOrCreateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, UE::Sequencer::FCommonAnimationTrackEditor& TrackEditor) override;
	virtual bool IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bIsolate) override;
	virtual void UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) override;
	virtual bool DeleteLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) override;
};
