// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/EventTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "UObject/Package.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "IDetailCustomization.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "DetailLayoutBuilder.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "Widgets/Layout/SBox.h"
#include "Sections/EventSection.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/MovieSceneEventTriggerSection.h"
#include "Sections/MovieSceneEventRepeaterSection.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MovieSceneSequenceEditor.h"

#define LOCTEXT_NAMESPACE "FEventTrackEditor"


/* FEventTrackEditor static functions
 *****************************************************************************/

TSharedRef<ISequencerTrackEditor> FEventTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FEventTrackEditor(InSequencer));
}


TSharedRef<ISequencerSection> FEventTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	if (SectionObject.IsA<UMovieSceneEventSection>())
	{
		return MakeShared<FEventSection>(SectionObject, GetSequencer());
	}
	else if (SectionObject.IsA<UMovieSceneEventTriggerSection>())
	{
		return MakeShared<FEventTriggerSection>(SectionObject, GetSequencer());
	}
	else if (SectionObject.IsA<UMovieSceneEventRepeaterSection>())
	{
		return MakeShared<FEventRepeaterSection>(SectionObject, GetSequencer());
	}

	return MakeShared<FSequencerSection>(SectionObject);
}


/* FEventTrackEditor structors
 *****************************************************************************/

FEventTrackEditor::FEventTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }


/* ISequencerTrackEditor interface
 *****************************************************************************/

void FEventTrackEditor::AddEventSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewTriggerSection", "Trigger"),
		LOCTEXT("AddNewTriggerSectionTooltip", "Adds a new section that can trigger a specific event at a specific time"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FEventTrackEditor::HandleAddEventTrackMenuEntryExecute, ObjectBindings, UMovieSceneEventTriggerSection::StaticClass())
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddNewRepeaterSection", "Repeater"),
		LOCTEXT("AddNewRepeaterSectionTooltip", "Adds a new section that triggers an event every time it's evaluated"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FEventTrackEditor::HandleAddEventTrackMenuEntryExecute, ObjectBindings, UMovieSceneEventRepeaterSection::StaticClass())
		)
	);
}

FText FEventTrackEditor::GetDisplayName() const
{
	return LOCTEXT("EventTrackEditor_DisplayName", "Event");
}

void FEventTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	UMovieSceneSequence*       RootMovieSceneSequence = GetSequencer()->GetRootMovieSceneSequence();
	FMovieSceneSequenceEditor* SequenceEditor         = FMovieSceneSequenceEditor::Find(RootMovieSceneSequence);

	if (SequenceEditor && SequenceEditor->SupportsEvents(RootMovieSceneSequence))
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddEventTrack", "Event Track"),
			LOCTEXT("AddEventTooltip", "Adds a new event track that can trigger events on the timeline."),
			FNewMenuDelegate::CreateRaw(this, &FEventTrackEditor::AddEventSubMenu, TArray<FGuid>()),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Event")
		);
	}
}

void FEventTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	UMovieSceneSequence*       RootMovieSceneSequence = GetSequencer()->GetRootMovieSceneSequence();
	FMovieSceneSequenceEditor* SequenceEditor         = FMovieSceneSequenceEditor::Find(RootMovieSceneSequence);

	if (SequenceEditor && SequenceEditor->SupportsEvents(RootMovieSceneSequence))
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddEventTrack_ObjectBinding", "Event"),
			LOCTEXT("AddEventTooltip_ObjectBinding", "Adds a new event track that will trigger events on this object binding."),
			FNewMenuDelegate::CreateRaw(this, &FEventTrackEditor::AddEventSubMenu, ObjectBindings)
		);
	}
}

TSharedPtr<SWidget> FEventTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	check(Track);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TWeakObjectPtr<UMovieSceneTrack> WeakTrack = Track;
	const int32 RowIndex = Params.TrackInsertRowIndex;
	auto SubMenuCallback = [this, WeakTrack, RowIndex]
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		UMovieSceneTrack* TrackPtr = WeakTrack.Get();
		if (TrackPtr)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddNewTriggerSection", "Trigger"),
				LOCTEXT("AddNewTriggerSectionTooltip", "Adds a new section that can trigger a specific event at a specific time"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FEventTrackEditor::CreateNewSection, TrackPtr, RowIndex + 1, UMovieSceneEventTriggerSection::StaticClass(), true))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddNewRepeaterSection", "Repeater"),
				LOCTEXT("AddNewRepeaterSectionTooltip", "Adds a new section that triggers an event every time it's evaluated"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FEventTrackEditor::CreateNewSection, TrackPtr, RowIndex + 1, UMovieSceneEventRepeaterSection::StaticClass(), true))
			);
		}
		else
		{
			MenuBuilder.AddWidget(SNew(STextBlock).Text(LOCTEXT("InvalidTrack", "Track is no longer valid")), FText(), true);
		}

		return MenuBuilder.MakeWidget();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddSection", "Section"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
}

void FEventTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	UMovieSceneEventTrack* EventTrack = CastChecked<UMovieSceneEventTrack>(Track);
	FProperty* EventPositionProperty = FindFProperty<FProperty>(Track->GetClass(), GET_MEMBER_NAME_STRING_CHECKED(UMovieSceneEventTrack, EventPosition));

	FGuid ObjectBinding;
	EventTrack->GetTypedOuter<UMovieScene>()->FindTrackBinding(*EventTrack, ObjectBinding);

	/** Specific details customization for the event track */
	class FEventTrackCustomization : public IDetailCustomization
	{
	public:
		FGuid ObjectBindingID;

		FEventTrackCustomization(const FGuid& InObjectBindingID)
			: ObjectBindingID(InObjectBindingID)
		{}

		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
		{
			DetailBuilder.HideCategory("Track");
			DetailBuilder.HideCategory("General");
		}
	};

	auto PopulateSubMenu = [this, EventTrack, ObjectBinding](FMenuBuilder& SubMenuBuilder)
	{
		TSharedPtr<ISequencer> Sequencer = GetSequencer();
			
		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Create a details view for the track
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.ColumnWidth = 0.55f;

		TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
		
		// Register the custom type layout for the class
		FOnGetDetailCustomizationInstance CreateInstance = FOnGetDetailCustomizationInstance::CreateLambda([ObjectBinding]{ return MakeShared<FEventTrackCustomization>(ObjectBinding); });
		DetailsView->RegisterInstancedCustomPropertyLayout(UMovieSceneEventTrack::StaticClass(), CreateInstance);

		if (Sequencer.IsValid())
		{
			Sequencer->OnInitializeDetailsPanel().Broadcast(DetailsView, Sequencer.ToSharedRef());
		}

		// Assign the object
		DetailsView->SetObject(EventTrack, true);

		const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		// Add it to the menu
		TSharedRef< SWidget > DetailsViewWidget =
			SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
		[
			DetailsView
		];

		SubMenuBuilder.AddWidget(DetailsViewWidget, FText(), true, false);
	};

	MenuBuilder.AddSubMenu(LOCTEXT("Properties_MenuText", "Properties"), FText(), FNewMenuDelegate::CreateLambda(PopulateSubMenu));
}


bool FEventTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneEventTrack::StaticClass());
}

bool  FEventTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneEventTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

const FSlateBrush* FEventTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Event");
}

/* FEventTrackEditor callbacks
 *****************************************************************************/

void FEventTrackEditor::HandleAddEventTrackMenuEntryExecute(TArray<FGuid> InObjectBindingIDs, UClass* SectionType)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddEventTrack_Transaction", "Add Event Track"));
	FocusedMovieScene->Modify();

	TArray<UMovieSceneEventTrack*> NewTracks;

	for (FGuid InObjectBindingID : InObjectBindingIDs)
	{
		if (InObjectBindingID.IsValid())
		{
			UMovieSceneEventTrack* NewObjectTrack = FocusedMovieScene->AddTrack<UMovieSceneEventTrack>(InObjectBindingID);
			NewTracks.Add(NewObjectTrack);
		
			if (GetSequencer().IsValid())
			{
				GetSequencer()->OnAddTrack(NewObjectTrack, InObjectBindingID);
			}
		}
	}

	if (!NewTracks.Num())
	{
		UMovieSceneEventTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneEventTrack>();
		NewTracks.Add(NewTrack);
		if (GetSequencer().IsValid())
		{
			GetSequencer()->OnAddTrack(NewTrack, FGuid());
		}
	}

	check(NewTracks.Num() != 0);

	for (UMovieSceneEventTrack* NewTrack : NewTracks)
	{
		CreateNewSection(NewTrack, 0, SectionType, false);

		NewTrack->SetDisplayName(LOCTEXT("TrackName", "Events"));

		
	}
}

void FEventTrackEditor::CreateNewSection(UMovieSceneTrack* Track, int32 RowIndex, UClass* SectionType, bool bSelect)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
		FQualifiedFrameTime CurrentTime = SequencerPtr->GetLocalTime();

		FScopedTransaction Transaction(LOCTEXT("CreateNewSectionTransactionText", "Add Section"));

		UMovieSceneSection* NewSection = NewObject<UMovieSceneSection>(Track, SectionType, NAME_None, RF_Transactional);
		check(NewSection);

		int32 OverlapPriority = 0;
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section->GetRowIndex() >= RowIndex)
			{
				Section->SetRowIndex(Section->GetRowIndex() + 1);
			}
			OverlapPriority = FMath::Max(Section->GetOverlapPriority() + 1, OverlapPriority);
		}

		Track->Modify();

		if (SectionType == UMovieSceneEventTriggerSection::StaticClass())
		{
			NewSection->SetRange(TRange<FFrameNumber>::All());
		}
		else
		{

			TRange<FFrameNumber> NewSectionRange;

			if (CurrentTime.Time.FrameNumber < FocusedMovieScene->GetPlaybackRange().GetUpperBoundValue())
			{
				NewSectionRange = TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, FocusedMovieScene->GetPlaybackRange().GetUpperBoundValue());
			}
			else
			{
				const float DefaultLengthInSeconds = 5.f;
				NewSectionRange = TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, CurrentTime.Time.FrameNumber + (DefaultLengthInSeconds * SequencerPtr->GetFocusedTickResolution()).FloorToFrame());
			}

			NewSection->SetRange(NewSectionRange);
		}

		NewSection->SetOverlapPriority(OverlapPriority);
		NewSection->SetRowIndex(RowIndex);

		Track->AddSection(*NewSection);
		Track->UpdateEasing();

		if (bSelect)
		{
			SequencerPtr->EmptySelection();
			SequencerPtr->SelectSection(NewSection);
			SequencerPtr->ThrobSectionSelection();
		}

		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

#undef LOCTEXT_NAMESPACE
