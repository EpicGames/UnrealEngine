// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class FMenuBuilder;

/**
* A property track editor for named events.
*/
class FEventTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Factory function to create an instance of this class (called by a sequencer).
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FEventTrackEditor(TSharedRef<ISequencer> InSequencer);	

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track ) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;

	//~ FPropertyTrackEditor interface
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

private:

	void AddEventSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid>);

	/** Callback for executing the "Add Event Track" menu entry. */
	void HandleAddEventTrackMenuEntryExecute(TArray<FGuid> InObjectBindingIDs, UClass* SectionType);

	void CreateNewSection(UMovieSceneTrack* Track, int32 RowIndex, UClass* SectionType, bool bSelect);
};
