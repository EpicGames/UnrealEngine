// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/FadeTrackEditor.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "MovieSceneTrackEditor.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneFadeSection.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "TimeToPixel.h"
#include "Tracks/MovieSceneFadeTrack.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ISequencerTrackEditor;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FFadeTrackEditor"

/**
 * Class for fade sections handles drawing of fade gradient
 */
class FFadeSection
	: public FSequencerSection
{
public:

	/** Constructor. */
	FFadeSection(UMovieSceneSection& InSectionObject) : FSequencerSection(InSectionObject) {}

public:

	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override
	{
		int32 LayerId = Painter.PaintSectionBackground();

		const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		FVector2D GradientSize = FVector2D( Painter.SectionGeometry.Size.X - 2.f, Painter.SectionGeometry.Size.Y - 3.0f );
		FPaintGeometry PaintGeometry = Painter.SectionGeometry.ToPaintGeometry( GradientSize, FSlateLayoutTransform(FVector2f( 1.f, 3.f )) );

		const UMovieSceneFadeSection* FadeSection = Cast<const UMovieSceneFadeSection>( WeakSection.Get() );

		FTimeToPixel TimeConverter    = Painter.GetTimeConverter();
		FFrameRate   TickResolution   = TimeConverter.GetTickResolution();

		const double StartTimeSeconds = TimeConverter.PixelToSeconds(1.f);
		const double EndTimeSeconds   = TimeConverter.PixelToSeconds(Painter.SectionGeometry.GetLocalSize().X-2.f);
		const double TimeThreshold    = FMath::Max(0.0001, TimeConverter.PixelToSeconds(5) - TimeConverter.PixelToSeconds(0));
		const double DurationSeconds  = EndTimeSeconds - StartTimeSeconds;

		TArray<TTuple<double, double>> CurvePoints;
		FadeSection->FloatCurve.PopulateCurvePoints(StartTimeSeconds, EndTimeSeconds, TimeThreshold, 0.1f, TickResolution, CurvePoints);

		TArray<FSlateGradientStop> GradientStops;
		for (TTuple<double, double> Vector : CurvePoints)
		{
			GradientStops.Add( FSlateGradientStop(
				FVector2D( (Vector.Get<0>() - StartTimeSeconds) / DurationSeconds * Painter.SectionGeometry.Size.X, 0 ),
				FadeSection->FadeColor.CopyWithNewOpacity(Vector.Get<1>()) )
			);
		}

		if ( GradientStops.Num() > 0 )
		{
			FSlateDrawElement::MakeGradient(
				Painter.DrawElements,
				Painter.LayerId + 1,
				PaintGeometry,
				GradientStops,
				Orient_Vertical,
				DrawEffects
				);
		}

		return LayerId + 1;
	}
};


/* FFadeTrackEditor static functions
 *****************************************************************************/

TSharedRef<ISequencerTrackEditor> FFadeTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FFadeTrackEditor(InSequencer));
}


/* FFadeTrackEditor structors
 *****************************************************************************/

FFadeTrackEditor::FFadeTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor(InSequencer)
{ }

/* ISequencerTrackEditor interface
 *****************************************************************************/

FText FFadeTrackEditor::GetDisplayName() const
{
	return LOCTEXT("FadeTrackEditor_DisplayName", "Fade");
}

TSharedRef<ISequencerSection> FFadeTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShareable(new FFadeSection(SectionObject));
}

void FFadeTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddFadeTrack", "Fade Track"),
		LOCTEXT("AddFadeTrackTooltip", "Adds a new track that controls the fade of the sequence."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Fade"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FFadeTrackEditor::HandleAddFadeTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FFadeTrackEditor::HandleAddFadeTrackMenuEntryCanExecute)
		)
	);
}

bool FFadeTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneFadeTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FFadeTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneFadeTrack::StaticClass());
}

const FSlateBrush* FFadeTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Fade");
}


/* FFadeTrackEditor callbacks
 *****************************************************************************/

void FFadeTrackEditor::HandleAddFadeTrackMenuEntryExecute()
{
	UMovieScene* MovieScene = GetFocusedMovieScene();

	if (MovieScene == nullptr)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	UMovieSceneTrack* FadeTrack = MovieScene->FindTrack<UMovieSceneFadeTrack>();

	if (FadeTrack != nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddFadeTrack_Transaction", "Add Fade Track"));

	MovieScene->Modify();

	FadeTrack = FindOrCreateRootTrack<UMovieSceneFadeTrack>().Track;
	check(FadeTrack);

	UMovieSceneSection* NewSection = FadeTrack->CreateNewSection();
	check(NewSection);

	FadeTrack->Modify();
	FadeTrack->AddSection(*NewSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(FadeTrack, FGuid());
	}
}

bool FFadeTrackEditor::HandleAddFadeTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	
	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->FindTrack<UMovieSceneFadeTrack>() == nullptr));
}

#undef LOCTEXT_NAMESPACE
