// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "KeyframeTrackEditor.h"
#include "Tracks/MovieSceneFadeTrack.h"

class FMenuBuilder;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneTrack;
struct FGuid;
struct FSlateBrush;

/**
* A property track editor for fade control.
*/
class FFadeTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneFadeTrack>
{
public:

	/**
	 * Factory function to create an instance of this class (called by a sequencer).
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FFadeTrackEditor(TSharedRef<ISequencer> InSequencer);	

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual const FSlateBrush* GetIconBrush() const override;

private:

	/** Callback for executing the "Add Fade Track" menu entry. */
	void HandleAddFadeTrackMenuEntryExecute();
	bool HandleAddFadeTrackMenuEntryCanExecute() const;
};
