// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/LevelVisibilityTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "Sections/LevelVisibilitySection.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "LevelUtils.h"

#define LOCTEXT_NAMESPACE "LevelVisibilityTrackEditor.h"

FLevelVisibilityTrackEditor::FLevelVisibilityTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer ) 
{ }


TSharedRef<ISequencerTrackEditor> FLevelVisibilityTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FLevelVisibilityTrackEditor( InSequencer ) );
}

bool FLevelVisibilityTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneLevelVisibilityTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FLevelVisibilityTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneLevelVisibilityTrack::StaticClass();
}

const FSlateBrush* FLevelVisibilityTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.LevelVisibility");
}

FText FLevelVisibilityTrackEditor::GetDisplayName() const
{
	return LOCTEXT("LevelVisibilityTrackEditor_DisplayName", "Level Visibility");
}

TSharedRef<ISequencerSection> FLevelVisibilityTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	UMovieSceneLevelVisibilitySection* LevelVisibilitySection = Cast<UMovieSceneLevelVisibilitySection>(&SectionObject);
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) && LevelVisibilitySection != nullptr );

	return MakeShareable( new FLevelVisibilitySection( *LevelVisibilitySection ) );
}

void FLevelVisibilityTrackEditor::BuildAddTrackMenu( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Level Visibility Track" ),
		LOCTEXT("AddAdTrackToolTip", "Adds a new track which can control level visibility." ),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.LevelVisibility"),
		FUIAction( FExecuteAction::CreateRaw( this, &FLevelVisibilityTrackEditor::OnAddTrack ) ) );
}


TSharedPtr<SWidget> FLevelVisibilityTrackEditor::BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params )
{
	return UE::Sequencer::MakeAddButton(
		LOCTEXT( "AddVisibilityTrigger", "Visibility Trigger" ),
		FOnGetContent::CreateSP( this, &FLevelVisibilityTrackEditor::BuildAddVisibilityTriggerMenu, Track ),
		Params.ViewModel);
}

void FLevelVisibilityTrackEditor::BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track )
{
	MenuBuilder.BeginSection("Level Visibility", LOCTEXT("LevelVisibilitySection", "Level Visibility"));

	FFrameNumber CurrentTime = GetSequencer()->GetLocalTime().Time.FrameNumber;

	if (MovieSceneHelpers::FindSectionAtTime(Track->GetAllSections(), CurrentTime))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT( "ReplaceCurentLevelVisibility", "Replace with Current Level Visibility" ),
			LOCTEXT( "ReplaceCurentLevelVisibilityTooltip", "Replace the existing sections with the current level visibility state" ),
			FSlateIcon(),
			FUIAction(
			FExecuteAction::CreateRaw(this, &FLevelVisibilityTrackEditor::OnSetCurrentLevelVisibility, Track )));
	}
	else
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT( "AddCurentLevelVisibility", "Add from Current Level Visibility" ),
			LOCTEXT( "AddCurentLevelVisibilityTooltip", "Add new sections from the current level visibility state" ),
			FSlateIcon(),
			FUIAction(
			FExecuteAction::CreateRaw(this, &FLevelVisibilityTrackEditor::OnSetCurrentLevelVisibility, Track )));
	}

	MenuBuilder.EndSection();
}


UMovieSceneLevelVisibilitySection* FLevelVisibilityTrackEditor::AddNewSection( UMovieScene* MovieScene, UMovieSceneTrack* LevelVisibilityTrack, ELevelVisibility Visibility )
{
	const FScopedTransaction Transaction( LOCTEXT( "AddLevelVisibilitySection_Transaction", "Add Level Visibility Trigger" ) );

	LevelVisibilityTrack->Modify();

	UMovieSceneLevelVisibilitySection* LevelVisibilitySection = CastChecked<UMovieSceneLevelVisibilitySection>( LevelVisibilityTrack->CreateNewSection() );
	LevelVisibilitySection->SetVisibility( Visibility );

	TRange<FFrameNumber> SectionRange = MovieScene->GetPlaybackRange();
	LevelVisibilitySection->SetRange(SectionRange);

	int32 RowIndex = -1;
	for ( const UMovieSceneSection* Section : LevelVisibilityTrack->GetAllSections() )
	{
		RowIndex = FMath::Max( RowIndex, Section->GetRowIndex() );
	}
	LevelVisibilitySection->SetRowIndex(RowIndex + 1);

	LevelVisibilityTrack->AddSection( *LevelVisibilitySection );

	return LevelVisibilitySection;
}


void FLevelVisibilityTrackEditor::OnAddTrack()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if ( FocusedMovieScene == nullptr )
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "AddLevelVisibilityTrack_Transaction", "Add Level Visibility Track" ) );
	FocusedMovieScene->Modify();

	UMovieSceneLevelVisibilityTrack* NewTrack = FocusedMovieScene->AddTrack<UMovieSceneLevelVisibilityTrack>();
	checkf( NewTrack != nullptr, TEXT("Failed to create new level visibility track.") );

	UMovieSceneLevelVisibilitySection* NewSection = AddNewSection( FocusedMovieScene, NewTrack, ELevelVisibility::Visible );
	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}


TSharedRef<SWidget> FLevelVisibilityTrackEditor::BuildAddVisibilityTriggerMenu( UMovieSceneTrack* LevelVisibilityTrack )
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT( "AddVisibleTrigger", "Visible" ),
		LOCTEXT( "AddVisibleTriggerToolTip", "Add a trigger for visible levels." ),
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP(
			this, &FLevelVisibilityTrackEditor::OnAddNewSection, LevelVisibilityTrack, ELevelVisibility::Visible ) ) );

	MenuBuilder.AddMenuEntry(
		LOCTEXT( "AddHiddenTrigger", "Hidden" ),
		LOCTEXT( "AddHiddenTriggerToolTip", "Add a trigger for hidden levels." ),
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP(
		this, &FLevelVisibilityTrackEditor::OnAddNewSection, LevelVisibilityTrack, ELevelVisibility::Hidden ) ) );

	return MenuBuilder.MakeWidget();
}


void FLevelVisibilityTrackEditor::OnAddNewSection( UMovieSceneTrack* LevelVisibilityTrack, ELevelVisibility Visibility )
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if ( FocusedMovieScene == nullptr )
	{
		return;
	}

	UMovieSceneLevelVisibilitySection* NewSection = AddNewSection( FocusedMovieScene, LevelVisibilityTrack, Visibility );

	GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
	GetSequencer()->EmptySelection();
	GetSequencer()->SelectSection(NewSection);
	GetSequencer()->ThrobSectionSelection();
}

void FLevelVisibilityTrackEditor::GetCurrentLevelVisibility(TArray<FName>& OutVisibleLevelNames, TArray<FName>& OutHiddenLevelNames)
{
	UObject* PlaybackContext = GetSequencer()->GetPlaybackContext();
	if (!PlaybackContext)
	{
		return;
	}

	UWorld* World = PlaybackContext->GetWorld();
	if (!World)
	{
		return;
	}

	for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
	{
		if (LevelStreaming)
		{
			FName LevelName = FPackageName::GetShortFName( LevelStreaming->GetWorldAssetPackageFName() );

			if (FLevelUtils::IsStreamingLevelVisibleInEditor(LevelStreaming))
			{
				OutVisibleLevelNames.Add(LevelName);
			}
			else
			{
				OutHiddenLevelNames.Add(LevelName);
			}
		}
	}
}

void FLevelVisibilityTrackEditor::OnSetCurrentLevelVisibility( UMovieSceneTrack* LevelVisibilityTrack)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if ( FocusedMovieScene == nullptr )
	{
		return;
	}

	TArray<FName> VisibleLevelNames, HiddenLevelNames;
	GetCurrentLevelVisibility(VisibleLevelNames, HiddenLevelNames);

	if (VisibleLevelNames.Num() + HiddenLevelNames.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "SetCurrentLevelVisibility_Transaction", "Set Current Level Visibility" ) );

	UMovieSceneLevelVisibilitySection* VisibleSection = nullptr;
	UMovieSceneLevelVisibilitySection* HiddenSection = nullptr;

	FFrameNumber CurrentTime = GetSequencer()->GetLocalTime().Time.FrameNumber;

	for (UMovieSceneSection* Section : LevelVisibilityTrack->GetAllSections())
	{
		if (Section->IsTimeWithinSection(CurrentTime))
		{
			UMovieSceneLevelVisibilitySection* LevelVisibilitySection = Cast<UMovieSceneLevelVisibilitySection>(Section);

			LevelVisibilitySection->Modify();

			if (LevelVisibilitySection->GetVisibility() == ELevelVisibility::Visible)
			{
				if (!VisibleSection)
				{
					VisibleSection = LevelVisibilitySection;
					LevelVisibilitySection->SetLevelNames(VisibleLevelNames);
				}
				else
				{
					LevelVisibilitySection->SetLevelNames(TArray<FName>());
				}
			}
			else
			{
				if (!HiddenSection)
				{
					HiddenSection = LevelVisibilitySection;
					LevelVisibilitySection->SetLevelNames(HiddenLevelNames);
				}
				else
				{
					LevelVisibilitySection->SetLevelNames(TArray<FName>());
				}
			}
		}
	}

	if (!VisibleSection && VisibleLevelNames.Num() > 0)
	{
		VisibleSection = AddNewSection( FocusedMovieScene, LevelVisibilityTrack, ELevelVisibility::Visible );
		VisibleSection->SetLevelNames(VisibleLevelNames);
	}

	if (!HiddenSection && HiddenLevelNames.Num() > 0)
	{
		HiddenSection = AddNewSection( FocusedMovieScene, LevelVisibilityTrack, ELevelVisibility::Hidden );
		HiddenSection->SetLevelNames(HiddenLevelNames);
	}

	if (!VisibleSection && !HiddenSection)
	{
		return;
	}

	GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
	GetSequencer()->EmptySelection();
	if (VisibleSection != nullptr)
	{
		GetSequencer()->SelectSection(VisibleSection);
	}
	
	if (HiddenSection != nullptr)
	{
		GetSequencer()->SelectSection(HiddenSection);
	}
	GetSequencer()->ThrobSectionSelection();
}

#undef LOCTEXT_NAMESPACE
