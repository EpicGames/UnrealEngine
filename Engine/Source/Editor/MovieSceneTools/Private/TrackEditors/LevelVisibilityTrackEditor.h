// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class FMenuBuilder;
class UMovieSceneLevelVisibilitySection;
enum class ELevelVisibility : uint8;

/**
 * A sequencer track editor for level visibility movie scene tracks.
 */
class FLevelVisibilityTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FLevelVisibilityTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FLevelVisibilityTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer.
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void BuildAddTrackMenu( FMenuBuilder& MenuBuilder ) override;
	TSharedPtr<SWidget> BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params ) override;
	virtual void BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track ) override;

private:
	/** Adds a new section which spans the length of the owning movie scene with the specified visibility. */
	UMovieSceneLevelVisibilitySection* AddNewSection(UMovieScene* MovieScene, UMovieSceneTrack* LevelVisibilityTrack, ELevelVisibility Visibility );

	/** Handles when the add track menu item is activated. */
	void OnAddTrack();

	/** Builds the add visibility trigger menu which is displayed on the track. */
	TSharedRef<SWidget> BuildAddVisibilityTriggerMenu( UMovieSceneTrack* LevelVisibilityTrack );

	/** Handles when the add visibility trigger menu item is activated. */
	void OnAddNewSection( UMovieSceneTrack* LevelVisibilityTrack, ELevelVisibility Visibility );

	/** Handles when the add current level visibility menu item is activated. */
	void OnAddCurrentLevelVisibility(UMovieSceneTrack* LevelVisibilityTrack);

	/** Handles when the replace current level visibility menu item is activated. */
	void OnSetCurrentLevelVisibility(UMovieSceneTrack* LevelVisibilityTrack);

	void GetCurrentLevelVisibility(TArray<FName>& OutVisibleLevelNames, TArray<FName>& OutHiddenLevelNames);
};
