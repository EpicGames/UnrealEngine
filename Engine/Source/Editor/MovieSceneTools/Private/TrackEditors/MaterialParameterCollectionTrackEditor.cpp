// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/MaterialParameterCollectionTrackEditor.h"

#include "Algo/Sort.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Sections/ParameterSection.h"
#include "SequencerUtilities.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class ISequencerSection;
class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "MaterialParameterCollectionTrackEditor"


FMaterialParameterCollectionTrackEditor::FMaterialParameterCollectionTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FMaterialParameterCollectionTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FMaterialParameterCollectionTrackEditor>(OwningSequencer);
}

TSharedRef<ISequencerSection> FMaterialParameterCollectionTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(&SectionObject);
	checkf(ParameterSection != nullptr, TEXT("Unsupported section type."));

	return MakeShareable(new FParameterSection(*ParameterSection, GetSequencer()));
}

TSharedRef<SWidget> CreateAssetPicker(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed, TWeakPtr<ISequencer> InSequencer)
{
	TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
	UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = OnAssetSelected;
		AssetPickerConfig.OnAssetEnterPressed = OnAssetEnterPressed;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.Filter.ClassPaths.Add(UMaterialParameterCollection::StaticClass()->GetClassPathName());
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	
	const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	return SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void FMaterialParameterCollectionTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	auto SubMenuCallback = [this](FMenuBuilder& SubMenuBuilder)
	{
		SubMenuBuilder.AddWidget(CreateAssetPicker(FOnAssetSelected::CreateRaw(this, &FMaterialParameterCollectionTrackEditor::AddTrackToSequence), FOnAssetEnterPressed::CreateRaw(this, &FMaterialParameterCollectionTrackEditor::AddTrackToSequenceEnterPressed), GetSequencer()), FText::GetEmpty(), true);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddMPCTrack", "Material Parameter Collection Track"),
		LOCTEXT("AddMPCTrackToolTip", "Adds a new track that controls parameters within a Material Parameter Collection."),
		FNewMenuDelegate::CreateLambda(SubMenuCallback),
		false,
		FSlateIconFinder::FindIconForClass(UMaterialParameterCollection::StaticClass())
	);
}

void FMaterialParameterCollectionTrackEditor::AddTrackToSequence(const FAssetData& InAssetData)
{
	FSlateApplication::Get().DismissAllMenus();

	UMaterialParameterCollection* MPC = Cast<UMaterialParameterCollection>(InAssetData.GetAsset());
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (!MPC || !MovieScene)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	// Attempt to find an existing MPC track that animates this object
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (auto* MPCTrack = Cast<UMovieSceneMaterialParameterCollectionTrack>(Track))
		{
			if (MPCTrack->MPC == MPC)
			{
				return;
			}
		}
	}

	const FScopedTransaction Transaction(LOCTEXT("AddTrackDescription", "Add Material Parameter Collection Track"));

	MovieScene->Modify();
	UMovieSceneMaterialParameterCollectionTrack* Track = MovieScene->AddTrack<UMovieSceneMaterialParameterCollectionTrack>();
	check(Track);

	UMovieSceneSection* NewSection = Track->CreateNewSection();
	check(NewSection);

	Track->AddSection(*NewSection);
	Track->MPC = MPC;

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(Track, FGuid());
	}
}

void FMaterialParameterCollectionTrackEditor::AddTrackToSequenceEnterPressed(const TArray<FAssetData>& InAssetData)
{
	if (InAssetData.Num() > 0)
	{
		AddTrackToSequence(InAssetData[0].GetAsset());
	}
}

bool FMaterialParameterCollectionTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneMaterialParameterCollectionTrack::StaticClass();
}

bool FMaterialParameterCollectionTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneMaterialParameterCollectionTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

const FSlateBrush* FMaterialParameterCollectionTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(UMaterialParameterCollection::StaticClass()).GetIcon();
}

FText FMaterialParameterCollectionTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MaterialParameterCollectionTrackEditor_DisplayName", "Material Parameter Collection");
}

TSharedPtr<SWidget> FMaterialParameterCollectionTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneMaterialParameterCollectionTrack* MPCTrack = Cast<UMovieSceneMaterialParameterCollectionTrack>(Track);
	FOnGetContent MenuContent = FOnGetContent::CreateSP(this, &FMaterialParameterCollectionTrackEditor::OnGetAddParameterMenuContent, MPCTrack, Params.RowIndex, Params.TrackInsertRowIndex);

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddParameterButton", "Parameter"), MenuContent, Params.ViewModel);
}

TSharedPtr<SWidget> FMaterialParameterCollectionTrackEditor::BuildOutlinerEditColumnWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneMaterialParameterCollectionTrack* MPCTrack = Cast<UMovieSceneMaterialParameterCollectionTrack>(Track);
	FOnGetContent MenuContent = FOnGetContent::CreateSP(this, &FMaterialParameterCollectionTrackEditor::OnGetAddParameterMenuContent, MPCTrack, Params.RowIndex, Params.TrackInsertRowIndex);

	auto OnSetObjectLambda = [MPCTrack](const FAssetData& Asset) mutable
	{
		UMaterialParameterCollection* MPC = Cast<UMaterialParameterCollection>(Asset.GetAsset());

		FScopedTransaction Transaction(LOCTEXT("SetAssetTransaction", "Assign Material Parameter Collection"));
		MPCTrack->Modify();
		MPCTrack->MPC = MPC;
	};

	auto GetObjectPathLambda = [MPCTrack]() -> FString
	{
		UMaterialParameterCollection* MPC = MPCTrack->MPC;
		return MPC ? MPC->GetPathName() : FString();
	};

	TArray<FAssetData> AssetDataArray;

	UMovieSceneSequence* Sequence = GetSequencer()->GetFocusedMovieSceneSequence();
	AssetDataArray.Add((FAssetData)Sequence);

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(100.f)
			[
				SNew(SObjectPropertyEntryBox)
				.DisplayBrowse(true)
				.DisplayUseSelected(false)
				.ObjectPath_Lambda(GetObjectPathLambda)
				.AllowedClass(UMaterialParameterCollection::StaticClass())
				.OnObjectChanged_Lambda(OnSetObjectLambda)
				.OwnerAssetDataArray(AssetDataArray)
			]
		];
}

TSharedRef<SWidget> FMaterialParameterCollectionTrackEditor::OnGetAddParameterMenuContent(UMovieSceneMaterialParameterCollectionTrack* MPCTrack, int32 RowIndex, int32 TrackInsertRowIndex)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// If this is supported, allow creating other sections with different blend types, and put
	// the material parameters after a separator. Otherwise, just show the parameters menu.
	const FMovieSceneBlendTypeField SupportedBlendTypes = MPCTrack->GetSupportedBlendTypes();
	if (SupportedBlendTypes.Num() > 1)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();
		FSequencerUtilities::PopulateMenu_CreateNewSection(MenuBuilder, TrackInsertRowIndex, MPCTrack, WeakSequencer);

		MenuBuilder.AddSeparator();
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ScalarParametersHeading", "Scalar"));
	{
		TArray<FCollectionScalarParameter> ScalarParameters = MPCTrack->MPC->ScalarParameters;
		Algo::SortBy(ScalarParameters, &FCollectionParameterBase::ParameterName, FNameLexicalLess());

		for (const FCollectionScalarParameter& Scalar : ScalarParameters)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromName(Scalar.ParameterName),
				FText(),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &FMaterialParameterCollectionTrackEditor::AddScalarParameter, MPCTrack, RowIndex, Scalar)
				);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("VectorParametersHeading", "Vector"));
	{
		TArray<FCollectionVectorParameter> VectorParameters = MPCTrack->MPC->VectorParameters;
		Algo::SortBy(VectorParameters, &FCollectionParameterBase::ParameterName, FNameLexicalLess());

		for (const FCollectionVectorParameter& Vector : VectorParameters)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromName(Vector.ParameterName),
				FText(),
				FSlateIcon(),
				FExecuteAction::CreateSP(this, &FMaterialParameterCollectionTrackEditor::AddVectorParameter, MPCTrack, RowIndex, Vector)
				);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void FMaterialParameterCollectionTrackEditor::AddScalarParameter(UMovieSceneMaterialParameterCollectionTrack* Track, int32 RowIndex, FCollectionScalarParameter Parameter)
{
	if (!Track->MPC)
	{
		return;
	}

	float Value = Parameter.DefaultValue;

	UObject* PlaybackContext = GetSequencer()->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	if (World)
	{
		if (UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Track->MPC))
		{
			Instance->GetScalarParameterValue(Parameter.ParameterName, Value);
		}
	}

	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddScalarParameter", "Add scalar parameter"));
	Track->Modify();
	Track->AddScalarParameterKey(Parameter.ParameterName, KeyTime, RowIndex, Value, GetSequencer()->GetKeyInterpolation());
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}


void FMaterialParameterCollectionTrackEditor::AddVectorParameter(UMovieSceneMaterialParameterCollectionTrack* Track, int32 RowIndex, FCollectionVectorParameter Parameter)
{
	if (!Track->MPC)
	{
		return;
	}

	FLinearColor Value = Parameter.DefaultValue;

	UObject* PlaybackContext = GetSequencer()->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	if (World)
	{
		if (UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Track->MPC))
		{
			Instance->GetVectorParameterValue(Parameter.ParameterName, Value);
		}
	}

	FFrameNumber KeyTime = GetTimeForKey();

	const FScopedTransaction Transaction(LOCTEXT("AddVectorParameter", "Add vector parameter"));
	Track->Modify();
	Track->AddColorParameterKey(Parameter.ParameterName, KeyTime, RowIndex, Value, GetSequencer()->GetKeyInterpolation());
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

bool FMaterialParameterCollectionTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	UMaterialParameterCollection* MPC = Cast<UMaterialParameterCollection>(Asset);
	if (MPC)
	{
		AddTrackToSequence(FAssetData(MPC));
	}

	return MPC != nullptr;
}

#undef LOCTEXT_NAMESPACE
