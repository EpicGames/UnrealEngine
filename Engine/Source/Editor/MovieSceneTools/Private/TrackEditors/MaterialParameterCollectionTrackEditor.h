// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"

#define UE_API MOVIESCENETOOLS_API

class FMenuBuilder;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class SWidget;
class UMaterialParameterCollection;
class UMovieSceneMaterialParameterCollectionTrack;
class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneTrack;
class UObject;
struct FAssetData;
struct FBuildEditWidgetParams;
struct FCollectionScalarParameter;
struct FCollectionVectorParameter;
struct FGuid;
struct FSlateBrush;

/**
 * Track editor for material parameter collections.
 */
class FMaterialParameterCollectionTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/** Constructor. */
	UE_API FMaterialParameterCollectionTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Factory function */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	//~ ISequencerTrackEditor interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditColumnWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;

private:

	/** Provides the contents of the add parameter menu. */
	UE_API TSharedRef<SWidget> OnGetAddParameterMenuContent(UMovieSceneMaterialParameterCollectionTrack* MaterialTrack, int32 RowIndex, int32 TrackInsertRowIndex);

	UE_API void AddScalarParameter(UMovieSceneMaterialParameterCollectionTrack* Track, int32 RowIndex, FCollectionScalarParameter Parameter);
	UE_API void AddVectorParameter(UMovieSceneMaterialParameterCollectionTrack* Track, int32 RowIndex, FCollectionVectorParameter Parameter);

	UE_API void AddTrackToSequence(const FAssetData& InAssetData);
	UE_API void AddTrackToSequenceEnterPressed(const TArray<FAssetData>& InAssetData);
};

#undef UE_API
