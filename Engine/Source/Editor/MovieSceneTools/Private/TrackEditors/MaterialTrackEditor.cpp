// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/MaterialTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Components/DecalComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Sections/MovieSceneComponentMaterialParameterSection.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Sections/ComponentMaterialParameterSection.h"
#include "Sections/ParameterSection.h"
#include "SequencerUtilities.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Modules/ModuleManager.h"
#include "MaterialEditorModule.h"
#include "Engine/Selection.h"
#include "ISequencerModule.h"
#include "Components/MeshComponent.h"
#include "Components/VolumetricCloudComponent.h"


#define LOCTEXT_NAMESPACE "MaterialTrackEditor"


FMaterialTrackEditor::FMaterialTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer )
{
}


TSharedRef<ISequencerSection> FMaterialTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	UMovieSceneComponentMaterialParameterSection* ComponentMaterialParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(&SectionObject);
	UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(&SectionObject);
	checkf( ComponentMaterialParameterSection != nullptr || ParameterSection != nullptr, TEXT("Unsupported section type.") );

	if (ComponentMaterialParameterSection)
	{
		return MakeShareable(new FComponentMaterialParameterSection(*ComponentMaterialParameterSection, GetSequencer()));
	}
	else
	{
		return MakeShareable(new FParameterSection(*ParameterSection, GetSequencer()));
	}
}

FText FMaterialTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MaterialTrackEditor_DisplayName", "Material");
}

TSharedPtr<SWidget> FMaterialTrackEditor::BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params )
{
	UMovieSceneMaterialTrack* MaterialTrack = Cast<UMovieSceneMaterialTrack>(Track);
	FOnGetContent MenuContent = FOnGetContent::CreateSP(this, &FMaterialTrackEditor::OnGetAddMenuContent, ObjectBinding, MaterialTrack, Params.TrackInsertRowIndex);

	return UE::Sequencer::MakeAddButton(LOCTEXT( "AddParameterButton", "Parameter" ), MenuContent, Params.ViewModel);
}


TSharedRef<SWidget> FMaterialTrackEditor::OnGetAddMenuContent( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, int32 TrackInsertRowIndex )
{
	// IF this is supported, allow creating other sections with different blend types, and put
	// the material parameters after a separator. Otherwise, just show the parameters menu.
	const FMovieSceneBlendTypeField SupportedBlendTypes = MaterialTrack->GetSupportedBlendTypes();
	if (SupportedBlendTypes.Num() > 1)
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();
		FSequencerUtilities::PopulateMenu_CreateNewSection(MenuBuilder, TrackInsertRowIndex, MaterialTrack, WeakSequencer);

		MenuBuilder.AddSeparator();

		OnBuildAddParameterMenu(MenuBuilder, ObjectBinding, MaterialTrack);

		return MenuBuilder.MakeWidget();
	}
	else
	{
		return OnGetAddParameterMenuContent(ObjectBinding, MaterialTrack);
	}
}


struct FParameterInfoAndAction
{
	FMaterialParameterInfo ParameterInfo;
	FText ParameterDisplayName;
	FUIAction Action;

	FParameterInfoAndAction(const FMaterialParameterInfo& InParameterInfo, FText InParameterDisplayName, FUIAction InAction )
	{
		ParameterInfo = InParameterInfo;
		ParameterDisplayName = InParameterDisplayName;
		Action = InAction;
	}

	bool operator<(FParameterInfoAndAction const& Other) const
	{
		if (ParameterInfo.Index == Other.ParameterInfo.Index)
		{
			if (ParameterInfo.Association == Other.ParameterInfo.Association)
			{
				return ParameterInfo.Name.LexicalLess(Other.ParameterInfo.Name);
			}
			return ParameterInfo.Association < Other.ParameterInfo.Association;
		}
		return ParameterInfo.Index < Other.ParameterInfo.Index;
	}
};


TSharedRef<SWidget> FMaterialTrackEditor::OnGetAddParameterMenuContent( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack )
{
	FMenuBuilder AddParameterMenuBuilder( true, nullptr );
	OnBuildAddParameterMenu(AddParameterMenuBuilder, ObjectBinding, MaterialTrack);
	return AddParameterMenuBuilder.MakeWidget();
}


void FMaterialTrackEditor::OnBuildAddParameterMenu( FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack )
{
	UMaterial* Material = GetMaterialForTrack( ObjectBinding, MaterialTrack );
	if ( Material != nullptr )
	{
		UMaterialInterface* MaterialInterface = GetMaterialInterfaceForTrack(ObjectBinding, MaterialTrack);
		
		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>( MaterialInterface );	
		TArray<FMaterialParameterInfo> VisibleExpressions;

		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		bool bCollectedVisibleParameters = false;
		if (MaterialEditorModule && MaterialInstance)
		{
			MaterialEditorModule->GetVisibleMaterialParameters(Material, MaterialInstance, VisibleExpressions);
			bCollectedVisibleParameters = true;
		}

		TArray<FParameterInfoAndAction> ParameterInfosAndActions;

		// Collect scalar parameters.
		TArray<FMaterialParameterInfo> ScalarParameterInfos;
		TArray<FGuid> ScalarParameterGuids;
		MaterialInterface->GetAllScalarParameterInfo(ScalarParameterInfos, ScalarParameterGuids );
		// In case we need to grab layer names.
		FMaterialLayersFunctions Layers;
		MaterialInterface->GetMaterialLayers(Layers);

		auto GetMaterialParameterLayerName = [&Layers](const FMaterialParameterInfo& InParameterInfo)
		{
			FString LayerName;
			if (Layers.EditorOnly.LayerNames.IsValidIndex(InParameterInfo.Index))
			{
				LayerName = Layers.GetLayerName(InParameterInfo.Index).ToString();
			}
			return LayerName;
		};
		auto GetMaterialParameterAssetName = [&Layers](const FMaterialParameterInfo& InParameterInfo)
		{
			FString AssetName;
			if (InParameterInfo.Association == EMaterialParameterAssociation::LayerParameter && Layers.Layers.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Layers[InParameterInfo.Index]->GetName();
			}
			else if (InParameterInfo.Association == EMaterialParameterAssociation::BlendParameter && Layers.Blends.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Blends[InParameterInfo.Index]->GetName();
			}
			return AssetName;
		};

		auto GetMaterialParameterDisplayName = [](const FMaterialParameterInfo& InParameterInfo, const FString& InLayerName, const FString& InAssetName)
		{
			FText DisplayName = FText::FromName(InParameterInfo.Name);
			if (!InLayerName.IsEmpty() && !InAssetName.IsEmpty())
			{
				DisplayName = FText::Format(LOCTEXT("MaterialParameterDisplayName", "{0} ({1}.{2})"), DisplayName, FText::FromString(InLayerName), FText::FromString(InAssetName));
			}
			return DisplayName;
		};

		for (int32 ScalarParameterIndex = 0; ScalarParameterIndex < ScalarParameterInfos.Num(); ++ScalarParameterIndex)
		{
			FMaterialParameterInfo ScalarParameterInfo = ScalarParameterInfos[ScalarParameterIndex];
			if (!bCollectedVisibleParameters || VisibleExpressions.Contains(ScalarParameterInfo))
			{
				FString LayerName = GetMaterialParameterLayerName(ScalarParameterInfo);
				FString AssetName = GetMaterialParameterAssetName(ScalarParameterInfo);
				FText ParameterDisplayName = GetMaterialParameterDisplayName(ScalarParameterInfo, LayerName, AssetName);
				FUIAction AddParameterMenuAction( FExecuteAction::CreateSP( this, &FMaterialTrackEditor::AddScalarParameter, ObjectBinding, MaterialTrack, ScalarParameterInfo, LayerName, AssetName) );
				FParameterInfoAndAction InfoAndAction(ScalarParameterInfo, ParameterDisplayName, AddParameterMenuAction );
				ParameterInfosAndActions.Add(InfoAndAction);
			}
		}

		// Collect color parameters.
		TArray<FMaterialParameterInfo> ColorParameterInfos;
		TArray<FGuid> ColorParameterGuids;
		MaterialInterface->GetAllVectorParameterInfo(ColorParameterInfos, ColorParameterGuids );
		for (int32 ColorParameterIndex = 0; ColorParameterIndex < ColorParameterInfos.Num(); ++ColorParameterIndex)
		{
			FMaterialParameterInfo ColorParameterInfo = ColorParameterInfos[ColorParameterIndex];
			if (!bCollectedVisibleParameters || VisibleExpressions.Contains(ColorParameterInfo))
			{
				FString LayerName = GetMaterialParameterLayerName(ColorParameterInfo);
				FString AssetName = GetMaterialParameterAssetName(ColorParameterInfo);
				FText ParameterDisplayName = GetMaterialParameterDisplayName(ColorParameterInfo, LayerName, AssetName);
				FUIAction AddParameterMenuAction( FExecuteAction::CreateSP( this, &FMaterialTrackEditor::AddColorParameter, ObjectBinding, MaterialTrack, ColorParameterInfo, LayerName, AssetName ) );
				FParameterInfoAndAction InfoAndAction(ColorParameterInfo, ParameterDisplayName, AddParameterMenuAction );
				ParameterInfosAndActions.Add(InfoAndAction);
			}
		}

		// Sort and generate menu.
		ParameterInfosAndActions.Sort();

		for (FParameterInfoAndAction InfoAndAction : ParameterInfosAndActions)
		{
			MenuBuilder.AddMenuEntry(InfoAndAction.ParameterDisplayName, FText(), FSlateIcon(), InfoAndAction.Action );
		}
	}
}


UMaterial* FMaterialTrackEditor::GetMaterialForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack )
{
	UMaterialInterface* MaterialInterface = GetMaterialInterfaceForTrack( ObjectBinding, MaterialTrack );
	if ( MaterialInterface != nullptr )
	{
		UMaterial* Material = Cast<UMaterial>( MaterialInterface );
		if ( Material != nullptr )
		{
			return Material;
		}
		else
		{
			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>( MaterialInterface );
			if ( MaterialInstance != nullptr )
			{
				return MaterialInstance->GetMaterial();
			}
		}
	}
	return nullptr;
}


void FMaterialTrackEditor::AddScalarParameter( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, FMaterialParameterInfo ParameterInfo, FString InLayerName, FString InAssetName)
{
	FFrameNumber KeyTime = GetTimeForKey();

	UMaterialInterface* Material = GetMaterialInterfaceForTrack(ObjectBinding, MaterialTrack);
	if (Material != nullptr)
	{
		const FScopedTransaction Transaction( LOCTEXT( "AddScalarParameter", "Add scalar parameter" ) );
		float ParameterValue;
		Material->GetScalarParameterValue(ParameterInfo, ParameterValue);
		MaterialTrack->Modify();
		MaterialTrack->AddScalarParameterKey(ParameterInfo, KeyTime, ParameterValue, InLayerName, InAssetName, GetSequencer()->GetKeyInterpolation());
	}
	GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
}


void FMaterialTrackEditor::AddColorParameter( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, FMaterialParameterInfo ParameterInfo, FString InLayerName, FString InAssetName)
{
	FFrameNumber KeyTime = GetTimeForKey();

	UMaterialInterface* Material = GetMaterialInterfaceForTrack( ObjectBinding, MaterialTrack );
	if ( Material != nullptr )
	{
		const FScopedTransaction Transaction( LOCTEXT( "AddVectorParameter", "Add vector parameter" ) );
		FLinearColor ParameterValue;
		Material->GetVectorParameterValue(ParameterInfo, ParameterValue );
		MaterialTrack->Modify();
		FParameterChannelNames ChannelNames;
		Material->GetVectorParameterChannelNames(ParameterInfo, ChannelNames);
		MaterialTrack->AddColorParameterKey(ParameterInfo, KeyTime, INDEX_NONE, ParameterValue, InLayerName, InAssetName, ChannelNames, GetSequencer()->GetKeyInterpolation());
	}
	GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
}


FComponentMaterialTrackEditor::FComponentMaterialTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMaterialTrackEditor( InSequencer )
{
}


TSharedRef<ISequencerTrackEditor> FComponentMaterialTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable( new FComponentMaterialTrackEditor( OwningSequencer ) );
}


void FComponentMaterialTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	if (UMovieSceneComponentMaterialTrack* MaterialTrack = Cast<UMovieSceneComponentMaterialTrack>(Track))
	{
		FGuid ObjectBinding = MaterialTrack->FindObjectBindingGuid();
		UObject* BoundObject = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBinding);
		const FComponentMaterialInfo& MaterialInfo = MaterialTrack->GetMaterialInfo();
		if (MaterialInfo.MaterialType == EComponentMaterialType::Empty || MaterialInfo.MaterialType == EComponentMaterialType::IndexedMaterial)
		{
			if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(BoundObject))
			{
				int32 FoundMaterialIndex = INDEX_NONE;
				FName FoundSlotName;
				UMaterialInterface* Material = nullptr;
				FText AutoRebindTooltip;
				bool bNeedsRebindFix = false;

				if (!MaterialInfo.MaterialSlotName.IsNone())
				{
					Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
					if (Material)
					{
						FoundSlotName = MaterialInfo.MaterialSlotName;
					}
					FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
					if (FoundMaterialIndex != INDEX_NONE && FoundMaterialIndex != MaterialInfo.MaterialSlotIndex)
					{
						bNeedsRebindFix = true;
						// Found material by slot name, but the indices don't match what is cached. Auto-rebind would change index to the one in the found slot name.
						AutoRebindTooltip = FText::Format(LOCTEXT("AutoRebindToNewSlotIndex", "Rebind track to index {0}, keeping same slot {1}"), FText::AsNumber(FoundMaterialIndex), FText::FromName(MaterialInfo.MaterialSlotName));
					}
				}
				if (!Material)
				{
					bNeedsRebindFix = true;
					// Couldn't find binding by slot name or one wasn't specified, try by index
					Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
					if (Material)
					{
						// Found material by index, but not by slot name. Auto-rebind would change slot name to the current one
						TArray<FName> SlotNames = Component->GetMaterialSlotNames();
						if (SlotNames.IsValidIndex(MaterialInfo.MaterialSlotIndex))
						{
							FoundMaterialIndex = MaterialInfo.MaterialSlotIndex;
							FoundSlotName = SlotNames[MaterialInfo.MaterialSlotIndex];
							AutoRebindTooltip = FText::Format(LOCTEXT("AutoRebindToNewSlotName", "Rebind track to slot {0}, keeping same index {1}"), FText::FromName(FoundSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
					// If we didn't find a material, we don't create a tooltip, because we won't be able to 'auto' rebind, just manually bind.
				}

				// If we didn't find a material, a slot name wasn't specified or wasn't found, or an index was mismatched, allow the user to rebind
				if (bNeedsRebindFix)
				{
					MenuBuilder.BeginSection("Fix Material Binding", LOCTEXT("FixMaterialBindingSectionName", "Fix Material Binding"));

					// If we set this tooltip, then we are able to auto-rebind
					if (!AutoRebindTooltip.IsEmpty())
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("AutoFixMaterialBinding", "Auto-Fix Material Binding"),
							AutoRebindTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([MaterialTrack, FoundMaterialIndex, FoundSlotName]() 
									{ 
										FScopedTransaction Transaction(LOCTEXT("FixupMaterialBinding", "Fixup material binding"));
										MaterialTrack->Modify();
										MaterialTrack->SetMaterialInfo(FComponentMaterialInfo{ FoundSlotName, FoundMaterialIndex, EComponentMaterialType::IndexedMaterial });
										MaterialTrack->SetDisplayName(FText::Format(LOCTEXT("SlotMaterialTrackName", "Material Slot: {0}"), FText::FromName(FoundSlotName)));
									})));
					}

					MenuBuilder.AddSubMenu(
						LOCTEXT("RebindMaterial", "Re-Bind Material Track..."),
						LOCTEXT("RebindMaterialTooltip", "Re-Bind this material track to a different material slot"),
						FNewMenuDelegate::CreateSP(this, &FComponentMaterialTrackEditor::FillRebindMaterialTrackMenu, MaterialTrack, Component, ObjectBinding)
					);

					MenuBuilder.EndSection();
					MenuBuilder.AddMenuSeparator();
				}
			}
		}
	}
}

void FComponentMaterialTrackEditor::FillRebindMaterialTrackMenu(FMenuBuilder& MenuBuilder, UMovieSceneComponentMaterialTrack* MaterialTrack, UPrimitiveComponent* PrimitiveComponent, FGuid ObjectBinding)
{
	auto GetMaterialInfoForTrack = [](UMovieSceneTrack* InTrack)
	{
		UMovieSceneComponentMaterialTrack* MaterialTrack = Cast<UMovieSceneComponentMaterialTrack>(InTrack);
		return MaterialTrack ? MaterialTrack->GetMaterialInfo() : FComponentMaterialInfo();
	};
	int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
	TArray<FName> MaterialSlotNames = PrimitiveComponent->GetMaterialSlotNames();
	const UMovieScene* MovieScene = GetFocusedMovieScene();
	const FMovieSceneBinding* Binding = Algo::FindBy(MovieScene->GetBindings(), ObjectBinding, &FMovieSceneBinding::GetObjectGuid);
	if (Binding && NumMaterials > 0)
	{
		MenuBuilder.BeginSection("RebindMaterialSlots", LOCTEXT("RebindMaterialSlots", "Material Slots"));
		{
			for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
			{
				FName MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : FName();
				FComponentMaterialInfo MaterialInfo{ MaterialSlotName, MaterialIndex, EComponentMaterialType::IndexedMaterial };
				const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
				if (bAlreadyExists)
				{
					continue;
				}
				FUIAction RebindToMaterialAction(FExecuteAction::CreateLambda([MaterialTrack, MaterialInfo]() 
					{
						FScopedTransaction Transaction(LOCTEXT("FixupMaterialBinding", "Fixup material binding"));
						MaterialTrack->Modify();
						MaterialTrack->SetMaterialInfo(MaterialInfo); 
						MaterialTrack->SetDisplayName(FText::Format(LOCTEXT("SlotMaterialTrackName", "Material Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName)));
					}));
				FText RebindToMaterialLabel = FText::Format(LOCTEXT("RebindToMaterialSlot", "Slot: {0}, Index: {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex));
				FText RebindToMaterialToolTip = FText::Format(LOCTEXT("RebindToMaterialSlotToolTip", "Rebind this track to material slot {0}, index {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex));
				MenuBuilder.AddMenuEntry(RebindToMaterialLabel, RebindToMaterialToolTip, FSlateIcon(), RebindToMaterialAction);
			}
		}
		MenuBuilder.EndSection();
	}
}

bool FComponentMaterialTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneComponentMaterialTrack::StaticClass();
}

bool FComponentMaterialTrackEditor::GetDefaultExpansionState(UMovieSceneTrack* InTrack) const
{
	return true;
}


UMaterialInterface* FComponentMaterialTrackEditor::GetMaterialInterfaceForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack )
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return nullptr;
	}

	UMovieSceneComponentMaterialTrack* ComponentMaterialTrack = Cast<UMovieSceneComponentMaterialTrack>( MaterialTrack );
	if (!ComponentMaterialTrack)
	{
		return nullptr;
	}

	UObject* Object = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBinding);
	if (!Object)
	{
		return nullptr;
	}

	const FComponentMaterialInfo& MaterialInfo = ComponentMaterialTrack->GetMaterialInfo();
	switch (MaterialInfo.MaterialType)
	{
	case EComponentMaterialType::Empty:
		break;
	case EComponentMaterialType::IndexedMaterial:
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object))
		{
			UMaterialInterface* Material = nullptr;
			if (!MaterialInfo.MaterialSlotName.IsNone())
			{
				Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
			}
			if (!Material)
			{
				Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
			}
			return Material;
		}
		break;
	case EComponentMaterialType::OverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			return Component->GetOverlayMaterial();
		}
		break;
	case EComponentMaterialType::DecalMaterial:
		if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(Object))
		{
			return DecalComponent->GetDecalMaterial();
		}
		break;
	case EComponentMaterialType::VolumetricCloudMaterial:
		if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(Object))
		{
			return CloudComponent->GetMaterial();
		}
		break;
	case EComponentMaterialType::SlotOverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			return Component->GetOverlayMaterial(true, MaterialInfo.MaterialSlotIndex);
		}
		break;
	default:
		break;
	}
	return nullptr;
}

void FComponentMaterialTrackEditor::ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UPrimitiveComponent::StaticClass()) || ObjectClass->IsChildOf(UDecalComponent::StaticClass()) || ObjectClass->IsChildOf(UVolumetricCloudComponent::StaticClass()))
	{
		Extender->AddMenuExtension(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection, EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &FComponentMaterialTrackEditor::ConstructObjectBindingTrackMenu, ObjectBindings));
	}
}

void FComponentMaterialTrackEditor::ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	auto GetMaterialInfoForTrack = [](UMovieSceneTrack* InTrack)
	{
		UMovieSceneComponentMaterialTrack* MaterialTrack = Cast<UMovieSceneComponentMaterialTrack>(InTrack);
		return MaterialTrack ? MaterialTrack->GetMaterialInfo() : FComponentMaterialInfo();
	};
	UObject* Object = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBindings[0]);
	if (!Object)
	{
		return;
	}

	USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
	if (!SceneComponent)
	{
		return;
	}

	const UMovieScene* MovieScene = GetFocusedMovieScene();
	const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindings[0]);

	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent))
	{
		int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
		TArray<FName> MaterialSlotNames = PrimitiveComponent->GetMaterialSlotNames();
		UMeshComponent* MeshComponent = Cast<UMeshComponent>(SceneComponent);
		if (NumMaterials > 0 || MeshComponent)
		{
			MenuBuilder.BeginSection("Materials", LOCTEXT("MaterialSection", "Material Parameters"));
			{
				for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
				{
					FName MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : FName();
					FComponentMaterialInfo MaterialInfo{ MaterialSlotName, MaterialIndex, EComponentMaterialType::IndexedMaterial };
					const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
					if (bAlreadyExists)
					{
						continue;
					}
					FUIAction AddComponentMaterialAction(FExecuteAction::CreateRaw(this, &FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute, SceneComponent, MaterialInfo));
					FText AddComponentMaterialLabel = !MaterialSlotName.IsNone() ? 
						FText::Format(LOCTEXT("ComponentMaterialSlotNameLabelFormat", "Slot: {0}"), FText::FromName(MaterialSlotName)) :
						FText::Format(LOCTEXT("ComponentMaterialIndexLabelFormat", "Element {0}"), FText::AsNumber(MaterialIndex));
					FText AddComponentMaterialToolTip = !MaterialSlotName.IsNone() ? 
						FText::Format(LOCTEXT("ComponentMaterialSlotNameToolTipFormat", "Add material slot {0}, index {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex)) :
						FText::Format(LOCTEXT("ComponentMaterialIndexToolTipFormat", "Add material element {0}"), FText::AsNumber(MaterialIndex));
					MenuBuilder.AddMenuEntry(AddComponentMaterialLabel, AddComponentMaterialToolTip, FSlateIcon(), AddComponentMaterialAction);
				}
				if (MeshComponent)
				{
					if (UMaterialInterface* OverlayMaterial = MeshComponent->GetOverlayMaterial())
					{
						FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::OverlayMaterial };
						const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
						if (!bAlreadyExists)
						{
							FUIAction AddComponentMaterialAction(FExecuteAction::CreateRaw(this, &FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute, SceneComponent, MaterialInfo));
							FText AddOverlayMaterialLabel = FText::Format(LOCTEXT("AddOverlayMaterialLabelFormat", "Overlay: {0}"), FText::FromString(OverlayMaterial->GetName()));
							FText AddOverlayMaterialToolTip = FText::Format(LOCTEXT("AddOverlayMaterialToolTipFormat", "Add overlay material {0}"), FText::FromString(OverlayMaterial->GetName()));
							MenuBuilder.AddMenuEntry(AddOverlayMaterialLabel, AddOverlayMaterialToolTip, FSlateIcon(), AddComponentMaterialAction);
						}
					}

					const TArray<TObjectPtr<UMaterialInterface>>& SlotOverlayMaterials = MeshComponent->GetComponentMaterialSlotsOverlayMaterial();
					for (int32 SlotIndex = 0; SlotIndex < SlotOverlayMaterials.Num(); ++SlotIndex)
					{
						UMaterialInterface* SlotOverlayMaterial = SlotOverlayMaterials[SlotIndex];
						if (!SlotOverlayMaterial)
						{
							continue;
						}
						FName SlotName = MaterialSlotNames.IsValidIndex(SlotIndex) ? MaterialSlotNames[SlotIndex] : FName();
						FComponentMaterialInfo MaterialInfo{ SlotName, SlotIndex, EComponentMaterialType::SlotOverlayMaterial };
						const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
						if (bAlreadyExists)
						{
							continue;
						}
						FUIAction AddComponentMaterialAction(FExecuteAction::CreateRaw(this, &FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute, SceneComponent, MaterialInfo));
						FText AddSlotOverlayLabel = !SlotName.IsNone() ?
							FText::Format(LOCTEXT("AddSlotOverlayMaterialLabelFormat", "Overlay Slot: {0}"), FText::FromName(SlotName)) :
							FText::Format(LOCTEXT("AddSlotOverlayMaterialIndexLabelFormat", "Overlay Element {0}"), FText::AsNumber(SlotIndex));
						FText AddSlotOverlayToolTip = !SlotName.IsNone() ?
							FText::Format(LOCTEXT("AddSlotOverlayMaterialToolTipFormat", "Add per-slot overlay material for slot {0}, index {1}"), FText::FromName(SlotName), FText::AsNumber(SlotIndex)) :
							FText::Format(LOCTEXT("AddSlotOverlayMaterialIndexToolTipFormat", "Add per-slot overlay material for element {0}"), FText::AsNumber(SlotIndex));
						MenuBuilder.AddMenuEntry(AddSlotOverlayLabel, AddSlotOverlayToolTip, FSlateIcon(), AddComponentMaterialAction);
					}
				}
			}
			MenuBuilder.EndSection();
		}
	}
	else if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(SceneComponent))
	{
		if (UMaterialInterface* DecalMaterial = DecalComponent->GetDecalMaterial())
		{
			MenuBuilder.BeginSection("Materials", LOCTEXT("MaterialSection", "Material Parameters"));
			{
				FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::DecalMaterial};
				const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
				if (!bAlreadyExists)
				{
					FUIAction AddComponentMaterialAction(FExecuteAction::CreateRaw(this, &FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute, SceneComponent, MaterialInfo));
					FText AddDecalMaterialLabel = FText::Format(LOCTEXT("AddDecalMaterialLabelFormat", "Decal: {0}"), FText::FromString(DecalMaterial->GetName()));
					FText AddDecalMaterialToolTip = FText::Format(LOCTEXT("AddDecalMaterialToolTipFormat", "Add decal material {0}"), FText::FromString(DecalMaterial->GetName()));
					MenuBuilder.AddMenuEntry(AddDecalMaterialLabel, AddDecalMaterialToolTip, FSlateIcon(), AddComponentMaterialAction);
				}
			}
			MenuBuilder.EndSection();
		}
	}
	else if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(SceneComponent))
	{
		if (UMaterialInterface* CloudMaterial = CloudComponent->GetMaterial())
		{
			MenuBuilder.BeginSection("Materials", LOCTEXT("MaterialSection", "Material Parameters"));
			{
				FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::VolumetricCloudMaterial };
				const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
				if (!bAlreadyExists)
				{
					FUIAction AddComponentMaterialAction(FExecuteAction::CreateRaw(this, &FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute, SceneComponent, MaterialInfo));
					FText AddCloudMaterialLabel = FText::Format(LOCTEXT("AddCloudMaterialLabelFormat", "Volumetric Cloud: {0}"), FText::FromString(CloudMaterial->GetName()));
					FText AddCloudMaterialToolTip = FText::Format(LOCTEXT("AddCloudMaterialToolTipFormat", "Add volumetric cloud material {0}"), FText::FromString(CloudMaterial->GetName()));
					MenuBuilder.AddMenuEntry(AddCloudMaterialLabel, AddCloudMaterialToolTip, FSlateIcon(), AddComponentMaterialAction);
				}
			}
			MenuBuilder.EndSection();
		}
	}
}

void FComponentMaterialTrackEditor::HandleAddComponentMaterialActionExecute(USceneComponent* Component, FComponentMaterialInfo MaterialInfo)
{
	auto GetMaterialInfoForTrack = [](UMovieSceneTrack* InTrack)
	{
		UMovieSceneComponentMaterialTrack* MaterialTrack = Cast<UMovieSceneComponentMaterialTrack>(InTrack);
		return MaterialTrack ? MaterialTrack->GetMaterialInfo() : FComponentMaterialInfo();
	};
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddComponentMaterialTrack", "Add component material track"));

	MovieScene->Modify();

	FString ComponentName = Component->GetName();

	TArray<UActorComponent*> ActorComponents;
	ActorComponents.Add(Component);

	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (SelectedActors && SelectedActors->Num() > 0)
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = CastChecked<AActor>(*Iter);

			TArray<UActorComponent*> OutActorComponents;
			Actor->GetComponents(OutActorComponents);
			for (UActorComponent* ActorComponent : OutActorComponents)
			{
				if (ActorComponent->GetName() == ComponentName)
				{
					ActorComponents.AddUnique(ActorComponent);
				}
			}
		}
	}

	for (UActorComponent* ActorComponent : ActorComponents)
	{
		FGuid ObjectHandle = SequencerPtr->GetHandleToObject(ActorComponent);
		const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectHandle);

		const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
		if (!bAlreadyExists)
		{
			UMovieSceneComponentMaterialTrack* MaterialTrack = MovieScene->AddTrack<UMovieSceneComponentMaterialTrack>(ObjectHandle);
			MaterialTrack->Modify();
			MaterialTrack->SetMaterialInfo(MaterialInfo);
			// Construct display name from MaterialInfo
			UMaterialInterface* MaterialInterface = GetMaterialInterfaceForTrack(ObjectHandle, MaterialTrack);
			FText TrackDisplayName;
			switch (MaterialInfo.MaterialType)
			{
			case EComponentMaterialType::Empty:
				break;
			case EComponentMaterialType::IndexedMaterial:
				TrackDisplayName = !MaterialInfo.MaterialSlotName.IsNone() ? FText::Format(LOCTEXT("SlotMaterialTrackName", "Material Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName))
					: FText::Format(LOCTEXT("IndexedMaterialTrackName", "Material Element {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
				break;
			case EComponentMaterialType::OverlayMaterial:
				TrackDisplayName = FText::Format(LOCTEXT("OverlayMaterialTrackName", "Overlay Material {0}"), MaterialInterface ? FText::FromString(MaterialInterface->GetName()) : FText());
				break;
			case EComponentMaterialType::DecalMaterial:
				TrackDisplayName = FText::Format(LOCTEXT("DecalMaterialTrackName", "Decal Material {0}"), MaterialInterface ? FText::FromString(MaterialInterface->GetName()) : FText());
				break;			
			case EComponentMaterialType::VolumetricCloudMaterial:
					TrackDisplayName = FText::Format(LOCTEXT("CloudMaterialTrackName", "Volumetric Cloud Material {0}"), MaterialInterface ? FText::FromString(MaterialInterface->GetName()) : FText());
					break;
			case EComponentMaterialType::SlotOverlayMaterial:
				TrackDisplayName = !MaterialInfo.MaterialSlotName.IsNone() ?
					FText::Format(LOCTEXT("SlotOverlayMaterialTrackName", "Overlay Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName)) :
					FText::Format(LOCTEXT("IndexedSlotOverlayMaterialTrackName", "Overlay Element {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
				break;
			default:
				break;

			}
			if (!TrackDisplayName.IsEmpty())
			{
				MaterialTrack->SetDisplayName(TrackDisplayName);
			}
		}
	}

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

#undef LOCTEXT_NAMESPACE
