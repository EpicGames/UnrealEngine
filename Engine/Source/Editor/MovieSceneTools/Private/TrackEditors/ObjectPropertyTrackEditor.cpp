// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/ObjectPropertyTrackEditor.h"

#include "AssetToolsModule.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "KeyPropertyParams.h"
#include "Math/Color.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneFwd.h"
#include "PropertyPath.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakFieldPtr.h"

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UObject;


#define LOCTEXT_NAMESPACE "ObjectPropertyTrackEditor"


FObjectPropertyTrackEditor::FObjectPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
{}

TSharedRef<ISequencerTrackEditor> FObjectPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FObjectPropertyTrackEditor>(OwningSequencer);
}

FText FObjectPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ObjectPropertyTrackEditor_DisplayName", "Object Property");
}

void FObjectPropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	// Verify the property is actually an object property before attempting to get the value.
	// This prevents a crash when a non-object property (e.g., FInterfaceProperty) is incorrectly
	// routed to this track editor, as the type size assertion in GetPropertyValue would fail.
	const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get());
	if (!ObjectProperty)
	{
		UE_LOGF(LogMovieScene, Warning, "FObjectPropertyTrackEditor::GenerateKeysFromPropertyChanged called with non-object property type");
		return;
	}

	UObject* Existing = PropertyChangedParams.GetPropertyValue<UObject*>();
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneObjectPathChannel>(0, Existing, true));
}

void FObjectPropertyTrackEditor::InitializeNewTrack(UMovieSceneObjectPropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams)
{
	FPropertyTrackEditor::InitializeNewTrack(NewTrack, PropertyChangedParams);

	const FObjectPropertyBase* KeyedProperty = CastField<FObjectPropertyBase>(PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get());
	if (KeyedProperty)
	{
		NewTrack->PropertyClass = KeyedProperty->PropertyClass;

		if (KeyedProperty->HasAllPropertyFlags(CPF_UObjectWrapper))
		{
			const FClassProperty* ClassProperty = CastField<FClassProperty>(KeyedProperty);
			if (ClassProperty)
			{
				NewTrack->PropertyClass = ClassProperty->MetaClass;
				NewTrack->bClassProperty = true;
			}
		}

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		TArray< TWeakPtr<IAssetTypeActions> > AssetTypeActions;
		AssetToolsModule.Get().GetAssetTypeActionsList(AssetTypeActions);

		TSharedPtr<IAssetTypeActions> ClosestActions;

		// Find asset type actions that are either the same as the property class, or the closest base class
		for (TWeakPtr<IAssetTypeActions> WeakActions : AssetTypeActions)
		{
			TSharedPtr<IAssetTypeActions> Actions = WeakActions.Pin();
			if (Actions.IsValid())
			{
				UClass* SupportedClass = Actions->GetSupportedClass();
				if (SupportedClass)
				{
					// If we've found an exact match, just apply the color immediately
					if (NewTrack->PropertyClass == SupportedClass)
					{
						ClosestActions = Actions;
						break;
					}
					// If the property class is derived from the supported class, we can use it
					if (NewTrack->PropertyClass->IsChildOf(SupportedClass))
					{
						// Only assign the actions if they are currently null, or the supported class is a child of the existing ones
						if (!ClosestActions.IsValid() || SupportedClass->IsChildOf(ClosestActions->GetSupportedClass()))
						{
							ClosestActions = Actions;
						}
					}
				}
			}
		}

		if (ClosestActions.IsValid())
		{
			NewTrack->SetColorTint(ClosestActions->GetTypeColor().WithAlpha(75));
		}
	}
}

#undef LOCTEXT_NAMESPACE
