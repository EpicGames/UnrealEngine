// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "UObject/UnrealType.h"

class FPropertyChangedParams;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

class FObjectPropertyTrackEditor : public FPropertyTrackEditor<UMovieSceneObjectPropertyTrack>
{
public:

	/** Constructor. */
	FObjectPropertyTrackEditor(TSharedRef<ISequencer> InSequencer);

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromPropertyType(FObjectPropertyBase::StaticClass()) });
	}

	/** Factory function */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	//~ ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	virtual void InitializeNewTrack(UMovieSceneObjectPropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams) override;
};
