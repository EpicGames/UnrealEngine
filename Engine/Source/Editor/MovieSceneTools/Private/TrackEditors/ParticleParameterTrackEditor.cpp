// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/ParticleParameterTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Tracks/MovieSceneParticleParameterTrack.h"
#include "Sections/ParameterSection.h"
#include "MVVM/Views/ViewUtilities.h"


#define LOCTEXT_NAMESPACE "ParticleParameterTrackEditor"

FName FParticleParameterTrackEditor::TrackName( "ParticleParameter" );

FParticleParameterTrackEditor::FParticleParameterTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer )
{
}


TSharedRef<ISequencerTrackEditor> FParticleParameterTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable( new FParticleParameterTrackEditor( OwningSequencer ) );
}


TSharedRef<ISequencerSection> FParticleParameterTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(&SectionObject);
	checkf( ParameterSection != nullptr, TEXT("Unsupported section type.") );

	return MakeShareable(new FParameterSection( *ParameterSection, GetSequencer() ));
}

FText FParticleParameterTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ParticleParameterTrackEditor_DisplayName", "Particle Parameter");
}

TSharedPtr<SWidget> FParticleParameterTrackEditor::BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params )
{
	UMovieSceneParticleParameterTrack* ParticleParameterTrack = Cast<UMovieSceneParticleParameterTrack>(Track);

	return UE::Sequencer::MakeAddButton(LOCTEXT("ParameterText", "Parameter"),
		FOnGetContent::CreateSP(this, &FParticleParameterTrackEditor::OnGetAddParameterMenuContent, ObjectBinding, ParticleParameterTrack),
		Params.ViewModel);
}


void FParticleParameterTrackEditor::BuildObjectBindingTrackMenu( FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass )
{
	if ( ObjectClass->IsChildOf( AEmitter::StaticClass() ) || ObjectClass->IsChildOf( UParticleSystemComponent::StaticClass() ) )
	{
		const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();

		MenuBuilder.AddMenuEntry(
			LOCTEXT( "AddParticleParameterTrackLabel", "Particle Parameter Track" ),
			LOCTEXT( "AddParticleParameterTrackTooltip", "Adds a track for controlling particle parameter values." ),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateSP( this, &FParticleParameterTrackEditor::AddParticleParameterTrack, ObjectBindings ),
				FCanExecuteAction::CreateSP( this, &FParticleParameterTrackEditor::CanAddParticleParameterTrack, ObjectBindings[0] )
			));
	}
}


bool FParticleParameterTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneParticleParameterTrack::StaticClass();
}


TSharedRef<SWidget> FParticleParameterTrackEditor::OnGetAddParameterMenuContent(FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack)
{
	FMenuBuilder AddParameterMenuBuilder(true, nullptr);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UParticleSystemComponent* ParticleSystemComponent = GetParticleSystemComponentForBinding(ObjectBinding);

	if ( ParticleSystemComponent != nullptr )
	{
		TArray<FParameterNameAndAction> ParameterNamesAndActions;
		const TArray<FParticleSysParam> InstanceParameters = ParticleSystemComponent->GetAsyncInstanceParameters();
		for ( const FParticleSysParam& ParticleSystemParameter : InstanceParameters )
		{
			switch ( ParticleSystemParameter.ParamType )
			{
			case PSPT_Scalar:
			{
				FUIAction AddParameterMenuAction( FExecuteAction::CreateSP( this, &FParticleParameterTrackEditor::AddScalarParameter, ObjectBinding, ParticleParameterTrack, ParticleSystemParameter.Name ) );
				FParameterNameAndAction NameAndAction( ParticleSystemParameter.Name, AddParameterMenuAction );
				ParameterNamesAndActions.Add( NameAndAction );
				break;
			}
			case PSPT_Vector:
			{
				FUIAction AddParameterMenuAction( FExecuteAction::CreateSP( this, &FParticleParameterTrackEditor::AddVectorParameter, ObjectBinding, ParticleParameterTrack, ParticleSystemParameter.Name ) );
				FParameterNameAndAction NameAndAction( ParticleSystemParameter.Name, AddParameterMenuAction );
				ParameterNamesAndActions.Add( NameAndAction );
				break;
			}
			case PSPT_Color:
			{
				FUIAction AddParameterMenuAction( FExecuteAction::CreateSP( this, &FParticleParameterTrackEditor::AddColorParameter, ObjectBinding, ParticleParameterTrack, ParticleSystemParameter.Name ) );
				FParameterNameAndAction NameAndAction( ParticleSystemParameter.Name, AddParameterMenuAction );
				ParameterNamesAndActions.Add( NameAndAction );
				break;
			}
			}
		}

		// Sort and generate menu.
		ParameterNamesAndActions.Sort();
		for ( FParameterNameAndAction NameAndAction : ParameterNamesAndActions )
		{
			AddParameterMenuBuilder.AddMenuEntry( FText::FromName( NameAndAction.ParameterName ), FText(), FSlateIcon(), NameAndAction.Action );
		}
	}

	return AddParameterMenuBuilder.MakeWidget();
}


bool FParticleParameterTrackEditor::CanAddParticleParameterTrack( FGuid ObjectBinding )
{
	return GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack( UMovieSceneParticleParameterTrack::StaticClass(), ObjectBinding, TrackName ) == nullptr;
}


void FParticleParameterTrackEditor::AddParticleParameterTrack( TArray<FGuid> ObjectBindings )
{
	const FScopedTransaction Transaction(LOCTEXT("AddParticleParameterTrack", "Add particle parameter track"));

	for (FGuid ObjectBinding : ObjectBindings)
	{
		FindOrCreateTrackForObject(ObjectBinding, UMovieSceneParticleParameterTrack::StaticClass(), TrackName, true);
	}

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}


void FParticleParameterTrackEditor::AddScalarParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName )
{
	UParticleSystemComponent* ParticleSystemComponent = GetParticleSystemComponentForBinding(ObjectBinding);
	if (ParticleSystemComponent != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddScalarParameter", "Add scalar parameter"));

		FFrameNumber KeyTime = GetTimeForKey();
		float Value;
		ParticleSystemComponent->GetFloatParameter(ParameterName, Value);

		ParticleParameterTrack->Modify();
		ParticleParameterTrack->AddScalarParameterKey(ParameterName, KeyTime, Value, GetSequencer()->GetKeyInterpolation());

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}


void FParticleParameterTrackEditor::AddVectorParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName )
{
	UParticleSystemComponent* ParticleSystemComponent = GetParticleSystemComponentForBinding(ObjectBinding);
	if (ParticleSystemComponent != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddVectorParameter", "Add vector parameter"));

		FFrameNumber KeyTime = GetTimeForKey();
		FVector Value;
		ParticleSystemComponent->GetVectorParameter(ParameterName, Value);

		ParticleParameterTrack->Modify();
		ParticleParameterTrack->AddVectorParameterKey(ParameterName, KeyTime, Value, GetSequencer()->GetKeyInterpolation());

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}


void FParticleParameterTrackEditor::AddColorParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName )
{
	UParticleSystemComponent* ParticleSystemComponent = GetParticleSystemComponentForBinding(ObjectBinding);
	if (ParticleSystemComponent != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddColorParameter", "Add color parameter"));

		FFrameNumber KeyTime = GetTimeForKey();
		FLinearColor Value;
		ParticleSystemComponent->GetColorParameter(ParameterName, Value);

		ParticleParameterTrack->Modify();
		ParticleParameterTrack->AddColorParameterKey(ParameterName, KeyTime, Value, GetSequencer()->GetKeyInterpolation());

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}


UParticleSystemComponent* FParticleParameterTrackEditor::GetParticleSystemComponentForBinding(FGuid ObjectBinding)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UParticleSystemComponent* ParticleSystemComponent = nullptr;

	if (SequencerPtr.IsValid())
	{
		UObject* BoundObject = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);
		ParticleSystemComponent = Cast<UParticleSystemComponent>(BoundObject);
		if (ParticleSystemComponent == nullptr)
		{
			AEmitter* Emitter = Cast<AEmitter>(BoundObject);
			ParticleSystemComponent = Emitter != nullptr
				? Emitter->GetParticleSystemComponent()
				: nullptr;
		}
	}

	return ParticleSystemComponent;
}


#undef LOCTEXT_NAMESPACE
