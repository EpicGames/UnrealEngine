// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "Framework/Commands/UIAction.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class FMenuBuilder;
class UMovieSceneParticleParameterTrack;
class UParticleSystemComponent;
struct FParameterNameAndAction;

/**
 * Track editor for material parameters.
 */
class FParticleParameterTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/** Constructor. */
	FParticleParameterTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FParticleParameterTrackEditor() = default;

	/**
	* Creates an instance of this class.  Called by a sequencer.
	*
	* @param OwningSequencer The sequencer instance to be used by this tool.
	* @return The new instance of this class.
	*/
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params ) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;

private:

	static FName TrackName;

	// Struct used for building the parameter menu.
	struct FParameterNameAndAction
	{
		FName ParameterName;
		FUIAction Action;

		FParameterNameAndAction( FName InParameterName, FUIAction InAction )
		{
			ParameterName = InParameterName;
			Action = InAction;
		}

		bool operator<( FParameterNameAndAction const& Other ) const
		{
			return ParameterName.LexicalLess(Other.ParameterName);
		}
	};

	void BuildObjectBindingTrackMenu( FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass );

	/** Provides the contents of the add parameter menu. */
	TSharedRef<SWidget> OnGetAddParameterMenuContent( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack );

	bool CanAddParticleParameterTrack (FGuid ObjectBinding );

	void AddParticleParameterTrack( TArray<FGuid> ObjectBindings );

	/** Adds a scalar parameter and initial key to a material track.
	 * @param ObjectBinding The object binding which owns the material track.
	 * @param ParticleParameterTrack The track to Add the section to.
	 * @param ParameterName The name of the parameter to add an initial key for.
	 */
	void AddScalarParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName );

	/** Adds a vector parameter and initial key to a material track.
	* @param ObjectBinding The object binding which owns the material track.
	* @param ParticleParameterTrack The track to Add the section to.
	* @param ParameterName The name of the parameter to add an initial key for.
	*/
	void AddVectorParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName );

	/** Adds a Color parameter and initial key to a material track.
	* @param ObjectBinding The object binding which owns the material track.
	* @param ParticleParameterTrack The track to Add the section to.
	* @param ParameterName The name of the parameter to add an initial key for.
	*/
	void AddColorParameter( FGuid ObjectBinding, UMovieSceneParticleParameterTrack* ParticleParameterTrack, FName ParameterName );

	/** Gets the particle system component for the supplied object binding.  This works for emitter actors and
		directly bound particle components. */
	UParticleSystemComponent* GetParticleSystemComponentForBinding(FGuid ObjectBinding);
};
