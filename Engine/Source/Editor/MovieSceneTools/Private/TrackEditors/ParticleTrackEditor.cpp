// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/ParticleTrackEditor.h"
#include "AnimatedRange.h"
#include "Rendering/DrawElements.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Curves/IntegralCurve.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "UnrealEdGlobals.h"
#include "Tracks/MovieSceneParticleTrack.h"
#include "ISectionLayoutBuilder.h"
#include "Sections/MovieSceneParticleSection.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "TimeToPixel.h"
#include "MVVM/ViewModels/ViewDensity.h"

namespace AnimatableParticleEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 ParticleTrackHeight = 28;
}


#define LOCTEXT_NAMESPACE "FParticleTrackEditor"


FParticleSection::FParticleSection( UMovieSceneSection& InSection, TSharedRef<ISequencer>InOwningSequencer )
	: Section( InSection )
	, OwningSequencerPtr( InOwningSequencer )
{
}


FParticleSection::~FParticleSection()
{
}


UMovieSceneSection* FParticleSection::GetSectionObject()
{
	return &Section;
}

float FParticleSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(AnimatableParticleEditorConstants::ParticleTrackHeight);
}

int32 FParticleSection::OnPaintSection( FSequencerSectionPainter& InPainter ) const
{
	TSharedPtr<ISequencer> OwningSequencer = OwningSequencerPtr.Pin();

	if (!OwningSequencer.IsValid())
	{
		return InPainter.LayerId + 1;
	}

	const ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	UMovieSceneParticleSection* AnimSection = Cast<UMovieSceneParticleSection>( &Section );
	const FTimeToPixel& TimeToPixelConverter = InPainter.GetTimeConverter();

	FLinearColor TrackColor;

	// @todo Sequencer - These values should be cached and then refreshed only when the particle system changes.
	bool bIsLooping = false;
	double LastEmitterEndTime = 0;
	UMovieSceneParticleSection* ParticleSection = Cast<UMovieSceneParticleSection>( &Section );
	if ( ParticleSection != nullptr )
	{
		UMovieSceneParticleTrack* ParentTrack = Cast<UMovieSceneParticleTrack>( ParticleSection->GetOuter() );
		if ( ParentTrack != nullptr )
		{
			TrackColor = ParentTrack->GetColorTint();

			FGuid ObjectHandle;
			for ( const FMovieSceneBinding& Binding : ((const UMovieScene*)OwningSequencer->GetFocusedMovieSceneSequence()->GetMovieScene())->GetBindings() )
			{
				if ( Binding.GetTracks().Contains( ParentTrack ) )
				{
					ObjectHandle = Binding.GetObjectGuid();
					break;
				}
			}

			if ( ObjectHandle.IsValid() )
			{
				for (TWeakObjectPtr<> Object : OwningSequencer->FindObjectsInCurrentSequence(ObjectHandle))
				{
					UParticleSystemComponent* ParticleSystemComponent = Cast<UParticleSystemComponent>(Object.Get());
					if(AEmitter* ParticleSystemActor = Cast<AEmitter>(Object.Get()))
					{
						ParticleSystemComponent = ParticleSystemActor->GetParticleSystemComponent();
					}

					if ( ParticleSystemComponent != nullptr && ParticleSystemComponent->Template != nullptr )
					{
						for ( UParticleEmitter* Emitter : ParticleSystemComponent->Template->Emitters )
						{
							UParticleModuleRequired* RequiredModule = Emitter->GetLODLevel( 0 )->RequiredModule;
							bIsLooping |= RequiredModule->EmitterLoops == 0;
							LastEmitterEndTime = FMath::Max( LastEmitterEndTime, double(RequiredModule->EmitterDelay) + RequiredModule->EmitterDuration );
						}
					}
				}
			}
		}
	}

	// @todo Sequencer - This should only draw the visible ranges.
	TArray<TRange<float>>   DrawRanges;
	TOptional<double> CurrentRangeStart;

	if (ParticleSection != nullptr)
	{
		TMovieSceneChannelData<const uint8> ChannelData = ParticleSection->ParticleKeys.GetData();

		TArrayView<const FFrameNumber> Times  = ChannelData.GetTimes();
		TArrayView<const uint8>        Values = ChannelData.GetValues();

		for (int32 Index = 0; Index < Times.Num(); ++Index)
		{
			const double       Time  = Times[Index] / TimeToPixelConverter.GetTickResolution();
			const EParticleKey Value = (EParticleKey)Values[Index];

			if ( Value == EParticleKey::Activate )
			{
				if ( CurrentRangeStart.IsSet() == false )
				{
					CurrentRangeStart = Time;
				}
				else
				{
					if ( bIsLooping == false )
					{
						if ( Time > CurrentRangeStart.GetValue() + LastEmitterEndTime )
						{
							DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), CurrentRangeStart.GetValue() + LastEmitterEndTime ) );
						}
						else
						{
							DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), Time ) );
						}
						CurrentRangeStart = Time;
					}
				}
			}
			if ( Value == EParticleKey::Deactivate )
			{
				if ( CurrentRangeStart.IsSet() )
				{
					if (bIsLooping)
					{
						DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), Time ) );
					}
					else
					{
						if ( Time > CurrentRangeStart.GetValue() + LastEmitterEndTime )
						{
							DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), CurrentRangeStart.GetValue() + LastEmitterEndTime ) );
						}
						else
						{
							DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), Time ) );
						}
					}
					CurrentRangeStart.Reset();
				}
			}
		}
	}
	if ( CurrentRangeStart.IsSet() )
	{
		if ( bIsLooping )
		{
			DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), OwningSequencer->GetViewRange().GetUpperBoundValue() ) );
		}
		else
		{
			DrawRanges.Add( TRange<float>( CurrentRangeStart.GetValue(), CurrentRangeStart.GetValue() + LastEmitterEndTime ) );
		}
	}

	for ( const TRange<float>& DrawRange : DrawRanges )
	{
		float XOffset = TimeToPixelConverter.SecondsToPixel(DrawRange.GetLowerBoundValue());
		float XSize   = TimeToPixelConverter.SecondsToPixel(DrawRange.GetUpperBoundValue()) - XOffset;
		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId,
			InPainter.SectionGeometry.ToPaintGeometry( FVector2f( XSize, SequencerSectionConstants::KeySize.Y ), FSlateLayoutTransform(FVector2f( XOffset, (InPainter.SectionGeometry.GetLocalSize().Y - SequencerSectionConstants::KeySize.Y) / 2.f )) ),
			FAppStyle::GetBrush( "Sequencer.Section.Background" ),
			DrawEffects
			);
		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId,
			InPainter.SectionGeometry.ToPaintGeometry( FVector2f( XSize, SequencerSectionConstants::KeySize.Y ), FSlateLayoutTransform(FVector2f( XOffset, (InPainter.SectionGeometry.GetLocalSize().Y - SequencerSectionConstants::KeySize.Y) / 2.f )) ),
			FAppStyle::GetBrush( "Sequencer.Section.BackgroundTint" ),
			DrawEffects,
			TrackColor
			);
	}

	return InPainter.LayerId+1;
}


FParticleTrackEditor::FParticleTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer ) 
{ }


TSharedRef<ISequencerTrackEditor> FParticleTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FParticleTrackEditor( InSequencer ) );
}


bool FParticleTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneParticleTrack::StaticClass();
}


TSharedRef<ISequencerSection> FParticleTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );

	const TSharedPtr<ISequencer> OwningSequencer = GetSequencer();
	return MakeShareable( new FParticleSection( SectionObject, OwningSequencer.ToSharedRef() ) );
}

FText FParticleTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ParticleTrackEditor_DisplayName", "Particle");
}

void FParticleTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(AEmitter::StaticClass()) || ObjectClass->IsChildOf(UFXSystemComponent::StaticClass()))
	{
		const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddParticleTrack", "FX System Toggle Track"),
			LOCTEXT("TriggerParticlesTooltip", "Adds a track for controlling FX system state."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FParticleTrackEditor::AddParticleKey, ObjectBindings))
		);
	}
}


void FParticleTrackEditor::AddParticleKey( TArray<FGuid> ObjectGuids )
{
	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddParticleKey", "Add Particle Key"));

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	for (FGuid ObjectGuid : ObjectGuids)
	{
		UObject* Object = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(ObjectGuid) : nullptr;

		if (Object)
		{
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FParticleTrackEditor::AddKeyInternal, Object));
		}
	}
}


FKeyPropertyResult FParticleTrackEditor::AddKeyInternal( FFrameNumber KeyTime, UObject* Object )
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

	if (ObjectHandle.IsValid())
	{
		FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneParticleTrack::StaticClass());
		UMovieSceneTrack* Track = TrackResult.Track;
		KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

		if (KeyPropertyResult.bTrackCreated && ensure(Track))
		{
			UMovieSceneParticleTrack* ParticleTrack = Cast<UMovieSceneParticleTrack>(Track);
			ParticleTrack->AddNewSection(KeyTime);
			ParticleTrack->SetDisplayName(LOCTEXT("TrackName", "FX System"));
			KeyPropertyResult.bTrackModified = true;
		}
	}

	return KeyPropertyResult;
}


#undef LOCTEXT_NAMESPACE
