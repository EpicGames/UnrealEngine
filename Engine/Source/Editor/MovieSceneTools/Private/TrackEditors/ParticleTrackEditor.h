// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Curves/KeyHandle.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

class FMenuBuilder;
class FSequencerSectionPainter;

/**
 * Tools for particle tracks
 */
class FParticleTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FParticleTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FParticleTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer.
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	void AddParticleKey(const FGuid ObjectGuid, bool bTrigger);

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	
	void AddParticleKey(TArray<FGuid> ObjectGuids);
private:

	/** Delegate for AnimatablePropertyChanged in AddKey. */
	FKeyPropertyResult AddKeyInternal( FFrameNumber KeyTime, UObject* Object);
};


/**
 * Class for particle sections.
 */
class FParticleSection
	: public ISequencerSection
	, public TSharedFromThis<FParticleSection>
{
public:

	FParticleSection( UMovieSceneSection& InSection, TSharedRef<ISequencer> InOwningSequencer );
	~FParticleSection();

public:

	// ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override;
	virtual bool SectionIsResizable() const override { return false; }

private:

	/** The section we are visualizing. */
	UMovieSceneSection& Section;

	/** The sequencer that owns this section */
	TWeakPtr<ISequencer> OwningSequencerPtr;
};
