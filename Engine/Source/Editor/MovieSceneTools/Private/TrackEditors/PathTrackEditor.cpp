// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PathTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "SequencerSectionPainter.h"
#include "GameFramework/WorldSettings.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Sections/MovieScene3DPathSection.h"
#include "ISectionLayoutBuilder.h"
#include "ActorEditorUtils.h"
#include "Components/SplineComponent.h"
#include "MovieSceneToolHelpers.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSpawnableAnnotation.h"


#define LOCTEXT_NAMESPACE "FPathTrackEditor"

/**
 * Class that draws a path section in the sequencer
 */
class F3DPathSection
	: public ISequencerSection
{
public:
	F3DPathSection( UMovieSceneSection& InSection, F3DPathTrackEditor* InPathTrackEditor )
		: Section( InSection )
		, PathTrackEditor(InPathTrackEditor)
	{ }

	/** ISequencerSection interface */
	virtual UMovieSceneSection* GetSectionObject() override
	{ 
		return &Section;
	}
	
	virtual FText GetSectionTitle() const override 
	{ 
		UMovieScene3DPathSection* PathSection = Cast<UMovieScene3DPathSection>(&Section);
		if (PathSection)
		{
			TSharedPtr<ISequencer> Sequencer = PathTrackEditor->GetSequencer();
			if (Sequencer.IsValid())
			{
				TArrayView<TWeakObjectPtr<UObject>> RuntimeObjects = PathSection->GetConstraintBindingID().ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer);
				if (RuntimeObjects.Num() == 1 && RuntimeObjects[0].IsValid())
				{
					if (AActor* Actor = Cast<AActor>(RuntimeObjects[0].Get()))
					{
						return FText::FromString(Actor->GetActorLabel());
					}
				}
			}
		}

		return FText::GetEmpty(); 
	}

	virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override 
	{
		return InPainter.PaintSectionBackground();
	}
	
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override
	{
		TArray<FGuid> ObjectBindings;
		ObjectBindings.Add(ObjectBinding);

		MenuBuilder.AddSubMenu(
			LOCTEXT("SetPath", "Path"), LOCTEXT("SetPathTooltip", "Set path"),
			FNewMenuDelegate::CreateRaw(PathTrackEditor, &FActorPickerTrackEditor::ShowActorSubMenu, ObjectBindings, &Section));
	}

private:

	/** The section we are visualizing */
	UMovieSceneSection& Section;

	/** The path track editor */
	F3DPathTrackEditor* PathTrackEditor;
};


F3DPathTrackEditor::F3DPathTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FActorPickerTrackEditor( InSequencer ) 
{ 
}


TSharedRef<ISequencerTrackEditor> F3DPathTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new F3DPathTrackEditor( InSequencer ) );
}


bool F3DPathTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	// We support animatable transforms
	return Type == UMovieScene3DPathTrack::StaticClass();
}

bool F3DPathTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieScene3DPathTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

TSharedRef<ISequencerSection> F3DPathTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );

	return MakeShareable( new F3DPathSection( SectionObject, this ) );
}


void F3DPathTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass != nullptr && (ObjectClass->IsChildOf(AActor::StaticClass()) || ObjectClass->IsChildOf(USceneComponent::StaticClass())))
	{
		UMovieSceneSection* DummySection = nullptr;

		MenuBuilder.AddSubMenu(
			LOCTEXT("AddPath", "Path"), LOCTEXT("AddPathTooltip", "Adds a path track."),
			FNewMenuDelegate::CreateRaw(this, &FActorPickerTrackEditor::ShowActorSubMenu, ObjectBindings, DummySection));
	}
}

bool F3DPathTrackEditor::IsActorPickable(const AActor* const ParentActor, FGuid ObjectBinding, UMovieSceneSection* InSection)
{
	// Can't pick the object that this track binds
	if (GetSequencer()->FindObjectsInCurrentSequence(ObjectBinding).Contains(ParentActor))
	{
		return false;
	}

	// Can't pick the object that this track attaches to
	UMovieScene3DPathSection* PathSection = Cast<UMovieScene3DPathSection>(InSection);
	if (PathSection != nullptr)
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		TArrayView<TWeakObjectPtr<UObject>> RuntimeObjects = PathSection->GetConstraintBindingID().ResolveBoundObjects(SequencerPtr->GetFocusedTemplateID(), *SequencerPtr);
		if (RuntimeObjects.Contains(ParentActor))
		{
			return false;
		}
	}

	if (ParentActor->IsListedInSceneOutliner() &&
		!FActorEditorUtils::IsABuilderBrush(ParentActor) &&
		!ParentActor->IsA( AWorldSettings::StaticClass() ) &&
		IsValid(ParentActor))
	{			
		for (UActorComponent* Component : ParentActor->GetComponents())
		{
			if (Cast<USplineComponent>(Component))
			{
				return true;
			}
		}
	}
	return false;
}


void F3DPathTrackEditor::ActorSocketPicked(const FName SocketName, USceneComponent* Component, FActorPickerID ActorPickerID, TArray<FGuid> ObjectGuids, UMovieSceneSection* Section)
{
	if (Section != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("UndoSetPath", "Set Path"));

		UMovieScene3DPathSection* PathSection = (UMovieScene3DPathSection*)(Section);

		FMovieSceneObjectBindingID ConstraintBindingID;

		if (ActorPickerID.ExistingBindingID.IsValid())
		{
			ConstraintBindingID = ActorPickerID.ExistingBindingID;
		}
		else if (AActor* Actor = ActorPickerID.ActorPicked.Get())
		{
			TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

			TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
			if (Spawnable.IsSet())
			{
				// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
				ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(SequencerPtr->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, *SequencerPtr);
			}
			else
			{
				FGuid ParentActorId = FindOrCreateHandleToObject(Actor).Handle;
				ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(ParentActorId);
			}
		}

		if (ConstraintBindingID.IsValid())
		{
			PathSection->SetConstraintBindingID(ConstraintBindingID);
		}
	}
	else
	{
		TArray<TWeakObjectPtr<>> OutObjects;

		for (FGuid ObjectGuid : ObjectGuids)
		{
			for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectGuid))
			{
				OutObjects.Add(Object);
			}
		}

		AnimatablePropertyChanged( FOnKeyProperty::CreateRaw( this, &F3DPathTrackEditor::AddKeyInternal, OutObjects, ActorPickerID) );
	}
}

FKeyPropertyResult F3DPathTrackEditor::AddKeyInternal( FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, FActorPickerID ActorPickerID)
{
	FKeyPropertyResult KeyPropertyResult;

	FMovieSceneObjectBindingID ConstraintBindingID;

	if (ActorPickerID.ExistingBindingID.IsValid())
	{
		ConstraintBindingID = ActorPickerID.ExistingBindingID;
	}
	else if (AActor* Actor = ActorPickerID.ActorPicked.Get())
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

		TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Actor);
		if (Spawnable.IsSet())
		{
			// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
			ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(SequencerPtr->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, *SequencerPtr);
		}
		else
		{
			FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Actor);
			FGuid ParentActorId = HandleResult.Handle;
			KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

			ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(ParentActorId);
		}
	}

	if (!ConstraintBindingID.IsValid())
	{
		return KeyPropertyResult;
	}

	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();

	for( int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex )
	{
		UObject* Object = Objects[ObjectIndex].Get();

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieScene3DPathTrack::StaticClass());
			UMovieSceneTrack* Track = TrackResult.Track;
			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(Track))
			{
				// Clamp to next path section's start time or the end of the current movie scene range
				FFrameNumber PathEndTime = MovieScene->GetPlaybackRange().GetUpperBoundValue();
	
				for (UMovieSceneSection* Section : Track->GetAllSections())
				{
					FFrameNumber StartTime = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : TNumericLimits<int32>::Lowest();
					if (KeyTime < StartTime)
					{
						if (PathEndTime > StartTime)
						{
							PathEndTime = StartTime;
						}
					}
				}

				int32 Duration = FMath::Max(0, (PathEndTime - KeyTime).Value);
				UMovieSceneSection* NewSection = Cast<UMovieScene3DPathTrack>(Track)->AddConstraint( KeyTime, Duration, NAME_None, NAME_None, ConstraintBindingID );
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);
			}
		}
	}

	return KeyPropertyResult;
}

#undef LOCTEXT_NAMESPACE
