// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "TrackEditors/ActorPickerTrackEditor.h"

class AActor;
class FMenuBuilder;
class USceneComponent;

/**
 * Tools for animating a transform with a path curve
 */
class F3DPathTrackEditor
	: public FActorPickerTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	F3DPathTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~F3DPathTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface

	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	// FTrackEditorActorPicker
	virtual bool IsActorPickable( const AActor* const ParentActor, FGuid ObjectBinding, UMovieSceneSection* InSection ) override;
	virtual void ActorSocketPicked(const FName SocketName, USceneComponent* Component, FActorPickerID ActorPickerID, TArray<FGuid> ObjectBindings, UMovieSceneSection* Section) override;

private:

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, const TArray<TWeakObjectPtr<UObject>> Objects, FActorPickerID ActorPickerID);
};


