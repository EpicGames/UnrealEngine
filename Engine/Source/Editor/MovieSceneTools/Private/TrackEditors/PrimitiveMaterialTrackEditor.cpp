// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PrimitiveMaterialTrackEditor.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "Sections/MovieScenePrimitiveMaterialSection.h"
#include "ISequencerModule.h"
#include "Components/PrimitiveComponent.h"
#include "Components/DecalComponent.h"
#include "Materials/MaterialInterface.h"
#include "Algo/Find.h"
#include "Components/MeshComponent.h"
#include "Components/VolumetricCloudComponent.h"


#define LOCTEXT_NAMESPACE "PrimitiveMaterialTrackEditor"


FPrimitiveMaterialTrackEditor::FPrimitiveMaterialTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor(InSequencer)
{}

TSharedRef<ISequencerTrackEditor> FPrimitiveMaterialTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FPrimitiveMaterialTrackEditor>(OwningSequencer);
}

FText FPrimitiveMaterialTrackEditor::GetDisplayName() const
{
	return LOCTEXT("PrimitiveMaterialTrackEditor_DisplayName", "Primitive Material");
}

void FPrimitiveMaterialTrackEditor::ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UPrimitiveComponent::StaticClass()))
	{
		Extender->AddMenuExtension(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection, EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &FPrimitiveMaterialTrackEditor::ConstructObjectBindingTrackMenu, ObjectBindings));
	}
}

void FPrimitiveMaterialTrackEditor::ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings)
{
	auto GetMaterialInfoForTrack = [](UMovieSceneTrack* InTrack)
	{
		UMovieScenePrimitiveMaterialTrack* MaterialTrack = Cast<UMovieScenePrimitiveMaterialTrack>(InTrack);
		return MaterialTrack ? MaterialTrack->GetMaterialInfo() : FComponentMaterialInfo();
	};

	UObject* Object = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBindings[0]);
	if (!Object)
	{
		return;
	}

	USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
	if (!SceneComponent)
	{
		return;
	}	

	const UMovieScene* MovieScene = GetFocusedMovieScene();
	const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindings[0]);

	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent))
	{
		int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
		TArray<FName> MaterialSlotNames = PrimitiveComponent->GetMaterialSlotNames();
		UMeshComponent* MeshComponent = Cast<UMeshComponent>(SceneComponent);
		if (NumMaterials > 0 || MeshComponent)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("MaterialSwitcherTitle", "Material Switchers"));
			{
				for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
				{
					FName MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : FName();
					FComponentMaterialInfo MaterialInfo{ MaterialSlotName, MaterialIndex, EComponentMaterialType::IndexedMaterial };

					const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
					if (bAlreadyExists)
					{
						continue;
					}
					FUIAction AddMaterialSwitcherAction(FExecuteAction::CreateSP(this, &FPrimitiveMaterialTrackEditor::CreateTrackForElement, ObjectBindings, MaterialInfo));
					FText MaterialSwitcherLabel = !MaterialSlotName.IsNone() ?
						FText::Format(LOCTEXT("MaterialSlot_Format", "Material Slot {0} Switcher"), FText::FromName(MaterialSlotName)) :
						FText::Format(LOCTEXT("MaterialID_Format", "Material Element {0} Switcher"), FText::AsNumber(MaterialIndex));
					FText MaterialSwitcherTooltip = !MaterialSlotName.IsNone() ?
						FText::Format(LOCTEXT("MaterialSlotTooltip_Format", "Add material switcher for slot {0}, index {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex)) :
						FText::Format(LOCTEXT("MaterialIDTooltip_Format", "Add material switcher for element {0}"), FText::AsNumber(MaterialIndex));
					MenuBuilder.AddMenuEntry(MaterialSwitcherLabel, MaterialSwitcherTooltip, FSlateIcon(), AddMaterialSwitcherAction);
				}
				if (MeshComponent)
				{
					FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::OverlayMaterial };
					const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
					if (!bAlreadyExists)
					{
						FUIAction AddMaterialSwitcherAction(FExecuteAction::CreateSP(this, &FPrimitiveMaterialTrackEditor::CreateTrackForElement, ObjectBindings, MaterialInfo));
						FText OverlayMaterialSwitcherLabel = LOCTEXT("OverlayMaterialSwitcher_Format", "Overlay Material Switcher");
						FText OverlayMaterialSwitcherTooltip = LOCTEXT("OverlayMaterialSwitcherTooltip_Format", "Add overlay material switcher");
						MenuBuilder.AddMenuEntry(OverlayMaterialSwitcherLabel, OverlayMaterialSwitcherTooltip, FSlateIcon(), AddMaterialSwitcherAction);
					}

					const TArray<TObjectPtr<UMaterialInterface>>& SlotOverlayMaterials = MeshComponent->GetComponentMaterialSlotsOverlayMaterial();
					for (int32 SlotIndex = 0; SlotIndex < SlotOverlayMaterials.Num(); ++SlotIndex)
					{
						UMaterialInterface* SlotOverlayMaterial = SlotOverlayMaterials[SlotIndex];
						if (!SlotOverlayMaterial)
						{
							continue;
						}
						FName SlotName = MaterialSlotNames.IsValidIndex(SlotIndex) ? MaterialSlotNames[SlotIndex] : FName();
						FComponentMaterialInfo SlotMaterialInfo{ SlotName, SlotIndex, EComponentMaterialType::SlotOverlayMaterial };
						const bool bSlotAlreadyExists = Algo::FindBy(Binding->GetTracks(), SlotMaterialInfo, GetMaterialInfoForTrack) != nullptr;
						if (bSlotAlreadyExists)
						{
							continue;
						}
						FUIAction AddSlotOverlaySwitcherAction(FExecuteAction::CreateSP(this, &FPrimitiveMaterialTrackEditor::CreateTrackForElement, ObjectBindings, SlotMaterialInfo));
						FText SlotOverlaySwitcherLabel = !SlotName.IsNone() ?
							FText::Format(LOCTEXT("SlotOverlayMaterialSwitcher_Format", "Overlay Slot {0} Switcher"), FText::FromName(SlotName)) :
							FText::Format(LOCTEXT("IndexedSlotOverlayMaterialSwitcher_Format", "Overlay Element {0} Switcher"), FText::AsNumber(SlotIndex));
						FText SlotOverlaySwitcherTooltip = !SlotName.IsNone() ?
							FText::Format(LOCTEXT("SlotOverlayMaterialSwitcherTooltip_Format", "Add per-slot overlay material switcher for slot {0}, index {1}"), FText::FromName(SlotName), FText::AsNumber(SlotIndex)) :
							FText::Format(LOCTEXT("IndexedSlotOverlayMaterialSwitcherTooltip_Format", "Add per-slot overlay material switcher for element {0}"), FText::AsNumber(SlotIndex));
						MenuBuilder.AddMenuEntry(SlotOverlaySwitcherLabel, SlotOverlaySwitcherTooltip, FSlateIcon(), AddSlotOverlaySwitcherAction);
					}
				}
			}
			MenuBuilder.EndSection();
		}
	}
	else if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(SceneComponent))
	{
		MenuBuilder.BeginSection("Materials", LOCTEXT("MaterialSection", "Material Parameters"));
		{
			FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::DecalMaterial };
			const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
			if (!bAlreadyExists)
			{
				FUIAction AddMaterialSwitcherAction(FExecuteAction::CreateSP(this, &FPrimitiveMaterialTrackEditor::CreateTrackForElement, ObjectBindings, MaterialInfo));
				FText DecalMaterialSwitcherLabel = LOCTEXT("DecalMaterialSwitcher_Format", "Decal Material Switcher");
				FText DecalMaterialSwitcherTooltip = LOCTEXT("DecalMaterialSwitcherTooltip_Format", "Add decal material switcher");
				MenuBuilder.AddMenuEntry(DecalMaterialSwitcherLabel, DecalMaterialSwitcherTooltip, FSlateIcon(), AddMaterialSwitcherAction);
			}
		}
		MenuBuilder.EndSection();
	}
	else if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(SceneComponent))
	{
		MenuBuilder.BeginSection("Materials", LOCTEXT("MaterialSection", "Material Parameters"));
		{
			FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::VolumetricCloudMaterial };
			const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
			if (!bAlreadyExists)
			{
				FUIAction AddMaterialSwitcherAction(FExecuteAction::CreateSP(this, &FPrimitiveMaterialTrackEditor::CreateTrackForElement, ObjectBindings, MaterialInfo));
				FText CloudMaterialSwitcherLabel = LOCTEXT("CloudMaterialSwitcher_Format", "Volumetric Cloud Material Switcher");
				FText CloudMaterialSwitcherTooltip = LOCTEXT("CloudMaterialSwitcherTooltip_Format", "Add volumetric cloud material switcher");
				MenuBuilder.AddMenuEntry(CloudMaterialSwitcherLabel, CloudMaterialSwitcherTooltip, FSlateIcon(), AddMaterialSwitcherAction);
			}
		}
		MenuBuilder.EndSection();
	}
}

void FPrimitiveMaterialTrackEditor::CreateTrackForElement(TArray<FGuid> ObjectBindingIDs, FComponentMaterialInfo MaterialInfo)
{
	UMovieScene* MovieScene = GetFocusedMovieScene();

	FScopedTransaction Transaction(LOCTEXT("CreateTrack", "Create Material Track"));
	MovieScene->Modify();

	for (FGuid ObjectBindingID : ObjectBindingIDs)
	{
		UMovieScenePrimitiveMaterialTrack* NewTrack = MovieScene->AddTrack<UMovieScenePrimitiveMaterialTrack>(ObjectBindingID);
		NewTrack->SetMaterialInfo(MaterialInfo);
		// Construct display names from MaterialInfo
		FText TrackDisplayName;
		switch (MaterialInfo.MaterialType)
		{
		case EComponentMaterialType::Empty:
			break;
		case EComponentMaterialType::IndexedMaterial:
			TrackDisplayName = !MaterialInfo.MaterialSlotName.IsNone() ? FText::Format(LOCTEXT("SlotMaterialSwitcherTrackName", "Material Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName))
				: FText::Format(LOCTEXT("IndexedMaterialSwitcherTrackName", "Material Element {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));

			break;
		case EComponentMaterialType::OverlayMaterial:
			TrackDisplayName = LOCTEXT("OverlayMaterialSwitcherTrackName", "Overlay Material");
			break;
		case EComponentMaterialType::DecalMaterial:
			TrackDisplayName = LOCTEXT("DecalMaterialSwitcherTrackName", "Decal Material");
			break;		
		case EComponentMaterialType::VolumetricCloudMaterial:
				TrackDisplayName = LOCTEXT("CloudMaterialSwitcherTrackName", "Volumetric Cloud Material");
				break;
		case EComponentMaterialType::SlotOverlayMaterial:
			TrackDisplayName = !MaterialInfo.MaterialSlotName.IsNone() ?
				FText::Format(LOCTEXT("SlotOverlayMaterialSwitcherTrackName", "Overlay Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName)) :
				FText::Format(LOCTEXT("IndexedSlotOverlayMaterialSwitcherTrackName", "Overlay Element {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
			break;
		default:
			break;

		}
		if (!TrackDisplayName.IsEmpty())
		{
			NewTrack->SetDisplayName(TrackDisplayName);
		}

		NewTrack->AddSection(*NewTrack->CreateNewSection());
	}

	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FPrimitiveMaterialTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	if (UMovieScenePrimitiveMaterialTrack* MaterialTrack = Cast<UMovieScenePrimitiveMaterialTrack>(Track))
	{
		FGuid ObjectBinding = MaterialTrack->FindObjectBindingGuid();
		UObject* BoundObject = GetSequencer()->FindSpawnedObjectOrTemplate(ObjectBinding);
		const FComponentMaterialInfo& MaterialInfo = MaterialTrack->GetMaterialInfo();
		if (MaterialInfo.MaterialType == EComponentMaterialType::Empty || MaterialInfo.MaterialType == EComponentMaterialType::IndexedMaterial)
		{
			if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(BoundObject))
			{
				int32 FoundMaterialIndex = INDEX_NONE;
				FName FoundSlotName;
				UMaterialInterface* Material = nullptr;
				FText AutoRebindTooltip;
				bool bNeedsRebindFix = false;

				if (!MaterialInfo.MaterialSlotName.IsNone())
				{
					Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
					if (Material)
					{
						FoundSlotName = MaterialInfo.MaterialSlotName;
					}
					FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
					if (FoundMaterialIndex != INDEX_NONE && FoundMaterialIndex != MaterialInfo.MaterialSlotIndex)
					{
						bNeedsRebindFix = true;
						// Found material by slot name, but the indices don't match what is cached. Auto-rebind would change index to the one in the found slot name.
						AutoRebindTooltip = FText::Format(LOCTEXT("AutoRebindToNewSlotSwitcherIndex", "Rebind track to index {0}, keeping same slot {1}"), FText::AsNumber(FoundMaterialIndex), FText::FromName(MaterialInfo.MaterialSlotName));
					}
				}
				if (!Material)
				{
					bNeedsRebindFix = true;
					// Couldn't find binding by slot name or one wasn't specified, try by index
					Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
					if (Material)
					{
						// Found material by index, but not by slot name. Auto-rebind would change slot name to the current one
						FoundMaterialIndex = MaterialInfo.MaterialSlotIndex;
						TArray<FName> SlotNames = Component->GetMaterialSlotNames();
						if (SlotNames.IsValidIndex(MaterialInfo.MaterialSlotIndex))
						{
							FoundSlotName = SlotNames[MaterialInfo.MaterialSlotIndex];
							AutoRebindTooltip = FText::Format(LOCTEXT("AutoRebindToNewSlotSwitcherName", "Rebind track to slot {0}, keeping same index {1}"), FText::FromName(FoundSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
					// If we didn't find a material, we don't create a tooltip, because we won't be able to 'auto' rebind, just manually bind.
				}

				// If we didn't find a material, a slot name wasn't specified or wasn't found, or an index was mismatched, allow the user to rebind
				if (bNeedsRebindFix)
				{
					MenuBuilder.BeginSection("Fix Material Switcher Binding", LOCTEXT("FixMaterialBindingSwitcherSectionName", "Fix Material Switcher Binding"));

					// If we set this tooltip, then we are able to auto-rebind
					if (!AutoRebindTooltip.IsEmpty())
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("AutoFixMaterialSwitcherBinding", "Auto-Fix Material Switcher Binding"),
							AutoRebindTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([MaterialTrack, FoundMaterialIndex, FoundSlotName]()
									{
										FScopedTransaction Transaction(LOCTEXT("FixupMaterialSwitcherBinding", "Fixup material switcher binding"));
										MaterialTrack->Modify();
										MaterialTrack->SetMaterialInfo(FComponentMaterialInfo{ FoundSlotName, FoundMaterialIndex, EComponentMaterialType::IndexedMaterial });
										MaterialTrack->SetDisplayName(FText::Format(LOCTEXT("SlotMaterialSwitcherTrackName", "Material Slot: {0}"), FText::FromName(FoundSlotName)));
									})));
					}

					MenuBuilder.AddSubMenu(
						LOCTEXT("RebindMaterialSwitcher", "Re-Bind Material Switcher Track..."),
						LOCTEXT("RebindMaterialSwitcherTooltip", "Re-Bind this material switcher track to a different material slot"),
						FNewMenuDelegate::CreateSP(this, &FPrimitiveMaterialTrackEditor::FillRebindMaterialTrackMenu, MaterialTrack, Component, ObjectBinding)
					);

					MenuBuilder.EndSection();
					MenuBuilder.AddMenuSeparator();
				}
			}
		}
	}
}

void FPrimitiveMaterialTrackEditor::FillRebindMaterialTrackMenu(FMenuBuilder& MenuBuilder, UMovieScenePrimitiveMaterialTrack* MaterialTrack, UPrimitiveComponent* PrimitiveComponent, FGuid ObjectBinding)
{
	auto GetMaterialInfoForTrack = [](UMovieSceneTrack* InTrack)
	{
		UMovieScenePrimitiveMaterialTrack* MaterialTrack = Cast<UMovieScenePrimitiveMaterialTrack>(InTrack);
		return MaterialTrack ? MaterialTrack->GetMaterialInfo() : FComponentMaterialInfo();
	};
	int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
	TArray<FName> MaterialSlotNames = PrimitiveComponent->GetMaterialSlotNames();
	const UMovieScene* MovieScene = GetFocusedMovieScene();
	const FMovieSceneBinding* Binding = Algo::FindBy(MovieScene->GetBindings(), ObjectBinding, &FMovieSceneBinding::GetObjectGuid);
	if (Binding && NumMaterials > 0)
	{
		MenuBuilder.BeginSection("RebindMaterialSlots", LOCTEXT("RebindMaterialSlots", "Material Slots"));
		{
			for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
			{
				FName MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : FName();
				FComponentMaterialInfo MaterialInfo{ MaterialSlotName, MaterialIndex, EComponentMaterialType::IndexedMaterial };
				const bool bAlreadyExists = Algo::FindBy(Binding->GetTracks(), MaterialInfo, GetMaterialInfoForTrack) != nullptr;
				if (bAlreadyExists)
				{
					continue;
				}
				FUIAction RebindToMaterialAction(FExecuteAction::CreateLambda([MaterialTrack, MaterialInfo]()
					{
						FScopedTransaction Transaction(LOCTEXT("FixupMaterialSwitcherBinding", "Fixup material switcher binding"));
						MaterialTrack->Modify();
						MaterialTrack->SetMaterialInfo(MaterialInfo);
						MaterialTrack->SetDisplayName(FText::Format(LOCTEXT("SlotMaterialSwitcherTrackName", "Material Slot: {0}"), FText::FromName(MaterialInfo.MaterialSlotName)));
					}));
				FText RebindToMaterialLabel = FText::Format(LOCTEXT("RebindToMaterialSwitcherSlot", "Slot: {0}, Index: {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex));
				FText RebindToMaterialToolTip = FText::Format(LOCTEXT("RebindToMaterialSwitcherSlotToolTip", "Rebind this track to material slot {0}, index {1}"), FText::FromName(MaterialSlotName), FText::AsNumber(MaterialIndex));
				MenuBuilder.AddMenuEntry(RebindToMaterialLabel, RebindToMaterialToolTip, FSlateIcon(), RebindToMaterialAction);
			}
		}
		MenuBuilder.EndSection();
	}
}

#undef LOCTEXT_NAMESPACE
