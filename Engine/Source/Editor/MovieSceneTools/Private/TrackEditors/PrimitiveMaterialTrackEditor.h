// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "Materials/MaterialInterface.h"

class FPrimitiveMaterialTrackEditor : public FKeyframeTrackEditor<UMovieScenePrimitiveMaterialTrack>
{
public:

	/** Constructor. */
	FPrimitiveMaterialTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Factory function */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	//~ ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;

	virtual void ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

	void ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);
	void CreateTrackForElement(TArray<FGuid> ObjectBindingIDs, FComponentMaterialInfo MaterialInfo);
	/** Callback for rebinding a component material track to a different material slot */
	void FillRebindMaterialTrackMenu(FMenuBuilder& MenuBuilder, class UMovieScenePrimitiveMaterialTrack* MaterialTrack, class UPrimitiveComponent* Component, FGuid ObjectBinding);
};
