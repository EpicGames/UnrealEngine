// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PropertyTrackEditors/ActorReferencePropertyTrackEditor.h"

#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "KeyPropertyParams.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "MovieSceneTrackEditor.h"
#include "Sections/MovieSceneActorReferenceSection.h"
#include "Templates/Casts.h"

class ISequencerTrackEditor;
class UMovieSceneSection;
class UObject;

#define LOCTEXT_NAMESPACE "ActorReferencePropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FActorReferencePropertyTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable(new FActorReferencePropertyTrackEditor(OwningSequencer));
}

FText FActorReferencePropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ActorReferencePropertyTrackEditor_DisplayName", "Actor Reference Property");
}

void FActorReferencePropertyTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	Operation.ApplyDefault(InKeyTime, InSequencer, OutResults);
}

void FActorReferencePropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	// Care is taken here to ensure that GetPropertyValue is templated on UObject* which causes it to use the correct instantiation of GetPropertyValueImpl
	AActor* NewReferencedActor = Cast<AActor>(PropertyChangedParams.GetPropertyValue<UObject*>());
	if (NewReferencedActor != nullptr)
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

		FMovieSceneObjectBindingID Binding;

		TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(NewReferencedActor);
		if (Spawnable.IsSet())
		{
			// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
			Binding = UE::MovieScene::FRelativeObjectBindingID(SequencerPtr->GetFocusedTemplateID(), Spawnable->SequenceID, Spawnable->ObjectBindingID, *SequencerPtr);
		}
		else
		{
			FGuid ParentActorId = FindOrCreateHandleToObject(NewReferencedActor).Handle;
			Binding = UE::MovieScene::FRelativeObjectBindingID(ParentActorId);
		}

		if (Binding.IsValid())
		{
			FMovieSceneActorReferenceKey NewKey;
			NewKey.Object = Binding;
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneActorReferenceChannel>(0, NewKey, true));
		}
	}
}

void FActorReferencePropertyTrackEditor::OnAnimatedPropertyChanged(const FPropertyChangedParams& PropertyChangedParams)
{
	// When the changed object is a spawnable, force a manual key so that the change will generate an actor reference track. 
	// Otherwise, the object will get referenced by an unstable soft object ptr.
	FPropertyChangedParams NewPropertyChangedParams(PropertyChangedParams);

	AActor* NewReferencedActor = Cast<AActor>(PropertyChangedParams.GetPropertyValue<UObject*>());
	if (NewReferencedActor != nullptr)
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

		TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(NewReferencedActor);
		if (Spawnable.IsSet())
		{
			NewPropertyChangedParams.KeyMode = ESequencerKeyMode::ManualKey;
		}
	}

	FPropertyTrackEditor::OnAnimatedPropertyChanged(NewPropertyChangedParams);
}

#undef LOCTEXT_NAMESPACE
