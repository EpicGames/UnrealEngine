// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneActorReferenceTrack.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

class FPropertyChangedParams;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

/**
 * A property track editor for actor references.
 */
class FActorReferencePropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneActorReferenceTrack>
{
public:

	/**
	 * Constructor.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FActorReferencePropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{
	}

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		FAnimatedPropertyKey Key = FAnimatedPropertyKey::FromPropertyType(FSoftObjectProperty::StaticClass());
		Key.ObjectTypeName = AActor::StaticClass()->GetFName();

		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ Key, FAnimatedPropertyKey::FromObjectType(AActor::StaticClass()) });
	}

	/**
	 * Creates an instance of this class (called by a sequencer).
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:

	//~ FPropertyTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) override;
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	virtual void OnAnimatedPropertyChanged(const FPropertyChangedParams& PropertyChangedParams) override;
};
