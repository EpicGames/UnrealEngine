// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PropertyTrackEditors/BytePropertyTrackEditor.h"

#include "Channels/MovieSceneByteChannel.h"
#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "ISequencer.h"
#include "KeyPropertyParams.h"
#include "MovieSceneTrack.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ISequencerTrackEditor;
class UMovieScene;
class UMovieSceneSection;
struct FGuid;

#define LOCTEXT_NAMESPACE "BytePropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FBytePropertyTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable(new FBytePropertyTrackEditor(OwningSequencer));
}

UEnum* GetEnumForByteTrack(TSharedPtr<ISequencer> Sequencer, const FGuid& OwnerObjectHandle, const FPropertyChangedParams& PropertyChangedParams, UMovieSceneByteTrack* ByteTrack)
{
	TSet<UEnum*> PropertyEnums;

	for (TWeakObjectPtr<> WeakObject : Sequencer->FindObjectsInCurrentSequence(OwnerObjectHandle))
	{
		UObject* RuntimeObject = WeakObject.Get();
		if (!RuntimeObject)
		{
			continue;
		}

		const FProperty* Property = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();
		if (Property != nullptr)
		{
			UEnum* Enum = nullptr;

			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				Enum = EnumProperty->GetEnum();
			}
			else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
			{
				Enum = ByteProperty->Enum;
			}

			if (Enum != nullptr)
			{
				PropertyEnums.Add(Enum);
			}
		}
	}

	UEnum* TrackEnum;

	if (PropertyEnums.Num() == 1)
	{
		TrackEnum = PropertyEnums.Array()[0];
	}
	else
	{
		TrackEnum = nullptr;
	}

	return TrackEnum;
}

FText FBytePropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("BytePropertyTrackEditor_DisplayName", "Byte Property");
}

void FBytePropertyTrackEditor::GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys )
{
	uint8 KeyedValue = PropertyChangedParams.GetPropertyValue<uint8>();
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneByteChannel>(0, KeyedValue, true));
}

void FBytePropertyTrackEditor::InitializeNewTrack(UMovieSceneByteTrack* NewTrack, FPropertyChangedParams PropertyChangedParams)
{
	FPropertyTrackEditor<UMovieSceneByteTrack>::InitializeNewTrack(NewTrack, PropertyChangedParams);
	if (NewTrack)
	{
		UEnum* TrackEnum = GetEnumForByteTrack(GetSequencer(), NewTrack->FindObjectBindingGuid(), PropertyChangedParams, NewTrack);

		if (TrackEnum != nullptr)
		{
			NewTrack->SetEnum(TrackEnum);
		}
	}

}

#undef LOCTEXT_NAMESPACE
