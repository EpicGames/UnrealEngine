// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PropertyTrackEditors/RegisteredPropertyTrackEditor.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Algo/IndexOf.h"
#include "IKeyArea.h"

#include "MovieSceneTracksComponentTypes.h"


namespace UE::Sequencer
{

struct FRegisteredPropertyRegistry
{
	static FRegisteredPropertyRegistry& Get()
	{
		static FRegisteredPropertyRegistry Registry;
		return Registry;
	}

	TArray<TSharedPtr<IRegisteredProperty>> RegisteredPropertyTypes;
};

void RegisterPropertyType(TSharedPtr<IRegisteredProperty> RegisteredProperty)
{
	FRegisteredPropertyRegistry::Get().RegisteredPropertyTypes.Add(RegisteredProperty);
}



TSharedRef<ISequencerTrackEditor> FRegisteredPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FRegisteredPropertyTrackEditor>(OwningSequencer);
}

FRegisteredPropertyTrackEditor::FRegisteredPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FPropertyTrackEditor<UMovieScenePropertyTrack>(InSequencer)
{
	using namespace UE::Sequencer;

	FRegisteredPropertyRegistry&    PropertyRegistry     = FRegisteredPropertyRegistry::Get();
	ISequencerObjectChangeListener& ObjectChangeListener = InSequencer->GetObjectChangeListener();

	ObjectChangeListener.AddFindPropertyChangedHandler(
		FFindAnimatablePropertyChangedHandler::CreateRaw(this, &FRegisteredPropertyTrackEditor::FindPropertyChangedHandler)
	);
}

bool FRegisteredPropertyTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const
{
	using namespace UE::MovieScene;

	const FPropertyRegistry& PropertyRegistry = FBuiltInComponentTypes::Get()->PropertyRegistry;

	for (const FPropertyDefinition& PropertyDefinition : PropertyRegistry.GetProperties())
	{
		UClass* DefaultTrackType = PropertyDefinition.DefaultTrackType.Get();
		if (DefaultTrackType && TrackClass->IsChildOf(DefaultTrackType))
		{
			return true;
		}
	}
	return false;
}

TSharedRef<ISequencerSection> FRegisteredPropertyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	for (TSharedPtr<IRegisteredProperty> Property : FRegisteredPropertyRegistry::Get().RegisteredPropertyTypes)
	{
		TSharedPtr<ISequencerSection> SectionInterface = Property->AppliesToTrack(&Track) && Property->AppliesToSection(&SectionObject)
			? Property->TryMakeSectionInterface(&SectionObject, &Track, ObjectBinding, GetSequencer())
			: nullptr;

		if (SectionInterface)
		{
			return SectionInterface.ToSharedRef();
		}
	}
	return MakeShared<FSequencerSection>(SectionObject);
}

const UE::MovieScene::FPropertyDefinition* FRegisteredPropertyTrackEditor::FindMatchingPropertyDefinition(const FProperty* InProperty)
{
	using namespace UE::MovieScene;

	const FPropertyRegistry& PropertyRegistry = FBuiltInComponentTypes::Get()->PropertyRegistry;

	// Find the property that applies to this
	for (const FPropertyDefinition& PropertyDefinition : PropertyRegistry.GetProperties())
	{
		if (PropertyDefinition.DefaultTrackType && PropertyDefinition.Handler->SupportsProperty(PropertyDefinition, *InProperty))
		{
			return &PropertyDefinition;
		}
	}

	return nullptr;
}

FOnAnimatablePropertyChanged FRegisteredPropertyTrackEditor::FindPropertyChangedHandler(const FProperty* InProperty) const
{
	const UE::MovieScene::FPropertyDefinition* PropertyDefinition = FindMatchingPropertyDefinition(InProperty);
	if (PropertyDefinition)
	{
		FRegisteredPropertyTrackEditor* This = const_cast<FRegisteredPropertyTrackEditor*>(this);

		FOnAnimatablePropertyChanged Delegate;
		Delegate.AddRaw(This, &FRegisteredPropertyTrackEditor::HandlePropertyChanged, PropertyDefinition);
		return Delegate;
	}

	return FOnAnimatablePropertyChanged();
}

void FRegisteredPropertyTrackEditor::HandlePropertyChanged(const FPropertyChangedParams& PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* Property)
{
	using namespace UE::Sequencer;
	using namespace UE::MovieScene;

	AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FRegisteredPropertyTrackEditor::KeyProperty, PropertyChangedParams, Property));
}


TSubclassOf<UMovieSceneTrack> FRegisteredPropertyTrackEditor::GetCustomizedTrackClass(const FProperty* Property) const
{
	// Look for a customized track class for this property on the meta data
	const FString& MetaSequencerTrackClass = Property->GetMetaData(TEXT("SequencerTrackClass"));
	if (!MetaSequencerTrackClass.IsEmpty())
	{
		UClass* MetaClass = UClass::TryFindTypeSlow<UClass>(MetaSequencerTrackClass);
		if (!MetaClass)
		{
			MetaClass = LoadObject<UClass>(nullptr, *MetaSequencerTrackClass);
		}
		return MetaClass;
	}

	const UE::MovieScene::FPropertyDefinition* PropertyDefinition = FindMatchingPropertyDefinition(Property);
	if (PropertyDefinition && PropertyDefinition->TraitsInstance)
	{
		return PropertyDefinition->TraitsInstance->GetTrackClass(Property).Get();
	}
	return nullptr;
}

void FRegisteredPropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	const FProperty* Property = PropertyChangedParams.GetPropertyAndValue().Key;
	if (!Property)
	{
		return;
	}

	const UE::MovieScene::FPropertyDefinition* PropertyDefinition = FindMatchingPropertyDefinition(Property);
	if (PropertyDefinition)
	{
		GenerateKeysFromPropertyChanged(*PropertyDefinition, *GetSequencer(), PropertyChangedParams, SectionToKey, OutGeneratedKeys);
	}
}

void FRegisteredPropertyTrackEditor::GenerateKeysFromPropertyChanged(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, ISequencer& Sequencer, const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	using namespace UE::MovieScene;

	TPair<const FProperty*, FSourcePropertyValue> PropertyAndValue = PropertyChangedParams.GetPropertyAndValue();
	if (!PropertyAndValue.Key)
	{
		return;
	}

	// Allow specialized tracks to process the source value (e.g. UMovieSceneVisibilityTrack inverts bool).
	if (UMovieScenePropertyTrack* PropertyTrack = SectionToKey ? SectionToKey->GetTypedOuter<UMovieScenePropertyTrack>() : nullptr)
	{
		PropertyTrack->ProcessSourceValueForKeying(PropertyAndValue.Key, PropertyAndValue.Value);
	}

	if (PropertyDefinition.Handler->SupportsProperty(PropertyDefinition, *PropertyAndValue.Key))
	{
		// Coerce the value to its operating type
		FIntermediatePropertyValue CoercedValue    = PropertyDefinition.Handler->CoercePropertyValue(PropertyDefinition, *PropertyAndValue.Key, PropertyAndValue.Value);

		// Recompose the value from any blending that might be present
		FIntermediatePropertyValue RecomposedValue = RecomposeValue(PropertyDefinition, Sequencer, CoercedValue, PropertyChangedParams.ObjectsThatChanged[0], SectionToKey);

		// Unpack the resulting value to its constituent channels
		FUnpackedChannelValues UnpackedChannelValues;
		PropertyDefinition.Handler->UnpackChannels(PropertyDefinition, *PropertyAndValue.Key, RecomposedValue, UnpackedChannelValues);

		FString LeafPath;
		FString QualifiedLeafPath;

		// First off, get the leaf-most property name
		int32 NumKeyedProperties = PropertyChangedParams.StructPathToKey.GetNumProperties();
		if (NumKeyedProperties >= 1)
		{
			LeafPath = PropertyChangedParams.StructPathToKey.GetPropertyInfo(NumKeyedProperties-1).Property->GetName();
		}

		// If we have 2 or more properties, add the penultimate property name to the path as well.
		if (NumKeyedProperties >= 2)
		{
			QualifiedLeafPath = PropertyChangedParams.StructPathToKey.GetPropertyInfo(NumKeyedProperties-2).Property->GetName();
			QualifiedLeafPath.AppendChar('.');
			QualifiedLeafPath += LeafPath;
		}

		// Generate key setters for each of the unpacked values
		for (FUnpackedChannelValue& UnpackedValue : UnpackedChannelValues.GetValues())
		{
			bool bAddKey = true;

			if (LeafPath.Len() > 0 && UnpackedValue.GetPropertyPath() != NAME_None)
			{
				const FString PropertyPath          = UnpackedValue.GetPropertyPath().ToString();
				const bool    bMatchesQualifiedPath = FPlatformString::Stricmp(*QualifiedLeafPath, *PropertyPath) == 0;

				// bMatchesLeafPath intentionally matches situations where the Leaf is a parent property of the unpacked value.
				//   For example, a channel name "SpcifiedColor.R" should match with a changed property "SpecifiedColor"
				const bool    bMatchesLeafPath      = FCString::Strnicmp(*PropertyPath, *LeafPath, LeafPath.Len()) == 0;

				bAddKey = bMatchesQualifiedPath || bMatchesLeafPath;
			}

			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create(MoveTemp(UnpackedValue), bAddKey));
		}
	}
}

FKeyPropertyResult FRegisteredPropertyTrackEditor::KeyProperty(FFrameNumber KeyTime, FPropertyChangedParams PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* PropertyDefinition)
{
	const FProperty* Property = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();

	TSubclassOf<UMovieSceneTrack> TrackType = Property ? GetCustomizedTrackClass(Property) : nullptr;
	if (TrackType == nullptr)
	{
		TrackType = PropertyDefinition->DefaultTrackType;
	}

	if (!ensure(TrackType))
	{
		return FKeyPropertyResult();
	}

	FName UniqueName(*PropertyChangedParams.PropertyPath.ToString(TEXT(".")));

	auto GenerateKeysCallback = [this, PropertyDefinition, &PropertyChangedParams](UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys)
	{
		this->GenerateKeysFromPropertyChanged(*PropertyDefinition, *this->GetSequencer(), PropertyChangedParams, Section, OutGeneratedKeys);
	};

	auto InitializeTrackCallback = [this, &PropertyChangedParams, PropertyDefinition](UMovieScenePropertyTrack* NewTrack)
	{
		this->InitializeNewTrack(NewTrack, PropertyChangedParams, PropertyDefinition);
	};

	return AddKeysToObjects(
		PropertyChangedParams.ObjectsThatChanged,
		KeyTime,
		PropertyChangedParams.KeyMode,
		TrackType,
		UniqueName,
		InitializeTrackCallback,
		GenerateKeysCallback
	);
}

void FRegisteredPropertyTrackEditor::InitializeNewTrack(UMovieScenePropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* PropertyDefinition)
{
	const FProperty* Property = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();

	if (!PropertyDefinition->TraitsInstance || !PropertyDefinition->TraitsInstance->InitializeTrackFromProperty(NewTrack, Property))
	{
		NewTrack->InitializeFromProperty(Property, PropertyDefinition);
	}

	FPropertyTrackEditor<UMovieScenePropertyTrack>::InitializeNewTrack(NewTrack, PropertyChangedParams);
}

void FRegisteredPropertyTrackEditor::InitializeNewTrack(UMovieScenePropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams)
{
	ensureMsgf(false, TEXT("This function should not be called. Please use the overload that takes a property definition"));
	FPropertyTrackEditor<UMovieScenePropertyTrack>::InitializeNewTrack(NewTrack, PropertyChangedParams);
}

void FRegisteredPropertyTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::Sequencer;

	auto Iterator = [this, InKeyTime, &Operation, &InSequencer, &OutResults](UMovieSceneTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> Operations)
	{
		FGuid ObjectBinding = Track->FindObjectBindingGuid();
		if (ObjectBinding.IsValid())
		{
			for (TWeakObjectPtr<> WeakObject : InSequencer.FindBoundObjects(ObjectBinding, InSequencer.GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					this->ProcessKeyOperation(Object, Operations, InSequencer, InKeyTime, OutResults);
					return;
				}
			}
		}

		// Default behavior
		FKeyOperation::ApplyOperations(InKeyTime, Operations, ObjectBinding, InSequencer, OutResults);
	};

	Operation.IterateOperations(Iterator);
}


void FRegisteredPropertyTrackEditor::ProcessKeyOperation(UObject* Object, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::MovieScene;

	UMovieScenePropertyTrack* Track = nullptr;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneSequenceID SequenceID = GetSequencer()->GetFocusedTemplateID();
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();
	UMovieSceneEntitySystemLinker* Linker = EvaluationTemplate.GetEntitySystemLinker();

	TArray<FMovieSceneEntityID> Entities;
	for (const FKeySectionOperation& Operation : SectionsToKey)
	{
		if (!Track)
		{
			Track = CastChecked<UMovieScenePropertyTrack>(Operation.Section->GetSectionObject()->GetOuter());
		}

		FMovieSceneEntityID EntityID = EvaluationTemplate.FindEntityFromOwner(Operation.Section->GetSectionObject(), 0, SequenceID);
		if (!EntityID)
		{
			continue;
		}

		// Find an entity that matches the bound object
		Linker->EntityManager.IterateImmediateChildren(EntityID,
			[Object, Linker, BuiltInComponents, &Entities](FMovieSceneEntityID ChildEntityID)
			{
				TOptionalComponentReader<UObject*> BoundObject = Linker->EntityManager.ReadComponent(ChildEntityID, BuiltInComponents->BoundObject);
				if (BoundObject && *BoundObject == Object)
				{
					Entities.Add(ChildEntityID);
				}
			}
		);
	}

	if (!ensure(Track))
	{
		return;
	}

	TOptional<TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue>> PropertyAndValue = FTrackInstancePropertyBindings::StaticPropertyAndValue(Object, Track->GetPropertyPath().ToString());
	if (!PropertyAndValue.IsSet())
	{
		return;
	}

	// Allow specialized tracks to process the source value (e.g. UMovieSceneVisibilityTrack inverts bool).
	Track->ProcessSourceValueForKeying(PropertyAndValue->Key, PropertyAndValue->Value);

	const UE::MovieScene::FPropertyDefinition* PropertyDefinition = FindMatchingPropertyDefinition(PropertyAndValue->Key);
	if (!PropertyDefinition)
	{
		return;
	}

	UMovieSceneEntitySystemLinker*         EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	UMovieScenePropertyInstantiatorSystem* System       = EntityLinker ? EntityLinker->FindSystem<UMovieScenePropertyInstantiatorSystem>() : nullptr;

	// Coerce the value to its operating type
	FIntermediatePropertyValue CoercedValue = PropertyDefinition->Handler->CoercePropertyValue(*PropertyDefinition, *PropertyAndValue->Key, PropertyAndValue->Value);

	// Recompose the value if possible
	if (System && Entities.Num() != 0)
	{
		FDecompositionQuery Query;
		Query.Entities = Entities;
		Query.Object = Object;
		Query.bConvertFromSourceEntityIDs = false;

		TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &EntityLinker->EntityManager);

		// Recompose the value from any blending that might be present
		FPropertyRecomposerImpl Recomposer;
		Recomposer.OnGetPropertyInfo = FOnGetPropertyRecomposerPropertyInfo::CreateUObject(
			System, &UMovieScenePropertyInstantiatorSystem::FindPropertyFromSource);

		CoercedValue = MoveTemp(Recomposer.RecomposeBlendOperational(*PropertyDefinition, Query, CoercedValue).Values[0]);
	}

	// Unpack the resulting value to its constituent channels
	FUnpackedChannelValues UnpackedChannelValues;
	PropertyDefinition->Handler->UnpackChannels(*PropertyDefinition, *PropertyAndValue->Key, CoercedValue, UnpackedChannelValues);

	if (UnpackedChannelValues.GetValues().Num() == 0)
	{
		return;
	}

	for (const FKeySectionOperation& Operation : SectionsToKey)
	{
		UMovieSceneSection* SectionObject = Operation.Section->GetSectionObject();
		if (!SectionObject)
		{
			continue;
		}

		FMovieSceneChannelProxy& ChannelProxy = SectionObject->GetChannelProxy();

		// Figure out which channels relate to which indices in the unpacked values
		TMap<FMovieSceneChannel*, int32> ChannelToIndex;
		for (int32 Index = 0; Index < UnpackedChannelValues.Num(); ++Index)
		{
			if (FMovieSceneChannel* Channel = UnpackedChannelValues[Index]->RetrieveChannel(ChannelProxy))
			{
				ChannelToIndex.Add(Channel, Index);
			}
		}

		for (int32 Index = 0; Index < Operation.KeyAreas.Num(); ++Index)
		{
			TSharedPtr<IKeyArea> KeyArea = Operation.KeyAreas[Index];
			FMovieSceneChannel* Channel = KeyArea->ResolveChannel();
			if (!Channel)
			{
				continue;
			}

			const int32* UnpackedIndex = ChannelToIndex.Find(Channel);
			if (UnpackedIndex == nullptr || !SectionObject->TryModify())
			{
				continue;
			}

			EMovieSceneKeyInterpolation InterpolationMode = KeyArea->GetInterpolationMode(KeyTime, GetSequencer()->GetKeyInterpolation());
			UnpackedChannelValues[*UnpackedIndex]->AddKey(Channel, KeyTime, InterpolationMode);

			if (OutResults)
			{
				TArray<FKeyHandle> Handles;
				KeyArea->GetKeyHandles(Handles, TRange<FFrameNumber>(KeyTime));
				for (FKeyHandle Handle : Handles)
				{
					OutResults->Add({ KeyArea, Handle });
				}
			}
		}
	}
}

UE::MovieScene::FIntermediatePropertyValue FRegisteredPropertyTrackEditor::RecomposeValue(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, ISequencer& Sequencer, const UE::MovieScene::FIntermediatePropertyValue& InCurrentValue, UObject* AnimatedObject, UMovieSceneSection* Section)
{
	using namespace UE::MovieScene;

	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer.GetEvaluationTemplate();

	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, EntityLinker ? &EntityLinker->EntityManager : nullptr);

	FMovieSceneEntityID EntityID = EvaluationTemplate.FindEntityFromOwner(Section, 0, Sequencer.GetFocusedTemplateID());

	if (EntityLinker && EntityID)
	{
		UMovieScenePropertyInstantiatorSystem* System = EntityLinker->FindSystem<UMovieScenePropertyInstantiatorSystem>();
		if (System)
		{
			FDecompositionQuery Query;
			Query.Entities = MakeArrayView(&EntityID, 1);
			Query.Object   = AnimatedObject;

			FPropertyRecomposerImpl Recomposer;

			Recomposer.OnGetPropertyInfo = FOnGetPropertyRecomposerPropertyInfo::CreateUObject(
						System, &UMovieScenePropertyInstantiatorSystem::FindPropertyFromSource);

			return MoveTemp(Recomposer.RecomposeBlendOperational(PropertyDefinition, Query, InCurrentValue).Values[0]);
		}
	}

	return InCurrentValue.Copy();
}


} // namespace UE::Sequencer