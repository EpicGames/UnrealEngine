// Copyright Epic Games, Inc. All Rights Reserved.

#include "RotatorPropertyTrackEditor.h"

#include "MovieSceneTracksComponentTypes.h"

#define LOCTEXT_NAMESPACE "RotatorPropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FRotatorPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InOwningSequencer)
{
	return MakeShared<FRotatorPropertyTrackEditor>(InOwningSequencer);
}

FText FRotatorPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("RotatorPropertyTrackEditor_DisplayName", "Rotator Property");
}

void FRotatorPropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& InPropertyChangedParams, UMovieSceneSection* InSectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	using namespace  UE::MovieScene;

	const FPropertyPath& StructPath = InPropertyChangedParams.StructPathToKey;

	const FName ChannelName = StructPath.GetNumProperties() != 0
		? StructPath.GetLeafMostProperty().Property->GetFName()
		: NAME_None;

	const FRotator CurrentRotator = InPropertyChangedParams.GetPropertyValue<FRotator>();

	const bool bKeyRoll = ChannelName == GET_MEMBER_NAME_CHECKED(FRotator, Roll) || ChannelName == NAME_None;
	const bool bKeyPitch = ChannelName == GET_MEMBER_NAME_CHECKED(FRotator, Pitch) || ChannelName == NAME_None;
	const bool bKeyYaw = ChannelName == GET_MEMBER_NAME_CHECKED(FRotator, Yaw) || ChannelName == NAME_None;

	int32 ChannelIndex = 0;

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelIndex++
		, CurrentRotator.Pitch
		, bKeyPitch));

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelIndex++
		, CurrentRotator.Yaw
		, bKeyYaw));

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelIndex
		, CurrentRotator.Roll
		, bKeyRoll));
}

#undef LOCTEXT_NAMESPACE
