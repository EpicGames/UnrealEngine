// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "KeyPropertyParams.h"
#include "Misc/Guid.h"
#include "PropertyTrackEditor.h"
#include "Tracks/MovieSceneRotatorTrack.h"

/** Property track editor for rotator properties */
class FRotatorPropertyTrackEditor : public FPropertyTrackEditor<UMovieSceneRotatorTrack>
{
public:
	FRotatorPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{}

	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>(
			{
				FAnimatedPropertyKey::FromStructType(NAME_Rotator)
			}
		);
	}

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InOwningSequencer);

protected:
	//~ Begin FPropertyTrackEditor
	virtual FText GetDisplayName() const override;
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& InPropertyChangedParams, UMovieSceneSection* InSectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	//~ End FPropertyTrackEditor
};
