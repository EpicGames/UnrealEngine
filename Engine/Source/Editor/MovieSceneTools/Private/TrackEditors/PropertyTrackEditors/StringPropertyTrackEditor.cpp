// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PropertyTrackEditors/StringPropertyTrackEditor.h"

#include "Channels/MovieSceneStringChannel.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "KeyPropertyParams.h"
#include "PropertyPath.h"
#include "Templates/RemoveReference.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

#define LOCTEXT_NAMESPACE "StringPropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FStringPropertyTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable(new FStringPropertyTrackEditor(OwningSequencer));
}

FText FStringPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("StringPropertyTrackEditor_DisplayName", "String Property");
}

void FStringPropertyTrackEditor::GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys )
{
	FString KeyedValue = PropertyChangedParams.GetPropertyValue<FString>();
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneStringChannel>(0, MoveTemp(KeyedValue), true));
}

#undef LOCTEXT_NAMESPACE
