// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneStringTrack.h"
#include "UObject/UnrealNames.h"

class FPropertyChangedParams;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

/**
 * A property track editor for strings.
 */
class FStringPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneStringTrack>
{
public:

	/**
	 * Constructor.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FStringPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{ }

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromPropertyTypeName(NAME_StrProperty) });
	}

	/**
	 * Creates an instance of this class (called by a sequencer).
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:

	//~ FPropertyTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
};
