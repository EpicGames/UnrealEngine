// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextPropertyTrackEditor.h"
#include "Channels/MovieSceneTextChannel.h"
#include "ISequencerChannelInterface.h"
#include "SequencerKeyStructGenerator.h"
#include "UObject/TextProperty.h"

TSharedRef<ISequencerTrackEditor> FTextPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
{
	return MakeShared<FTextPropertyTrackEditor>(OwningSequencer);
}

void FTextPropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams
	, UMovieSceneSection* SectionToKey
	, FGeneratedTrackKeys& OutGeneratedKeys)
{
	const FTextProperty* TextProperty = CastField<const FTextProperty>(PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get());
	if (!TextProperty)
	{
		return;
	}

	FText TextPropertyValue = PropertyChangedParams.GetPropertyValue<FText>();
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneTextChannel>(0, MoveTemp(TextPropertyValue), true));
}
