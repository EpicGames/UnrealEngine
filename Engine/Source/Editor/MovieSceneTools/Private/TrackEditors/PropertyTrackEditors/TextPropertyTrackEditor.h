// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrackEditor.h"
#include "Tracks/MovieSceneTextTrack.h"

class UMovieSceneKeyStructType;
class FSequencerKeyStructGenerator;
struct FMovieSceneTextChannel;

/** Track Editor for Text Property */
class FTextPropertyTrackEditor : public FPropertyTrackEditor<UMovieSceneTextTrack>
{
public:
	FTextPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{
	}

	/** Retrieve a list of all property types that this track editor animates  */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromPropertyTypeName(NAME_TextProperty) });
	}

	/**
	 * Creates an instance of this class (called by a sequencer).
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:
	//~ Begin FPropertyTrackEditor
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	//~ End FPropertyTrackEditor
};
