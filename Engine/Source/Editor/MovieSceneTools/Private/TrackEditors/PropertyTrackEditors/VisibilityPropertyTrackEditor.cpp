// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/PropertyTrackEditors/VisibilityPropertyTrackEditor.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "KeyPropertyParams.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Sections/BoolPropertySection.h"

class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UMovieSceneTrack;

#define LOCTEXT_NAMESPACE "VisibilityPropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FVisibilityPropertyTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer )
{
	return MakeShareable(new FVisibilityPropertyTrackEditor(OwningSequencer));
}

FText FVisibilityPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("VisibilityPropertyTrackEditor_DisplayName", "Visibility Property");
}

TSharedRef<ISequencerSection> FVisibilityPropertyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FBoolPropertySection>(SectionObject);
}


void FVisibilityPropertyTrackEditor::GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys )
{
	// Invert the property key since the underlying property is actually 'hidden'
	bool KeyedValue = !PropertyChangedParams.GetPropertyValue<bool>();
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneBoolChannel>(0, KeyedValue, true));
}

#undef LOCTEXT_NAMESPACE
