// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "KeyframeTrackEditor.h"
#include "PropertyTrackEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneVisibilityTrack.h"
#include "UObject/UnrealNames.h"

class FPropertyChangedParams;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FGuid;


/**
 * A property track for bool properties which have been set
 * to use a UMovieSceneVisibilityTrack through metadata.
 */
class FVisibilityPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneVisibilityTrack>
{
public:

	/** Constructor. */
	FVisibilityPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer)
	{ }

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

protected:

	//~ FPropertyTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	virtual bool ForCustomizedUseOnly() { return true; }
};
