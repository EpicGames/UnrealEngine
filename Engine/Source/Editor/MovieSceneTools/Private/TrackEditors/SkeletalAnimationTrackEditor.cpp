// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/SkeletalAnimationTrackEditor.h"

#include "AnimSequencerInstanceProxy.h"
#include "Animation/AnimSequenceBase.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "FSkeletalAnimationTrackEditor"


namespace UE::Sequencer::SkeletalAnimationEditorConstants
{
	static TAutoConsoleVariable<bool> CVarEvaluateSkeletalMeshOnPropertyChange(
		TEXT("Sequencer.EvaluateSkeletalMeshOnPropertyChange"),
		true,
		TEXT("Enable/disable sending a track value changed when properties change so that the skeletal mesh can be re-evaluated in Sequencer"));
} // namespace UE::Sequencer::SkeletalAnimationEditorConstants


FSkeletalAnimationTrackEditor::FSkeletalAnimationTrackEditor( TSharedRef<ISequencer> InSequencer )
	: UE::Sequencer::FCommonAnimationTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FSkeletalAnimationTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShared<FSkeletalAnimationTrackEditor>(InSequencer);
}

void FSkeletalAnimationTrackEditor::OnInitialize()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FSkeletalAnimationTrackEditor::OnPostPropertyChanged);
	FCommonAnimationTrackEditor::OnInitialize();

	// Register as animation mixer item menu provider
	IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);
}

void FSkeletalAnimationTrackEditor::OnRelease()
{
	// Unregister as animation mixer item menu provider
	IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCommonAnimationTrackEditor::OnRelease();
}

const UClass* FSkeletalAnimationTrackEditor::GetHandledMixerItemClass() const
{
	return UMovieSceneSkeletalAnimationSection::StaticClass();
}

void FSkeletalAnimationTrackEditor::PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex)
{
	if (ObjectBindings.Num() > 0 && Sequencer.IsValid())
	{
		USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], Sequencer);
		if (Skeleton)
		{
			AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track, RowIndex);
		}
	}
}

void FSkeletalAnimationTrackEditor::BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// Check if the Animation Mixer plugin is enabled, only do this if not, otherwise we get duplicates
	const bool bAnimationMixerEnabled = FModuleManager::Get().IsModuleLoaded(TEXT("MovieSceneAnimMixerEditor"));

	if (bAnimationMixerEnabled == false)
	{
		FCommonAnimationTrackEditor::BuildObjectBindingContextMenu(MenuBuilder, ObjectBindings, ObjectClass);
	}
}


void FSkeletalAnimationTrackEditor::BuildObjectBindingColumnWidgets(
	TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox,
	const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding,
	const UE::Sequencer::FCreateOutlinerViewParams& InParams,
	const FName& InColumnName)
{
	// Check if the Animation Mixer plugin is enabled, only do this if not, otherwise we get duplicates
	const bool bAnimationMixerEnabled = FModuleManager::Get().IsModuleLoaded(TEXT("MovieSceneAnimMixerEditor"));

	if (bAnimationMixerEnabled == false)
	{
		FCommonAnimationTrackEditor::BuildObjectBindingColumnWidgets(GetEditBox, ObjectBinding, InParams, InColumnName);
	}
}

void FSkeletalAnimationTrackEditor::PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu)
{
	if (!ObjectClass || (!ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) && !ObjectClass->IsChildOf(AActor::StaticClass()) && !ObjectClass->IsChildOf(UChildActorComponent::StaticClass())))
	{
		return;
	}

	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], GetSequencer());

	if (Skeleton)
	{
		// Load the asset registry module
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		// Collect a full list of assets with the specified class
		TArray<FAssetData> AssetDataList;
		AssetRegistryModule.Get().GetAssetsByClass(UAnimSequenceBase::StaticClass()->GetClassPathName(), AssetDataList, true);

		if (AssetDataList.Num())
		{
			UMovieSceneTrack* Track = nullptr;

			if (bIsInsideSubmenu)
			{
				// Inside the Animation submenu - add a section header and embed the asset picker directly
				MenuBuilder.BeginSection(NAME_None, LOCTEXT("AnimationSectionHeader", "Animation"));
				{
					AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track, INDEX_NONE);
				}
				MenuBuilder.EndSection();
			}
			else
			{
				// At the top-level menu - wrap in a submenu
				MenuBuilder.AddSubMenu(
					LOCTEXT("AddAnimationSubMenu", "Animation"),
					LOCTEXT("AddAnimationSubMenuTooltip", "Adds an animation track."),
					FNewMenuDelegate::CreateLambda([this, ObjectBindings, Skeleton, Track](FMenuBuilder& SubMenuBuilder)
					{
						AddAnimationSubMenu(SubMenuBuilder, ObjectBindings, Skeleton, Track, INDEX_NONE);
					})
				);
			}
		}
	}
}

bool FSkeletalAnimationTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSkeletalAnimationTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FSkeletalAnimationTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneSkeletalAnimationTrack::StaticClass();
}

void FSkeletalAnimationTrackEditor::OnPostPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (UE::Sequencer::SkeletalAnimationEditorConstants::CVarEvaluateSkeletalMeshOnPropertyChange->GetBool())
	{
		FProperty* MemberPropertyThatChanged = InPropertyChangedEvent.MemberProperty;
		const FName MemberPropertyName = MemberPropertyThatChanged != NULL ? MemberPropertyThatChanged->GetFName() : NAME_None;
		if (MemberPropertyName == USceneComponent::GetRelativeLocationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeRotationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeScale3DPropertyName())
		{
			return;
		}

		const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		if (!SequencerPtr)
		{
			return;
		}

		// If the object changed has any animation track, notify sequencer to update because animations tick on their own and sequencer needs to evaluate again
		const bool bCreateIfMissing = false;
		FFindOrCreateHandleResult ObjectHandleResult = FindOrCreateHandleToObject(InObject, bCreateIfMissing);
		FGuid ObjectHandle = ObjectHandleResult.Handle;
		
		bool bTrackFound = ObjectHandle.IsValid() ? FindOrCreateTrackForObject(ObjectHandle, UMovieSceneSkeletalAnimationTrack::StaticClass(), NAME_None, bCreateIfMissing).Track != nullptr : false;

		// A mesh component, or its owning actor can have the animation track, so both need to be searched for.
		if (!bTrackFound && !InObject->IsA<AActor>())
		{
			ObjectHandleResult = FindOrCreateHandleToObject(InObject->GetTypedOuter<AActor>(), bCreateIfMissing);
			ObjectHandle = ObjectHandleResult.Handle;

			bTrackFound = ObjectHandle.IsValid() ? FindOrCreateTrackForObject(ObjectHandle, UMovieSceneSkeletalAnimationTrack::StaticClass(), NAME_None, bCreateIfMissing).Track != nullptr : false;
		}

		if (bTrackFound)
		{
			SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

TSubclassOf<UMovieSceneCommonAnimationTrack> FSkeletalAnimationTrackEditor::GetTrackClass() const
{
	return UMovieSceneSkeletalAnimationTrack::StaticClass();
}

void FSkeletalAnimationTrackEditor::BuildTrackContextMenu_Internal(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track, const bool bAddSeparatorAtEnd)
{
	//there's a bug with a section being open already, so we end it.
	UMovieSceneSkeletalAnimationTrack* SkeletalAnimationTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track);

	/** Put this back when and if it works
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("MotionBlendingOptions", "Motion Blending Options"));
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "AutoMatchClipsRootMotions", "Auto Match Clips Root Motions"),
			NSLOCTEXT("Sequencer", "AutoMatchClipsRootMotionsTooltip", "Preceeding clips will auto match to the preceding clips root bones position. You can override this behavior per clip in it's section options."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=]()->void {
					SkeletalAnimationTrack->ToggleAutoMatchClipsRootMotions(); 
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);

					}),
				FCanExecuteAction::CreateLambda([=]()->bool { return SequencerPtr && SkeletalAnimationTrack != nullptr; }),
				FIsActionChecked::CreateLambda([=]()->bool { return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->bAutoMatchClipsRootMotions; })),
			NAME_None, 
			EUserInterfaceActionType::ToggleButton
		);
		MenuBuilder.EndSection();
	}
	*/

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("SkelAnimRootMOtion", "Root Motion"));
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "BlendFirstChildOfRoot", "Blend First Child Of Root"),
			NSLOCTEXT("Sequencer", "BlendFirstChildOfRootTooltip", "If True, do not blend and match the root bones but instead the first child bone of the root. Toggle this on when the matched sequences in the track have no motion on the root."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SkeletalAnimationTrack]() -> void
				{
					SkeletalAnimationTrack->bBlendFirstChildOfRoot = SkeletalAnimationTrack->bBlendFirstChildOfRoot ? false : true;
					SkeletalAnimationTrack->SetRootMotionsDirty();

					if (const TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
					{
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}
				}),
				FCanExecuteAction::CreateLambda([this, SkeletalAnimationTrack]() -> bool
				{
					return SkeletalAnimationTrack != nullptr;
				}),
				FIsActionChecked::CreateLambda([this, SkeletalAnimationTrack]() -> bool
				{
					return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->bBlendFirstChildOfRoot;
				})),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "ShowRootMotionTrails", "Show Root Motion Trail"),
			NSLOCTEXT("Sequencer", "ShowRootMotionTrailsTooltip", "Show the Root Motion Trail for all Animation Clips."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SkeletalAnimationTrack]() -> void
				{
					SkeletalAnimationTrack->ToggleShowRootMotionTrail();

					if (const TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
					{
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					}
				}),
				FCanExecuteAction::CreateLambda([this, SkeletalAnimationTrack]()->bool
				{
					if (const TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
					{
						return SkeletalAnimationTrack != nullptr;
					}
					return false;
				}),
				FIsActionChecked::CreateLambda([this, SkeletalAnimationTrack]()->bool
				{
					return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->bShowRootMotionTrail;
				})),
			NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "SwapRootBoneNone", "Swap Root Bone None"),
			NSLOCTEXT("Sequencer", "SwapRootBoneNoneTooltip", "Do not swap root bone for all sections."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SkeletalAnimationTrack]()->void {
					SkeletalAnimationTrack->SetSwapRootBone(ESwapRootBone::SwapRootBone_None);
					}),
				FCanExecuteAction::CreateLambda([SkeletalAnimationTrack]()->bool { return  SkeletalAnimationTrack != nullptr; }),
				FIsActionChecked::CreateLambda([SkeletalAnimationTrack]()->bool { return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->SwapRootBone == ESwapRootBone::SwapRootBone_None; })),
				NAME_None,
				EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "SwapRootBoneActor", "Swap Root Bone Actor"),
			NSLOCTEXT("Sequencer", "SwapRootBoneActorTooltip", "Swap root bone on root actor component for all sections."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SkeletalAnimationTrack]()->void {
					SkeletalAnimationTrack->SetSwapRootBone(ESwapRootBone::SwapRootBone_Actor);
					}),
				FCanExecuteAction::CreateLambda([SkeletalAnimationTrack]()->bool { return  SkeletalAnimationTrack != nullptr; }),
				FIsActionChecked::CreateLambda([SkeletalAnimationTrack]()->bool { return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->SwapRootBone == ESwapRootBone::SwapRootBone_Actor; })),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "SwapRootBoneComponent", "Swap Root Bone Component"),
			NSLOCTEXT("Sequencer", "SwapRootBoneComponentTooltip", "Swap root bone on current component for all sections."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([SkeletalAnimationTrack]()->void {
					SkeletalAnimationTrack->SetSwapRootBone(ESwapRootBone::SwapRootBone_Component);
					}),
				FCanExecuteAction::CreateLambda([SkeletalAnimationTrack]()->bool { return  SkeletalAnimationTrack != nullptr; }),
				FIsActionChecked::CreateLambda([SkeletalAnimationTrack]()->bool { return SkeletalAnimationTrack != nullptr && SkeletalAnimationTrack->SwapRootBone == ESwapRootBone::SwapRootBone_Component; })),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
	}
	MenuBuilder.EndSection();

	if (bAddSeparatorAtEnd)
	{
		MenuBuilder.AddSeparator();
	}
}

TSharedRef<ISequencerSection> FSkeletalAnimationTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	return MakeShared<FSkeletalAnimationSection>(SectionObject, GetSequencer());
}

FSkeletalAnimationSection::FSkeletalAnimationSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: UE::Sequencer::FCommonAnimationSection( InSection, InSequencer)
{
}

void FSkeletalAnimationTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	BuildTrackContextMenu_Internal(MenuBuilder, Track, true);
}

void FSkeletalAnimationTrackEditor::BuildTrackSidebarMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	BuildTrackContextMenu_Internal(MenuBuilder, Track, false);
}


#undef LOCTEXT_NAMESPACE
