// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"

/**
 * Tools for animation tracks
 */
class FSkeletalAnimationTrackEditor
	: public UE::Sequencer::FCommonAnimationTrackEditor
	, public IMovieSceneAnimMixerItemMenuProvider
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FSkeletalAnimationTrackEditor( TSharedRef<ISequencer> InSequencer );

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

	virtual void OnInitialize() override;
	virtual void OnRelease() override;

	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual void BuildTrackSidebarMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual TSubclassOf<UMovieSceneCommonAnimationTrack> GetTrackClass() const override;

	// IMovieSceneAnimMixerItemMenuProvider interface
	virtual const UClass* GetHandledMixerItemClass() const override;
	virtual void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) override;
	virtual void PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu) override;
	// Appear last in the Animation submenu so the large embedded asset picker sits at the bottom,
	// keeping the discrete menu entries (Animation Mixer, Control Rig, Stitch Animation, etc.) grouped at the top.
	virtual int32 GetMixerItemMenuPriority() const override { return 250; }
	virtual int32 GetObjectBindingAnimationMenuPriority() const override { return 250; }
	virtual void BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildObjectBindingColumnWidgets(
		TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox,
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding,
		const UE::Sequencer::FCreateOutlinerViewParams& InParams,
		const FName& InColumnName) override;
private:

	void OnPostPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);

	/** Common function used to build context and sidebar menus */
	void BuildTrackContextMenu_Internal(FMenuBuilder& MenuBuilder, UMovieSceneTrack* const InTrack, const bool bAddSeparatorAtEnd);
};


/** Class for animation sections */
class FSkeletalAnimationSection
	: public UE::Sequencer::FCommonAnimationSection
{
public:

	/** Constructor. */
	FSkeletalAnimationSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);
};
