// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/SlomoTrackEditor.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "MovieSceneTrackEditor.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Tracks/MovieSceneSlomoTrack.h"
#include "UObject/WeakObjectPtr.h"

class ISequencerTrackEditor;
class UMovieSceneSection;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "FSlomoTrackEditor"


/* FSlomoTrackEditor static functions
 *****************************************************************************/

TSharedRef<ISequencerTrackEditor> FSlomoTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FSlomoTrackEditor(InSequencer));
}


/* FSlomoTrackEditor structors
 *****************************************************************************/

FSlomoTrackEditor::FSlomoTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor(InSequencer)
{ }


/* ISequencerTrackEditor interface
 *****************************************************************************/

FText FSlomoTrackEditor::GetDisplayName() const
{
	return LOCTEXT("TimeDilationTrackEditor_DisplayName", "Time Dilation");
}

void FSlomoTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTimeDilationTrack", "Time Dilation Track"),
		LOCTEXT("AddTimeDilationTrackTooltip", "Adds a new track that controls the world's time dilation."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Slomo"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSlomoTrackEditor::HandleAddSlomoTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FSlomoTrackEditor::HandleAddSlomoTrackMenuEntryCanExecute)
		)
	);
}

bool FSlomoTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSlomoTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FSlomoTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneSlomoTrack::StaticClass());
}

const FSlateBrush* FSlomoTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Slomo");
}


/* FSlomoTrackEditor callbacks
 *****************************************************************************/

void FSlomoTrackEditor::HandleAddSlomoTrackMenuEntryExecute()
{
	UMovieScene* MovieScene = GetFocusedMovieScene();

	if (MovieScene == nullptr)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	UMovieSceneTrack* SlomoTrack = MovieScene->FindTrack<UMovieSceneSlomoTrack>();

	if (SlomoTrack != nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddSlomoTrack_Transaction", "Add Time Dilation Track"));

	MovieScene->Modify();

	SlomoTrack = FindOrCreateRootTrack<UMovieSceneSlomoTrack>().Track;
	check(SlomoTrack);

	UMovieSceneSection* NewSection = SlomoTrack->CreateNewSection();
	check(NewSection);

	SlomoTrack->Modify();
	SlomoTrack->AddSection(*NewSection);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(SlomoTrack, FGuid());
	}
}

bool FSlomoTrackEditor::HandleAddSlomoTrackMenuEntryCanExecute() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	return ((FocusedMovieScene != nullptr) && (FocusedMovieScene->FindTrack<UMovieSceneSlomoTrack>() == nullptr));
}

#undef LOCTEXT_NAMESPACE
