// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "KeyframeTrackEditor.h"
#include "Tracks/MovieSceneSlomoTrack.h"

class FMenuBuilder;
class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSequence;
class UMovieSceneTrack;
struct FSlateBrush;

/**
* A property track editor for slow motion control.
*/
class FSlomoTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneSlomoTrack>
{
public:

	/**
	 * Factory function to create an instance of this class (called by a sequencer).
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FSlomoTrackEditor(TSharedRef<ISequencer> InSequencer);	

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual const FSlateBrush* GetIconBrush() const override;

private:

	/** Callback for executing the "Add Slomo Track" menu entry. */
	void HandleAddSlomoTrackMenuEntryExecute();
	bool HandleAddSlomoTrackMenuEntryCanExecute() const;
};
