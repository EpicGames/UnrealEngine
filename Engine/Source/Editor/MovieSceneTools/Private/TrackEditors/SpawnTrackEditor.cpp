// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/SpawnTrackEditor.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Sections/BoolPropertySection.h"

class ISequencerTrackEditor;


#define LOCTEXT_NAMESPACE "FSpawnTrackEditor"


TSharedRef<ISequencerTrackEditor> FSpawnTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FSpawnTrackEditor(InSequencer));
}


FSpawnTrackEditor::FSpawnTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor(InSequencer)
{ }

TSharedRef<ISequencerSection> FSpawnTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FBoolPropertySection>(SectionObject);
}


UMovieSceneTrack* FSpawnTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	UMovieSceneTrack* NewTrack = FKeyframeTrackEditor::AddTrack(FocusedMovieScene, ObjectHandle, TrackClass, UniqueTypeName);

	if (UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(NewTrack))
	{
		SpawnTrack->Modify();
		SpawnTrack->SetObjectId(ObjectHandle);
		SpawnTrack->AddSection(*SpawnTrack->CreateNewSection());
	}

	return NewTrack;
}


void FSpawnTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	UMovieSceneSequence* MovieSequence = GetSequencer()->GetFocusedMovieSceneSequence();

	if (!MovieSequence || MovieSequence->GetClass()->GetName() != TEXT("LevelSequence"))
	{
		return;
	}
	else
	{
		bool bSpawnable = false;
		if (MovieSequence->GetMovieScene()->FindSpawnable(ObjectBindings[0]))
		{
			bSpawnable = true;
		}
		else if (const FMovieSceneBindingReferences* BindingReferences = MovieSequence->GetBindingReferences())
		{
			if (Algo::AnyOf(BindingReferences->GetReferences(ObjectBindings[0]), [](const FMovieSceneBindingReference& BindingReference) { return BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneSpawnableBindingBase>(); }))
			{
				bSpawnable = true;
			}
		}

		if (!bSpawnable)
		{
			return;
		}
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddSpawnTrack", "Spawn Track"),
		LOCTEXT("AddSpawnTrackTooltip", "Adds a new track that controls the lifetime of the track's spawnable object."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSpawnTrackEditor::HandleAddSpawnTrackMenuEntryExecute, ObjectBindings),
			FCanExecuteAction::CreateSP(this, &FSpawnTrackEditor::CanAddSpawnTrack, ObjectBindings[0])
		)
	);
}


bool FSpawnTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneSpawnTrack::StaticClass());
}


bool FSpawnTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSpawnTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}


void FSpawnTrackEditor::HandleAddSpawnTrackMenuEntryExecute(TArray<FGuid> ObjectBindings)
{
	FScopedTransaction AddSpawnTrackTransaction(LOCTEXT("AddSpawnTrack_Transaction", "Add Spawn Track"));

	for (FGuid ObjectBinding : ObjectBindings)
	{
		AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectBinding, UMovieSceneSpawnTrack::StaticClass(), NAME_None);
	}
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}


bool FSpawnTrackEditor::CanAddSpawnTrack(FGuid ObjectBinding) const
{
	return !GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieSceneSpawnTrack>(ObjectBinding);
}


#undef LOCTEXT_NAMESPACE
