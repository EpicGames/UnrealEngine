// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/SubTrackEditor.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Styling/AppStyle.h"
#include "GameFramework/PlayerController.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ContentBrowserModule.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "SequencerSectionPainter.h"
#include "SequencerSettings.h"
#include "TrackEditors/SubTrackEditorBase.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "MovieSceneMetaData.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneToolsProjectSettings.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneTimeHelpers.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Algo/Accumulate.h"
#include "SequencerUtilities.h"
#include "AssetToolsModule.h"
#include "EditorModeManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "IDetailsView.h"
#include "IKeyArea.h"
#include "IStructureDetailsView.h"
#include "LevelEditorViewport.h"
#include "PropertyEditorModule.h"
#include "Selection.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EditModes/SubTrackEditorMode.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Systems/MovieSceneTransformOriginSystem.h"

#define LOCTEXT_NAMESPACE "FSubTrackEditor"


/**
 * A generic implementation for displaying simple property sections.
 */
class FSubSection
	: public TSubSectionMixin<>
{
public:

	FSubSection(TSharedPtr<ISequencer> InSequencer, UMovieSceneSection& InSection, TSharedPtr<FSubTrackEditor> InSubTrackEditor)
		: TSubSectionMixin(InSequencer, *CastChecked<UMovieSceneSubSection>(&InSection))
		, SubTrackEditor(InSubTrackEditor)
	{
	}

public:

	// ISequencerSection interface

	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override
	{
		ISequencerSection::BuildSectionContextMenu(MenuBuilder, ObjectBinding);

		UMovieSceneSubSection* Section = &GetSubSectionObject();
		
		FString DisplayName = SubTrackEditor.Pin()->GetSubSectionDisplayName(Section);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("SequenceMenuText", "Sequence"));
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("TakesMenu", "Takes"),
				LOCTEXT("TakesMenuTooltip", "Subsequence takes"),
				FNewMenuDelegate::CreateLambda([this, Section](FMenuBuilder& InMenuBuilder) { SubTrackEditor.Pin()->AddTakesMenu(Section, InMenuBuilder); }));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("NewTake", "New Take"),
				FText::Format(LOCTEXT("NewTakeTooltip", "Create a new take for {0}"), FText::FromString(DisplayName)),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(SubTrackEditor.Pin().ToSharedRef(), &FSubTrackEditor::CreateNewTake, Section))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("InsertNewSequence", "Insert Sequence"),
				LOCTEXT("InsertNewSequenceTooltip", "Insert a new sequence at the current time"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(SubTrackEditor.Pin().ToSharedRef(), &FSubTrackEditor::InsertSection, Cast<UMovieSceneTrack>(Section->GetOuter())))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DuplicateSequence", "Duplicate Sequence"),
				FText::Format(LOCTEXT("DuplicateSequenceTooltip", "Duplicate {0} to create a new sequence"), FText::FromString(DisplayName)),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(SubTrackEditor.Pin().ToSharedRef(), &FSubTrackEditor::DuplicateSection, Section))
			);
		
			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditMetaData", "Edit Meta Data"),
				LOCTEXT("EditMetaDataTooltip", "Edit meta data"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(SubTrackEditor.Pin().ToSharedRef(), &FSubTrackEditor::EditMetaData, Section))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("PlayableDirectly_Label", "Playable Directly"),
				LOCTEXT("PlayableDirectly_Tip", "When enabled, this sequence will also support being played directly outside of the root sequence. Disable this to save some memory on complex hierarchies of sequences."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FSubSection::TogglePlayableDirectly),
					FCanExecuteAction::CreateLambda([]{ return true; }),
					FGetActionCheckState::CreateRaw(this, &FSubSection::IsPlayableDirectly)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
		MenuBuilder.EndSection();

		auto MakeUIAction = [Section](EMovieSceneTransformChannel ChannelsToToggle, const TSharedPtr<ISequencer>& Sequencer)
		{
			return FUIAction(
				FExecuteAction::CreateLambda([Section, ChannelsToToggle, Sequencer]
					{
						FScopedTransaction Transaction(LOCTEXT("SetActiveChannelsTransaction", "Set Active Channels"));
						Section->Modify();
						EMovieSceneTransformChannel Channels = Section->GetMask().GetChannels();

						if (EnumHasAllFlags(Channels, ChannelsToToggle) || (Channels & ChannelsToToggle) == EMovieSceneTransformChannel::None)
						{
							Section->SetMask(Section->GetMask().GetChannels() ^ ChannelsToToggle);
						}
						else
						{
							Section->SetMask(Section->GetMask().GetChannels() | ChannelsToToggle);
						}
					
						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
					}
				),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([Section, ChannelsToToggle]
				{
					EMovieSceneTransformChannel Channels = Section->GetMask().GetChannels();
					if (EnumHasAllFlags(Channels, ChannelsToToggle))
					{
						return ECheckBoxState::Checked;
					}
					if (EnumHasAnyFlags(Channels, ChannelsToToggle))
					{
						return ECheckBoxState::Undetermined;
					}
					return ECheckBoxState::Unchecked;
				})
			);
		};

		TSharedPtr<ISequencer> Sequencer = GetSequencer();
		
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("OriginChannelsText", "Active Channels"));
		MenuBuilder.AddSubMenu(
			LOCTEXT("AllTranslation", "Translation"), LOCTEXT("AllTranslation_ToolTip", "Causes this section to affect the translation of the transform"),
			FNewMenuDelegate::CreateLambda([Sequencer, MakeUIAction](FMenuBuilder& SubMenuBuilder){

				const EAxisList::Type XAxis = EAxisList::Forward;
				const EAxisList::Type YAxis = EAxisList::Left;
				const EAxisList::Type ZAxis = EAxisList::Up;

				const int32 NumMenuItems = 3;
				TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
					[&SubMenuBuilder, MakeUIAction, Sequencer, XAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(XAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationX, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);

					},
					[&SubMenuBuilder, MakeUIAction, Sequencer, YAxis]()
					{				
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(YAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationY, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
					},
					[&SubMenuBuilder, MakeUIAction, Sequencer, ZAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(ZAxis),
						FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationZ, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				};

				const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
				for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
				{
					const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
					MenuConstructors[SwizzledComponentIndex]();
				}
			}),
			MakeUIAction(EMovieSceneTransformChannel::Translation, Sequencer),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllRotation", "Rotation"), LOCTEXT("AllRotation_ToolTip", "Causes this section to affect the rotation of the transform"),
			FNewMenuDelegate::CreateLambda([Sequencer, MakeUIAction](FMenuBuilder& SubMenuBuilder){
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationX", "Roll"), LOCTEXT("RotationX_ToolTip", "Causes this section to affect the roll channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationX, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationY", "Pitch"), LOCTEXT("RotationY_ToolTip", "Causes this section to affect the pitch channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationY, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationZ", "Yaw"), LOCTEXT("RotationZ_ToolTip", "Causes this section to affect the yaw channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationZ, Sequencer), NAME_None, EUserInterfaceActionType::ToggleButton);
			}),
			MakeUIAction(EMovieSceneTransformChannel::Rotation, Sequencer),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
		MenuBuilder.EndSection();
	}

	void TogglePlayableDirectly()
	{
		TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (Sequencer)
		{
			FScopedTransaction Transaction(LOCTEXT("SetPlayableDirectly_Transaction", "Set Playable Directly"));

			TArray<UMovieSceneSection*> SelectedSections;
			Sequencer->GetSelectedSections(SelectedSections);

			const bool bNewPlayableDirectly = IsPlayableDirectly() != ECheckBoxState::Checked;

			for (UMovieSceneSection* Section : SelectedSections)
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					UMovieSceneSequence* Sequence = SubSection->GetSequence();
					if (Sequence->IsPlayableDirectly() != bNewPlayableDirectly)
					{
						Sequence->SetPlayableDirectly(bNewPlayableDirectly);
					}
				}
			}
		}
	}

	ECheckBoxState IsPlayableDirectly() const
	{
		ECheckBoxState CheckboxState = ECheckBoxState::Undetermined;

		TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (Sequencer)
		{
			TArray<UMovieSceneSection*> SelectedSections;
			Sequencer->GetSelectedSections(SelectedSections);

			for (UMovieSceneSection* Section : SelectedSections)
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					UMovieSceneSequence* Sequence = SubSection->GetSequence();
					if (Sequence)
					{
						if (CheckboxState == ECheckBoxState::Undetermined)
						{
							CheckboxState = Sequence->IsPlayableDirectly() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
						else if (CheckboxState == ECheckBoxState::Checked != Sequence->IsPlayableDirectly())
						{
							return ECheckBoxState::Undetermined;
						}
					}
				}
			}
		}

		return CheckboxState;
	}

	virtual bool IsReadOnly() const override
	{
		// Overridden to false regardless of movie scene section read only state so that we can double click into the sub section
		return false;
	}

private:

	/** The sub track editor that contains this section */
	TWeakPtr<FSubTrackEditor> SubTrackEditor;
};


/* FSubTrackEditor structors
 *****************************************************************************/

FSubTrackEditor::FSubTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor<UMovieSceneSubTrack>(InSequencer)
{ }


/* ISequencerTrackEditor interface
 *****************************************************************************/

void FSubTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		GetSubTrackName(),
		GetSubTrackToolTip(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), GetSubTrackBrushName()),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSubTrackEditor::HandleAddSubTrackMenuEntryExecute),
			FCanExecuteAction::CreateRaw(this, &FSubTrackEditor::HandleAddSubTrackMenuEntryCanExecute)
		)
	);
}

TSharedPtr<SWidget> FSubTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return UE::Sequencer::MakeAddButton(GetSubTrackName(), FOnGetContent::CreateSP(this, &FSubTrackEditor::HandleAddSubSequenceComboButtonGetMenuContent, Params.ViewModel.AsWeak()), Params.ViewModel);
}


void FSubTrackEditor::GetOriginKeys(const FVector& CurrentPosition, const FRotator& CurrentRotation, const EMovieSceneTransformChannel ChannelsToKey, UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys)
{
	FMovieSceneChannelProxy& SectionChannelProxy = Section->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> ChannelHandles[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Override.Rotation.Z")
	};
	
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[0].GetChannelIndex(), CurrentPosition.X,     EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationX)));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[1].GetChannelIndex(), CurrentPosition.Y,     EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationY)));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[2].GetChannelIndex(), CurrentPosition.Z,     EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationZ)));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[3].GetChannelIndex(), CurrentRotation.Roll,  EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationX)));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[4].GetChannelIndex(), CurrentRotation.Pitch, EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationY)));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[5].GetChannelIndex(), CurrentRotation.Yaw,   EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationZ)));
}

TSharedRef<ISequencerTrackEditor> FSubTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FSubTrackEditor(InSequencer));
}

int32 FSubTrackEditor::GetPreviousKey(FMovieSceneDoubleChannel& Channel, FFrameNumber Time)
{
	TArray<FFrameNumber> KeyTimes;
	TArray<FKeyHandle> KeyHandles;

	TRange<FFrameNumber> Range;
	Range.SetLowerBound(TRangeBound<FFrameNumber>::Open());
	Range.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(Time));
	Channel.GetData().GetKeys(Range, &KeyTimes, &KeyHandles);

	if (KeyHandles.Num() <= 0)
	{
		return INDEX_NONE;
	}

	int32 Index = Channel.GetData().GetIndex(KeyHandles[KeyHandles.Num() - 1]);
	return Index;
}

double FSubTrackEditor::UnwindChannel(const double& OldValue, double NewValue)
{
	while( NewValue - OldValue > 180.0f )
	{
		NewValue -= 360.0f;
	}
	while( NewValue - OldValue < -180.0f )
	{
		NewValue += 360.0f;
	}
	return NewValue;
}

FText FSubTrackEditor::GetDisplayName() const
{
	return LOCTEXT("SubsequenceTrackEditor_DisplayName", "Subsequence");
}

void FSubTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::Sequencer;

	auto Iterator = [this, InKeyTime, &InSequencer, OutResults](UMovieSceneTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> Operations)
	{
		this->ProcessKeyOperationInternal(Operations, InSequencer, InKeyTime, OutResults);
	};

	Operation.IterateOperations(Iterator);
}

void FSubTrackEditor::ProcessKeyOperationInternal(TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::Sequencer;

	for (int32 Index = 0; Index < SectionsToKey.Num(); ++Index)
	{
		for (TSharedPtr<IKeyArea> KeyArea : SectionsToKey[Index].KeyAreas)
		{
			UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(SectionsToKey[Index].Section->GetSectionObject());
			FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
			if(Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName() && SubSection)
			{
				FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(Handle.Get());

				if (ensureAlwaysMsgf(Channel, TEXT("Channel: %s for Key Area %s does not exist. Keying may not function properly"), *Handle.GetChannelTypeName().ToString(), *KeyArea->GetName().ToString()))
				{
					double Value = 0.0;

					FTransform RawTransformOrigin = GetTransformOriginDataForSubSection(SubSection);
					TOptional<FVector> KeyPosition = SubSection->GetKeyPreviewPosition();
					TOptional<FRotator> KeyRotation = SubSection->GetKeyPreviewRotation();
;
					switch (Handle.GetChannelIndex())
					{
					case 0:
						Value = KeyPosition.IsSet() ? KeyPosition.GetValue().X : RawTransformOrigin.GetLocation().X;
						break;
					case 1:
						Value = KeyPosition.IsSet() ? KeyPosition.GetValue().Y : RawTransformOrigin.GetLocation().Y;
						break;
					case 2:
						Value = KeyPosition.IsSet() ? KeyPosition.GetValue().Z : RawTransformOrigin.GetLocation().Z;
						break;
					case 3:
						Value = KeyRotation.IsSet() ? KeyRotation.GetValue().Roll : RawTransformOrigin.Rotator().Roll;
						break;
					case 4:
						Value = KeyRotation.IsSet() ? KeyRotation.GetValue().Pitch : RawTransformOrigin.Rotator().Pitch;
						break;
					case 5:
						Value = KeyRotation.IsSet() ? KeyRotation.GetValue().Yaw : RawTransformOrigin.Rotator().Yaw;
						break;
					default:
						Value = 0.0;
					}

					if (KeyArea->GetName() == "Rotation.X" ||
						KeyArea->GetName() == "Rotation.Y" ||
						KeyArea->GetName() == "Rotation.Z")
					{
						int32 PreviousKey = GetPreviousKey(*Channel, KeyTime);
						if (PreviousKey != INDEX_NONE && PreviousKey < Channel->GetData().GetValues().Num())
						{
							double OldValue = Channel->GetData().GetValues()[PreviousKey].Value;
							Value = UnwindChannel(OldValue, Value);
						}
					}

					EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(Channel, KeyTime, InSequencer.GetKeyInterpolation());
					FKeyHandle KeyHandle = AddKeyToChannel(Channel, KeyTime, Value, Interpolation);

					if (OutResults)
					{
						FAddKeyResult Result;
						Result.KeyArea = KeyArea;
						Result.KeyHandle = KeyHandle;
						OutResults->Add(Result);
					}
				}
			}
			else
			{
				FKeyHandle KeyHandle = KeyArea->AddOrUpdateKey(KeyTime, FGuid(), InSequencer);
				
				if (OutResults)
				{
					FAddKeyResult Result;
					Result.KeyArea = KeyArea;
					Result.KeyHandle = KeyHandle;
					OutResults->Add(Result);
				}
			}
		}
	}
}


TSharedRef<ISequencerSection> FSubTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShareable(new FSubSection(GetSequencer(), SectionObject, SharedThis(this)));
}


bool FSubTrackEditor::CanHandleAssetAdded(UMovieSceneSequence* Sequence) const
{
	// Only allow sequences without a camera cut track to be dropped as a subsequence. Otherwise, it'll be dropped as a shot.
	return Sequence->GetMovieScene()->GetCameraCutTrack() == nullptr;
}

bool FSubTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(Asset);

	if (Sequence == nullptr)
	{
		return false;
	}

	if (!SupportsSequence(Sequence))
	{
		return false;
	}

	if (!CanHandleAssetAdded(Sequence))
	{
		return false;
	}

	if (Sequence->GetMovieScene()->GetPlaybackRange().IsEmpty())
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequenceDuration", "Invalid level sequence {0}. The sequence has no duration."), Sequence->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return false;
	}

	if (CanAddSubSequence(*Sequence))
	{
		const FScopedTransaction Transaction(FText::Join(FText::FromString(" "), LOCTEXT("AddText", "Add"), GetSubTrackName()));

		int32 RowIndex = INDEX_NONE;
		UMovieSceneTrack* Track = nullptr;
		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubTrackEditor::HandleSequenceAdded, Sequence, Track, RowIndex));

		return true;
	}

	FNotificationInfo Info(FText::Format( LOCTEXT("InvalidSequence", "Invalid level sequence {0}. There could be a circular dependency or the sequences are not compatible."), Sequence->GetDisplayName()));	
	Info.bUseLargeFont = false;
	FSlateNotificationManager::Get().AddNotification(Info);

	return false;
}

bool FSubTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneSubTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

bool FSubTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == GetSubTrackClass();
}

const FSlateBrush* FSubTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush(GetSubTrackBrushName());
}

bool FSubTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid())
	{
		return false;
	}

	if (!DragDropParams.Track.Get()->IsA(GetSubTrackClass()))
	{
		return false;
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return false;
	}
	
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return false;
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return false;
	}

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

	TOptional<FFrameNumber> LongestLengthInFrames;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(AssetData.GetAsset());
		if (Sequence && CanAddSubSequence(*Sequence))
		{
			FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();

			const FQualifiedFrameTime InnerDuration = FQualifiedFrameTime(
				UE::MovieScene::DiscreteSize(Sequence->GetMovieScene()->GetPlaybackRange()),
				Sequence->GetMovieScene()->GetTickResolution());

			FFrameNumber LengthInFrames = InnerDuration.ConvertTo(TickResolution).FrameNumber;
			
			// Keep track of the longest sub-sequence asset we're trying to drop onto it for preview display purposes.
			LongestLengthInFrames = FMath::Max(LongestLengthInFrames.Get(FFrameNumber(0)), LengthInFrames);
		}
	}

	if (LongestLengthInFrames.IsSet())
	{
		DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, DragDropParams.FrameNumber + LongestLengthInFrames.GetValue());
		return true;
	}

	return false;
}

FReply FSubTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	if (!DragDropParams.Track.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!DragDropParams.Track.Get()->IsA(GetSubTrackClass()))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>() )
	{
		return FReply::Unhandled();
	}
	
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return FReply::Unhandled();
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return FReply::Unhandled();
	}

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );
	
	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(AssetData.GetAsset());
		if (CanAddSubSequence(*Sequence))
		{
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubTrackEditor::HandleSequenceAdded, Sequence, DragDropParams.Track.Get(), DragDropParams.RowIndex));

			bAnyDropped = true;
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}

bool FSubTrackEditor::IsResizable(UMovieSceneTrack* InTrack) const
{
	return true;
}

void FSubTrackEditor::OnInitialize()
{
	GLevelEditorModeTools().ActivateDefaultMode();

	GLevelEditorModeTools().ActivateMode(FSubTrackEditorMode::ModeName);
	FSubTrackEditorMode* EditorMode = static_cast<FSubTrackEditorMode*>(GLevelEditorModeTools().GetActiveMode(FSubTrackEditorMode::ModeName));
	if(EditorMode)
	{
		EditorMode->SetSequencer(GetSequencer());
		EditorMode->GetOnOriginValueChanged().RemoveAll(this);
		EditorMode->GetOnOriginValueChanged().AddSP(this, &FSubTrackEditor::UpdateOrigin);
	}
	GetSequencer()->GetViewModel()->GetSelection()->TrackArea.OnChanged.AddSP(this, &FSubTrackEditor::UpdateActiveMode);

	SectionsWithPreviews.Empty();
	GetSequencer()->OnPlayEvent().AddSP(this, &FSubTrackEditor::ResetSectionPreviews);
	GetSequencer()->OnBeginScrubbingEvent().AddSP(this, &FSubTrackEditor::ResetSectionPreviews);
	GetSequencer()->OnActivateSequence().AddSP(this, &FSubTrackEditor::ResetSectionPreviews);
	GetSequencer()->OnChannelChangedWithTime().AddSP(this, &FSubTrackEditor::ResetSectionPreviews);
}

void FSubTrackEditor::OnRelease()
{
	if(GLevelEditorModeTools().IsModeActive(FSubTrackEditorMode::ModeName))
	{
		GLevelEditorModeTools().DeactivateMode(FSubTrackEditorMode::ModeName);
	}

	if (!GetSequencer())
	{
		return;
	}
	
	GetSequencer()->GetViewModel()->GetSelection()->TrackArea.OnChanged.RemoveAll(this);

	SectionsWithPreviews.Empty();
	GetSequencer()->OnPlayEvent().RemoveAll(this);
	GetSequencer()->OnBeginScrubbingEvent().RemoveAll(this);
	GetSequencer()->OnActivateSequence().RemoveAll(this);
	GetSequencer()->OnChannelChangedWithTime().RemoveAll(this);
}

void FSubTrackEditor::Resize(float NewSize, UMovieSceneTrack* InTrack)
{
	UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(InTrack);
	if (SubTrack)
	{
		SubTrack->Modify();

		const int32 MaxNumRows = SubTrack->GetMaxRowIndex() + 1;
		SubTrack->SetRowHeight(FMath::RoundToInt(NewSize) / MaxNumRows);
		SubTrack->SetRowHeight(NewSize);
	}
}

bool FSubTrackEditor::GetDefaultExpansionState(UMovieSceneTrack* InTrack) const
{
	return true;
}


bool FSubTrackEditor::CanAddTransformKeysForSelectedObjects() const
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s). 
	// If the viewport is moving, disregard setting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return false;
		}
	}

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		if(SelectedActors->Num() > 0)
		{
			return false;
		}
	}

	const TSharedPtr<ISequencer> PinnedSequencer = GetSequencer();

	if (!PinnedSequencer.IsValid())
	{
		return false;
	}

	TArray<UMovieSceneSection*> OutSections;
	PinnedSequencer->GetSelectedSections(OutSections);

	for (UMovieSceneSection* Section : OutSections)
	{
		if (const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			UMovieSceneTrack* OuterTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter());
			if (OuterTrack && OuterTrack->GetSectionToKey())
			{
				if (OuterTrack->GetSectionToKey() == SubSection && SubSection->IsTransformOriginEditable())
				{
					return true;
				}
			}
			else if (SubSection->IsTransformOriginEditable())
			{
				return true;
			}
		}
	}

	TArray<UMovieSceneTrack*> OutTracks;
	PinnedSequencer->GetSelectedTracks(OutTracks);

	for (UMovieSceneTrack* Track : OutTracks)
	{
		if (const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			if (const UMovieSceneSection* SectionToKey = SubTrack->GetSectionToKey())
			{
				const UMovieSceneSubSection* SubSectionToKey = Cast<UMovieSceneSubSection>(SectionToKey);
				if (SubSectionToKey && SubSectionToKey->IsTransformOriginEditable())
				{
					return true;
				}
			}
			else if (SubTrack->GetAllSections().Num())
			{
				for (UMovieSceneSection* Section : SubTrack->FindAllSections(PinnedSequencer.Get()->GetLocalTime().Time.FrameNumber))
				{
					UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
					if (SubSection && SubSection->IsTransformOriginEditable())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FSubTrackEditor::GetSectionsToKey(TArray<UMovieSceneSubSection*>& OutSectionsToKey) const
{
	const TSharedPtr<ISequencer> PinnedSequencer = GetSequencer();

	if (!PinnedSequencer.IsValid())
	{
		return;
	}

	TArray<UMovieSceneSection*> OutSections;
	PinnedSequencer->GetSelectedSections(OutSections);

	for (UMovieSceneSection* Section : OutSections)
	{
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			const UMovieSceneTrack* OwningTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter());

			if(OwningTrack && OwningTrack->GetSectionToKey())
			{
				if (SubSection == OwningTrack->GetSectionToKey() && SubSection->IsTransformOriginEditable())
				{
					OutSectionsToKey.Add(SubSection);
				}
			}
			else if (SubSection->IsTransformOriginEditable())
			{
				OutSectionsToKey.Add(SubSection);
			}
		}
	}
	
	TArray<UMovieSceneTrack*> OutTracks;
	PinnedSequencer->GetSelectedTracks(OutTracks);

	for (const UMovieSceneTrack* Track : OutTracks)
	{
		if (const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			UMovieSceneSubSection* SubSectionToKey = Cast<UMovieSceneSubSection>(SubTrack->GetSectionToKey());
			if (SubSectionToKey && SubSectionToKey->IsTransformOriginEditable())
			{
				OutSectionsToKey.AddUnique(SubSectionToKey);
			}
			else if (SubTrack->GetAllSections().Num())
			{
				for (UMovieSceneSection* Section : SubTrack->FindAllSections(PinnedSequencer.Get()->GetLocalTime().Time.FrameNumber))
				{
					UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
					if (SubSection && SubSection->IsTransformOriginEditable())
					{
						OutSectionsToKey.AddUnique(SubSection);
					}
				}
			}
		}
	}
}

void FSubTrackEditor::OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel)
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s). 
	// If the viewport is moving, disregard setting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return;
		}
	}

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		if(SelectedActors->Num() > 0)
		{
			return;
		}
	}
	
	const TSharedPtr<ISequencer> PinnedSequencer = GetSequencer();

	// Create a transaction record because we are about to add keys
	const bool bShouldActuallyTransact = !GIsTransacting;		// Don't transact if we're recording in a PIE world.  That type of keyframe capture cannot be undone.
	FScopedTransaction AutoKeyTransaction( NSLOCTEXT("AnimatablePropertyTool", "PropertyChanged", "Animatable Property Changed"), bShouldActuallyTransact);

	TArray<UMovieSceneSubSection*> SectionsToKey;

	GetSectionsToKey(SectionsToKey);

	for(UMovieSceneSubSection* SubSection : SectionsToKey)
	{
		FTransform CurrentTransform = GetTransformOriginDataForSubSection(SubSection);
		
		FGeneratedTrackKeys GeneratedKeys;
		
		GetOriginKeys(CurrentTransform.GetLocation(), CurrentTransform.Rotator(), Channel, SubSection, GeneratedKeys);
		
		FKeyPropertyResult KeyResults = AddKeysToSection(SubSection, GetTimeForKey(), GeneratedKeys, ESequencerKeyMode::ManualKeyForced);
		
		SubSection->Modify(true);
	}
}

void FSubTrackEditor::OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel)
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s).
	// If the viewport is moving, disregard deleting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return;
		}
	}

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		if (SelectedActors->Num() > 0)
		{
			return;
		}
	}

	const TSharedPtr<ISequencer> PinnedSequencer = GetSequencer();
	if (!PinnedSequencer.IsValid())
	{
		return;
	}

	const FFrameNumber CurrentFrame = PinnedSequencer->GetLocalTime().Time.FrameNumber;
	const TRange<FFrameNumber> FrameRange(
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame),
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame));

	const bool bShouldActuallyTransact = !GIsTransacting;
	FScopedTransaction DeleteKeyTransaction(NSLOCTEXT("AnimatablePropertyTool", "PropertyChanged", "Animatable Property Changed"), bShouldActuallyTransact);

	TArray<UMovieSceneSubSection*> SectionsToKey;
	GetSectionsToKey(SectionsToKey);

	// Channel names and their corresponding transform channel mask bits
	struct FOriginChannelInfo { const TCHAR* Name; EMovieSceneTransformChannel MaskBit; };
	const FOriginChannelInfo OriginChannels[] =
	{
		{ TEXT("Override.Location.X"), EMovieSceneTransformChannel::TranslationX },
		{ TEXT("Override.Location.Y"), EMovieSceneTransformChannel::TranslationY },
		{ TEXT("Override.Location.Z"), EMovieSceneTransformChannel::TranslationZ },
		{ TEXT("Override.Rotation.X"), EMovieSceneTransformChannel::RotationX    },
		{ TEXT("Override.Rotation.Y"), EMovieSceneTransformChannel::RotationY    },
		{ TEXT("Override.Rotation.Z"), EMovieSceneTransformChannel::RotationZ    },
	};

	for (UMovieSceneSubSection* SubSection : SectionsToKey)
	{
		FMovieSceneChannelProxy& SectionChannelProxy = SubSection->GetChannelProxy();
		bool bModified = false;

		for (const FOriginChannelInfo& Info : OriginChannels)
		{
			if (!EnumHasAnyFlags(Channel, Info.MaskBit))
			{
				continue;
			}

			FMovieSceneDoubleChannel* DoubleChannel = SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>(Info.Name).Get();
			if (!DoubleChannel)
			{
				continue;
			}

			TArray<FFrameNumber> KeyTimes;
			TArray<FKeyHandle> KeyHandles;
			DoubleChannel->GetData().GetKeys(FrameRange, &KeyTimes, &KeyHandles);
			if (KeyHandles.Num() > 0)
			{
				if (!bModified)
				{
					SubSection->Modify();
					bModified = true;
				}
				DoubleChannel->DeleteKeys(MakeArrayView(KeyHandles));
			}
		}
	}
}

void FSubTrackEditor::InsertSection(UMovieSceneTrack* Track)
{
	FFrameTime NewSectionStartTime = GetSequencer()->GetLocalTime().Time;

	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UMovieSceneSubTrack* SubTrack = FindOrCreateSubTrack(MovieScene, Track);

	TSubclassOf<UMovieSceneSequence> OwningSequenceClass = nullptr;
	if (UMovieSceneSequence* OwningSequence = SubTrack->GetTypedOuter<UMovieSceneSequence>())
	{
		OwningSequenceClass = OwningSequence->GetClass();
	}

	FString NewSequenceName = MovieSceneToolHelpers::GenerateNewSubsequenceName(SubTrack->GetAllSections(), GetDefaultSubsequenceName(), NewSectionStartTime.FrameNumber);
	FString NewSequencePath = MovieSceneToolHelpers::GenerateNewSubsequencePath(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), GetDefaultSubsequenceDirectory(), NewSequenceName);

	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(NewSequencePath + TEXT("/") + NewSequenceName, TEXT(""), NewSequencePath, NewSequenceName);

	if (UMovieSceneSequence* NewSequence = MovieSceneToolHelpers::CreateSequence(NewSequenceName, NewSequencePath))
	{
		const FScopedTransaction Transaction(FText::Join(FText::FromString(" "), LOCTEXT("InsertText", "Insert"), GetSubTrackName()));

		int32 Duration = UE::MovieScene::DiscreteSize(NewSequence->GetMovieScene()->GetPlaybackRange());

		if (UMovieSceneSubSection* NewSection = SubTrack->AddSequence(NewSequence, NewSectionStartTime.FrameNumber, Duration))
		{
			NewSection->SetRowIndex(MovieSceneToolHelpers::FindAvailableRowIndex(Track, NewSection));

			GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}
}

void FSubTrackEditor::DuplicateSection(UMovieSceneSubSection* Section)
{
	UMovieSceneSubTrack* SubTrack = CastChecked<UMovieSceneSubTrack>(Section->GetOuter());

	FFrameNumber StartTime = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : 0;
	FString NewSectionName = MovieSceneToolHelpers::GenerateNewSubsequenceName(SubTrack->GetAllSections(), GetDefaultSubsequenceName(), StartTime);
	FString NewSequencePath = FPaths::GetPath(Section->GetSequence()->GetPathName());

	// Duplicate the section and put it on the next available row
	UMovieSceneSequence* NewSequence = MovieSceneToolHelpers::CreateSequence(NewSectionName, NewSequencePath, Section);
	if (NewSequence)
	{
		const FScopedTransaction Transaction(FText::Join(FText::FromString(" "), LOCTEXT("DuplicateText", "Duplicate"), GetSubTrackName()));

		int32 Duration = UE::MovieScene::DiscreteSize(Section->GetRange());

		if (UMovieSceneSubSection* NewSection = SubTrack->AddSequence(NewSequence, StartTime, Duration))
		{
			NewSection->SetRange(Section->GetRange());
			NewSection->SetRowIndex(MovieSceneToolHelpers::FindAvailableRowIndex(SubTrack, NewSection));
			NewSection->Parameters.StartFrameOffset = Section->Parameters.StartFrameOffset;
			NewSection->Parameters.TimeScale = Section->Parameters.TimeScale;
			NewSection->SetPreRollFrames(Section->GetPreRollFrames());
			NewSection->SetColorTint(Section->GetColorTint());

			GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}
}

void FSubTrackEditor::CreateNewTake(UMovieSceneSubSection* Section)
{
	FString ShotPrefix;
	uint32 ShotNumber = INDEX_NONE;
	uint32 TakeNumber = INDEX_NONE;
	uint32 ShotNumberDigits = 0;
	uint32 TakeNumberDigits = 0;
	
	FString SequenceName = Section->GetSequence() ? Section->GetSequence()->GetName() : FString();

	if (MovieSceneToolHelpers::ParseShotName(SequenceName, ShotPrefix, ShotNumber, TakeNumber, ShotNumberDigits, TakeNumberDigits))
	{
		TArray<FAssetData> AssetData;
		uint32 CurrentTakeNumber = INDEX_NONE;
		MovieSceneToolHelpers::GatherTakes(Section, AssetData, CurrentTakeNumber);
		uint32 NewTakeNumber = CurrentTakeNumber;

		for (auto ThisAssetData : AssetData)
		{
			uint32 ThisTakeNumber = INDEX_NONE;
			if (MovieSceneToolHelpers::GetTakeNumber(Section, ThisAssetData, ThisTakeNumber))
			{
				if (ThisTakeNumber >= NewTakeNumber)
				{
					NewTakeNumber = ThisTakeNumber + 1;
				}
			}
		}

		FString NewSectionName = MovieSceneToolHelpers::ComposeShotName(ShotPrefix, ShotNumber, NewTakeNumber, ShotNumberDigits, TakeNumberDigits);

		TRange<FFrameNumber> NewSectionRange         = Section->GetRange();
		FFrameNumber         NewSectionStartOffset   = Section->Parameters.StartFrameOffset;
		int32                NewSectionPrerollFrames = Section->GetPreRollFrames();
		int32                NewRowIndex          = Section->GetRowIndex();
		FFrameNumber         NewSectionStartTime     = NewSectionRange.GetLowerBound().IsClosed() ? UE::MovieScene::DiscreteInclusiveLower(NewSectionRange) : 0;
		FColor               NewSectionColorTint     = Section->GetColorTint();
		UMovieSceneSubTrack* SubTrack = CastChecked<UMovieSceneSubTrack>(Section->GetOuter());
		FString NewSequencePath = FPaths::GetPath(Section->GetSequence()->GetPathName());

		if (UMovieSceneSequence* NewSequence = MovieSceneToolHelpers::CreateSequence(NewSectionName, NewSequencePath, Section))
		{
			const FScopedTransaction Transaction(LOCTEXT("NewTake_Transaction", "New Take"));

			int32 Duration = UE::MovieScene::DiscreteSize(Section->GetRange());

			UMovieSceneSubSection* NewSection = SubTrack->AddSequence(NewSequence, NewSectionStartTime, Duration);
			SubTrack->RemoveSection(*Section);

			NewSection->SetRange(NewSectionRange);
			NewSection->Parameters.StartFrameOffset = NewSectionStartOffset;
			NewSection->Parameters.TimeScale = Section->Parameters.TimeScale.DeepCopy(NewSection);
			NewSection->SetPreRollFrames(NewSectionPrerollFrames);
			NewSection->SetRowIndex(NewRowIndex);
			NewSection->SetColorTint(NewSectionColorTint);

			UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section);
			UMovieSceneCinematicShotSection* NewShotSection = Cast<UMovieSceneCinematicShotSection>(NewSection);

			// If the old shot's name is not the same as the sequence's name, assume the user had customized the shot name, so carry it over
			if (ShotSection && NewShotSection && ShotSection->GetSequence() && ShotSection->GetShotDisplayName() != ShotSection->GetSequence()->GetName())
			{
				NewShotSection->SetShotDisplayName(ShotSection->GetShotDisplayName());
			}

			MovieSceneToolHelpers::SetTakeNumber(NewSection, NewTakeNumber);

			GetSequencer()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemsChanged );
			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}
}

void FSubTrackEditor::ChangeTake(UMovieSceneSequence* Sequence)
{
	bool bChangedTake = false;

	const FScopedTransaction Transaction(LOCTEXT("ChangeTake_Transaction", "Change Take"));

	TArray<UMovieSceneSection*> Sections;
	GetSequencer()->GetSelectedSections(Sections);

	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		if (!Sections[SectionIndex]->IsA<UMovieSceneSubSection>())
		{
			continue;
		}

		UMovieSceneSubSection* Section = Cast<UMovieSceneSubSection>(Sections[SectionIndex]);
		UMovieSceneSubTrack* SubTrack = CastChecked<UMovieSceneSubTrack>(Section->GetOuter());

		TRange<FFrameNumber> NewSectionRange = Section->GetRange();
		FFrameNumber		 NewSectionStartOffset = Section->Parameters.StartFrameOffset;
		int32                NewSectionPrerollFrames = Section->GetPreRollFrames();
		int32                NewRowIndex = Section->GetRowIndex();
		FFrameNumber         NewSectionStartTime = NewSectionRange.GetLowerBound().IsClosed() ? UE::MovieScene::DiscreteInclusiveLower(NewSectionRange) : 0;
		int32                NewSectionRowIndex = Section->GetRowIndex();
		FColor               NewSectionColorTint = Section->GetColorTint();

		const int32 Duration = (NewSectionRange.GetLowerBound().IsClosed() && NewSectionRange.GetUpperBound().IsClosed()) ? UE::MovieScene::DiscreteSize(NewSectionRange) : 1;
		UMovieSceneSubSection* NewSection = SubTrack->AddSequence(Sequence, NewSectionStartTime, Duration);

		if (NewSection != nullptr)
		{
			SubTrack->RemoveSection(*Section);

			NewSection->SetRange(NewSectionRange);
			NewSection->Parameters.StartFrameOffset = NewSectionStartOffset;
			NewSection->Parameters.TimeScale = Section->Parameters.TimeScale.DeepCopy(NewSection);
			NewSection->SetPreRollFrames(NewSectionPrerollFrames);
			NewSection->SetRowIndex(NewSectionRowIndex);
			NewSection->SetColorTint(NewSectionColorTint);

			UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section);
			UMovieSceneCinematicShotSection* NewShotSection = Cast<UMovieSceneCinematicShotSection>(NewSection);

			// If the old shot's name is not the same as the sequence's name, assume the user had customized the shot name, so carry it over
			if (ShotSection && NewShotSection && ShotSection->GetSequence() && ShotSection->GetShotDisplayName() != ShotSection->GetSequence()->GetName())
			{
				NewShotSection->SetShotDisplayName(ShotSection->GetShotDisplayName());
			}

			bChangedTake = true;
		}
	}

	if (bChangedTake)
	{
		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
}

void FSubTrackEditor::AddTakesMenu(UMovieSceneSubSection* Section, FMenuBuilder& MenuBuilder)
{
	TArray<FAssetData> AssetData;
	uint32 CurrentTakeNumber = INDEX_NONE;
	MovieSceneToolHelpers::GatherTakes(Section, AssetData, CurrentTakeNumber);

	AssetData.Sort([Section](const FAssetData& A, const FAssetData& B) {
		uint32 TakeNumberA = INDEX_NONE;
		uint32 TakeNumberB = INDEX_NONE;
		if (MovieSceneToolHelpers::GetTakeNumber(Section, A, TakeNumberA) && MovieSceneToolHelpers::GetTakeNumber(Section, B, TakeNumberB))
		{
			return TakeNumberA < TakeNumberB;
		}
		return true;
	});

	for (auto ThisAssetData : AssetData)
	{
		uint32 TakeNumber = INDEX_NONE;
		if (MovieSceneToolHelpers::GetTakeNumber(Section, ThisAssetData, TakeNumber))
		{
			UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(ThisAssetData.GetAsset());
			if (Sequence)
			{
				FText MetaDataText = FSubTrackEditorUtil::GetMetaDataText(Sequence);
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("TakeNumber", "Take {0}"), FText::AsNumber(TakeNumber)),
					MetaDataText.IsEmpty() ? 
					FText::Format(LOCTEXT("TakeNumberTooltip", "Change to {0}"), FText::FromString(Sequence->GetPathName())) : 
					FText::Format(LOCTEXT("TakeNumberWithMetaDataTooltip", "Change to {0}\n\n{1}"), FText::FromString(Sequence->GetPathName()), MetaDataText),
					TakeNumber == CurrentTakeNumber ? FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Star") : FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Empty"),
					FUIAction(FExecuteAction::CreateSP(this, &FSubTrackEditor::ChangeTake, Sequence))
				);
			}
		}
	}
}

TWeakPtr<SWindow> MetaDataWindow;

void FSubTrackEditor::EditMetaData(UMovieSceneSubSection* Section)
{
	UMovieSceneSequence* Sequence = Section->GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieSceneMetaData* MetaData = FSubTrackEditorUtil::FindOrAddMetaData(Sequence);
	if (!MetaData)
	{
		return;
	}

	TSharedPtr<SWindow> ExistingWindow = MetaDataWindow.Pin();
	if (ExistingWindow.IsValid())
	{
		ExistingWindow->BringToFront();
	}
	else
	{
		ExistingWindow = SNew(SWindow)
			.Title(FText::Format(LOCTEXT("MetaDataTitle", "Edit {0}"), FText::FromString(GetSubSectionDisplayName(Section))))
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.ClientSize(FVector2D(400, 200));

		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		if (ParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		}
	}

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowScrollBar = false;

	TSharedRef<IDetailsView> DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	TArray<UObject*> Objects;
	Objects.Add(MetaData);
	DetailsView->SetObjects(Objects, true);

	ExistingWindow->SetContent(DetailsView);

	MetaDataWindow = ExistingWindow;
}

void FSubTrackEditor::UpdateActiveMode()
{
	TArray<UMovieSceneSection*> Sections;
	GetSequencer()->GetSelectedSections(Sections);
	
	for(UMovieSceneSection* Section : Sections)
	{
		if(UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (SubSection->IsTransformOriginEditable())
			{
				GLevelEditorModeTools().ActivateDefaultMode();
				GLevelEditorModeTools().ActivateMode(FSubTrackEditorMode::ModeName);
				return;
			}
		}
	}
	
	TArray<UMovieSceneTrack*> Tracks;
	GetSequencer()->GetSelectedTracks(Tracks);

	for(UMovieSceneTrack* Track : Tracks)
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					if (SubSection->IsTransformOriginEditable())
					{
						GLevelEditorModeTools().ActivateDefaultMode();
						GLevelEditorModeTools().ActivateMode(FSubTrackEditorMode::ModeName);
						return;
					}
				}
			}
		}
	}
}

bool FSubTrackEditor::CanAddSubSequence(const UMovieSceneSequence& Sequence) const
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	return FSubTrackEditorUtil::CanAddSubSequence(FocusedSequence, Sequence);
}

FText FSubTrackEditor::GetSubTrackName() const
{
	return LOCTEXT("SubTrackName", "Subsequence Track");
}

FText FSubTrackEditor::GetSubTrackToolTip() const
{ 
	return LOCTEXT("SubTrackToolTip", "A track that can contain other sequences.");
}

FName FSubTrackEditor::GetSubTrackBrushName() const
{
	return TEXT("Sequencer.Tracks.Sub");
}

FString FSubTrackEditor::GetSubSectionDisplayName(const UMovieSceneSubSection* Section) const
{
	return Section && Section->GetSequence() ? Section->GetSequence()->GetName() : FString();
}

FString FSubTrackEditor::GetDefaultSubsequenceName() const
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	return ProjectSettings->SubsequencePrefix;
}

FString FSubTrackEditor::GetDefaultSubsequenceDirectory() const
{
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
	return ProjectSettings->SubsequenceDirectory;
}

TSubclassOf<UMovieSceneSubTrack> FSubTrackEditor::GetSubTrackClass() const
{
	return UMovieSceneSubTrack::StaticClass();
}

void FSubTrackEditor::UpdateOrigin(FVector InPosition, FRotator InRotation)
{
	if(!GetSequencer())
	{
		return;
	}
	
	if(!GetSequencer()->IsAllowedToChange())
	{
		return;
	}

	UMovieSceneSequence* MovieSceneSequence = GetMovieSceneSequence();
	if (!MovieSceneSequence)
	{
		return;
	}
	
	// @todo Sequencer - The sequencer probably should have taken care of this
	MovieSceneSequence->SetFlags(RF_Transactional);

	TArray<UMovieSceneSubSection*> SectionsToKey;

	GetSectionsToKey(SectionsToKey);
	
	for(UMovieSceneSubSection* SubSection : SectionsToKey)
	{
		FTransform PreviousTransform = GetTransformOriginDataForSubSection(SubSection);
		
		// Default to dirtying the section, since we may be adding keys 
		bool bShouldMarkDirty = true;
		
		FGeneratedTrackKeys GeneratedKeys;
		
		GetOriginKeys(PreviousTransform.GetLocation() + InPosition, (PreviousTransform * FTransform(InRotation)).Rotator(), EMovieSceneTransformChannel::Translation | EMovieSceneTransformChannel::Rotation, SubSection, GeneratedKeys);
		
		FKeyPropertyResult KeyResults = AddKeysToSection(SubSection, GetTimeForKey(), GeneratedKeys, ESequencerKeyMode::AutoKey);

		// If a key wasn't created, but there is keyframe data on this section, it's preview data needs to be set to visualize the origin position for manual keyframing.
		if(!KeyResults.bKeyCreated && SubSection->HasAnyChannelData())
		{
			// Preview key data is transient, and should not dirty the sequence.
			bShouldMarkDirty = false;
			SubSection->SetKeyPreviewPosition(PreviousTransform.GetLocation() + InPosition);
			SubSection->SetKeyPreviewRotation((PreviousTransform * FTransform(InRotation)).Rotator());

			// Preview data needs to be reverted when playing the sequence, scrubbing the sequence, or navigating to a different sequence
			// This array keeps track of sections with preview data and reverts them in FSubTrackEditor::RevertSectionPreviews
			SectionsWithPreviews.AddUnique(SubSection);
			
			// Manually mark as changed since modify will not call it if not marked as dirty.
			SubSection->MarkAsChanged();
		}
		SubSection->Modify(bShouldMarkDirty);
		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
	
}

void FSubTrackEditor::ResetSectionPreviews()
{
	bool bSectionReverted = false;
	
	for (UMovieSceneSubSection* SubSection : SectionsWithPreviews)
	{
		if (SubSection && (SubSection->GetKeyPreviewPosition().IsSet() || SubSection->GetKeyPreviewRotation().IsSet()))
		{
			SubSection->ResetKeyPreviewRotationAndLocation();
			SubSection->Modify(false);
			SubSection->MarkAsChanged();
			bSectionReverted = true;
		}
	}

	SectionsWithPreviews.Empty();

	FSubTrackEditorMode* EditorMode = static_cast<FSubTrackEditorMode*>(GLevelEditorModeTools().GetActiveMode(FSubTrackEditorMode::ModeName));

	if (EditorMode)
	{
		EditorMode->ClearCachedCoordinates();
	}
	
	if (bSectionReverted)
	{
		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);		
	}

}

FTransform FSubTrackEditor::GetTransformOriginDataForSubSection(const UMovieSceneSubSection* SubSection) const
{
	FTransform TransformOrigin;

	if(!GetSequencer())
	{
		return TransformOrigin;
	}
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();

	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if(!EntityLinker)
	{
		return TransformOrigin;
	}

	const UMovieSceneTransformOriginSystem* TransformOriginSystem = EntityLinker->FindSystem<UMovieSceneTransformOriginSystem>();

	if(TransformOriginSystem)
	{
		UE::MovieScene::FBuiltInComponentTypes* BuiltInComponentTypes = UE::MovieScene::FBuiltInComponentTypes::Get();
	
		UMovieSceneCompiledDataManager* CompiledDataManager = EvaluationTemplate.GetCompiledDataManager();
		UMovieSceneSequence* RootSequence = EvaluationTemplate.GetSequence(GetSequencer()->GetRootTemplateID());
		const FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence);

		const FMovieSceneSequenceHierarchy& Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);
	
		TArray<FMovieSceneSequenceID> SubSequenceHierarchy = GetSequencer()->GetSubSequenceHierarchy();

		UE::MovieScene::FSubSequencePath Path;
		const FMovieSceneSequenceID ParentSequenceID = SubSequenceHierarchy.Last();

		Path.Reset(ParentSequenceID, &Hierarchy);

		FMovieSceneSequenceID SequenceID = Path.ResolveChildSequenceID(SubSection->GetSequenceID());
		
		FVector OutLocation = FVector(0, 0, 0);
		FRotator OutRotation = FRotator(0, 0, 0);

		// Query the channel results directly. 
		auto ReadSectionTransformOrigin = [&OutLocation, &OutRotation, SequenceID](const UE::MovieScene::FEntityAllocation* Allocation, UE::MovieScene::TRead<UE::MovieScene::FRootInstanceHandle> RootInstances, UE::MovieScene::TRead<FMovieSceneSequenceID> SequenceIDs,
			UE::MovieScene::TReadOptional<double> LocationX, UE::MovieScene::TReadOptional<double> LocationY, UE::MovieScene::TReadOptional<double> LocationZ,
			UE::MovieScene::TReadOptional<double> RotationX, UE::MovieScene::TReadOptional<double> RotationY, UE::MovieScene::TReadOptional<double> RotationZ)
		{
			const int32 Num = Allocation->Num();
			
			for(int32 Index = 0; Index < Num; ++Index)
			{
				if(SequenceID == SequenceIDs[Index]) 
				{
					OutLocation = FVector(LocationX ? LocationX[Index] : 0.0f, LocationY ? LocationY[Index] : 0.f, LocationZ ? LocationZ[Index] : 0.f);
					OutRotation = FRotator(RotationY ? RotationY[Index] : 0.f, RotationZ ? RotationZ[Index] : 0.f, RotationX ? RotationX[Index] : 0.f);
				}
			}
		};

		UE::MovieScene::FEntityTaskBuilder()
		.Read(BuiltInComponentTypes->RootInstanceHandle)
		.Read(BuiltInComponentTypes->SequenceID)
		.ReadOptional(BuiltInComponentTypes->DoubleResult[0])
		.ReadOptional(BuiltInComponentTypes->DoubleResult[1])
		.ReadOptional(BuiltInComponentTypes->DoubleResult[2])
		.ReadOptional(BuiltInComponentTypes->DoubleResult[3])
		.ReadOptional(BuiltInComponentTypes->DoubleResult[4])
		.ReadOptional(BuiltInComponentTypes->DoubleResult[5])
		.FilterAll({BuiltInComponentTypes->Tags.SubInstance})
		.FilterNone({BuiltInComponentTypes->Tags.ImportedEntity})
		.FilterAny(
		{
			BuiltInComponentTypes->DoubleResult[0],
			BuiltInComponentTypes->DoubleResult[1],
			BuiltInComponentTypes->DoubleResult[2],
			BuiltInComponentTypes->DoubleResult[3],
			BuiltInComponentTypes->DoubleResult[4],
			BuiltInComponentTypes->DoubleResult[5]
		})
		.Iterate_PerAllocation(&EntityLinker->EntityManager, ReadSectionTransformOrigin);

		TransformOrigin = FTransform(OutRotation, OutLocation);
	}

	return  TransformOrigin;
}

void FSubTrackEditor::GetSupportedSequenceClassPaths(TArray<FTopLevelAssetPath>& ClassPaths) const
{
	ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/LevelSequence"), TEXT("LevelSequence")));
}

UMovieSceneSubTrack* FSubTrackEditor::CreateNewTrack(UMovieScene* MovieScene) const
{
	return Cast<UMovieSceneSubTrack>(MovieScene->AddTrack(GetSubTrackClass()));
}

/* FSubTrackEditor callbacks
 *****************************************************************************/

void FSubTrackEditor::HandleAddSubTrackMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(FText::Join(FText::FromString(" "), LOCTEXT("AddText", "Add"), GetSubTrackName()));
	FocusedMovieScene->Modify();

	UMovieSceneSubTrack* NewTrack = FindOrCreateSubTrack(FocusedMovieScene, nullptr);
	ensure(NewTrack);

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

UMovieSceneSubTrack* FSubTrackEditor::FindOrCreateSubTrack(UMovieScene* MovieScene, UMovieSceneTrack* Track) const
{
	UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
	if (!SubTrack)
	{
		SubTrack = Cast<UMovieSceneSubTrack>(MovieScene->AddTrack(GetSubTrackClass()));
	}
	return SubTrack;
}

TSharedRef<SWidget> FSubTrackEditor::HandleAddSubSequenceComboButtonGetMenuContent(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel)
{
	using namespace UE::Sequencer;

	TViewModelPtr<FViewModel> ViewModel = WeakViewModel.Pin();
	if (!ViewModel)
	{
		return SNullWidget::NullWidget;
	}

	TViewModelPtr<ITrackExtension> TrackModel = ViewModel->FindAncestorOfType<ITrackExtension>(true);

	if (!TrackModel)
	{
		return SNullWidget::NullWidget;
	}

	UMovieSceneTrack* Track = TrackModel->GetTrack();

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarpCategory", "Time Warp"));
	{
		FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, WeakViewModel);
	}

	MenuBuilder.BeginSection(TEXT("ChooseSequence"), LOCTEXT("InsertSequence", "Insert Sequence"));
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("CreateNewText", "Create New {0} Asset"), GetSubTrackName()),
			FText::Format(LOCTEXT("CreateNewSectionTooltip", "Create new {0} asset and insert it at current time"), GetSubTrackName()),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FSubTrackEditor::InsertSection, Track))
		);

		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;
		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw( this, &FSubTrackEditor::HandleAddSubSequenceComboButtonMenuEntryExecute, Track);
			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw( this, &FSubTrackEditor::HandleAddSubSequenceComboButtonMenuEntryEnterPressed, Track);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			GetSupportedSequenceClassPaths(AssetPickerConfig.Filter.ClassPaths);
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
			AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		TSharedPtr<SBox> MenuEntry = SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			];

		MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FSubTrackEditor::HandleAddSubSequenceComboButtonMenuEntryExecute(const FAssetData& AssetData, UMovieSceneTrack* InTrack)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();

	if (SelectedObject && SelectedObject->IsA(UMovieSceneSequence::StaticClass()))
	{
		UMovieSceneSequence* MovieSceneSequence = CastChecked<UMovieSceneSequence>(AssetData.GetAsset());

		int32 RowIndex = INDEX_NONE;
		AnimatablePropertyChanged( FOnKeyProperty::CreateRaw( this, &FSubTrackEditor::AddKeyInternal, MovieSceneSequence, InTrack, RowIndex) );
	}
}

void FSubTrackEditor::HandleAddSubSequenceComboButtonMenuEntryEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* InTrack)
{
	if (AssetData.Num() > 0)
	{
		HandleAddSubSequenceComboButtonMenuEntryExecute(AssetData[0].GetAsset(), InTrack);
	}
}

void FSubTrackEditor::AddSequenceToTrack(UMovieSceneSequence* InMovieSceneSequence, UMovieSceneTrack* InTrack)
{
	int32 RowIndex = INDEX_NONE;
	AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FSubTrackEditor::AddKeyInternal, InMovieSceneSequence, InTrack, RowIndex));
}

FKeyPropertyResult FSubTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UMovieSceneSequence* InMovieSceneSequence, UMovieSceneTrack* InTrack, int32 RowIndex)
{	
	FKeyPropertyResult KeyPropertyResult;

	if (InMovieSceneSequence->GetMovieScene()->GetPlaybackRange().IsEmpty())
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequenceDuration", "Invalid level sequence {0}. The sequence has no duration."), InMovieSceneSequence->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return KeyPropertyResult;
	}

	if (CanAddSubSequence(*InMovieSceneSequence))
	{
		UMovieScene* MovieScene = GetFocusedMovieScene();

		UMovieSceneSubTrack* SubTrack = FindOrCreateSubTrack(MovieScene, InTrack);

		const FFrameRate TickResolution = InMovieSceneSequence->GetMovieScene()->GetTickResolution();
		const FQualifiedFrameTime InnerDuration = FQualifiedFrameTime(
			UE::MovieScene::DiscreteSize(InMovieSceneSequence->GetMovieScene()->GetPlaybackRange()),
			TickResolution);

		const FFrameRate OuterFrameRate = SubTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
		const int32      OuterDuration  = InnerDuration.ConvertTo(OuterFrameRate).FrameNumber.Value;

		UMovieSceneSubSection* NewSection = SubTrack->AddSequenceOnRow(InMovieSceneSequence, KeyTime, OuterDuration, RowIndex);
		KeyPropertyResult.bTrackModified = true;
		KeyPropertyResult.SectionsCreated.Add(NewSection);

		GetSequencer()->EmptySelection();
		GetSequencer()->SelectSection(NewSection);
		GetSequencer()->ThrobSectionSelection();

		if (TickResolution != OuterFrameRate)
		{
			FNotificationInfo Info(FText::Format(LOCTEXT("TickResolutionMismatch", "The parent sequence has a different tick resolution {0} than the newly added sequence {1}"), OuterFrameRate.ToPrettyText(), TickResolution.ToPrettyText()));
			Info.bUseLargeFont = false;
			FSlateNotificationManager::Get().AddNotification(Info);
		}

		return KeyPropertyResult;
	}

	FNotificationInfo Info(FText::Format( LOCTEXT("InvalidSequence", "Invalid level sequence {0}. There could be a circular dependency or the sequences are not compatible."), InMovieSceneSequence->GetDisplayName()));
	Info.bUseLargeFont = false;
	FSlateNotificationManager::Get().AddNotification(Info);

	return KeyPropertyResult;
}

FKeyPropertyResult FSubTrackEditor::HandleSequenceAdded(FFrameNumber KeyTime, UMovieSceneSequence* Sequence, UMovieSceneTrack* Track, int32 RowIndex)
{
	FKeyPropertyResult KeyPropertyResult;

	UMovieScene* MovieScene = GetFocusedMovieScene();

	UMovieSceneSubTrack* SubTrack = FindOrCreateSubTrack(MovieScene, Track);

	const FFrameRate TickResolution = Sequence->GetMovieScene()->GetTickResolution();
	const FQualifiedFrameTime InnerDuration = FQualifiedFrameTime(
		UE::MovieScene::DiscreteSize(Sequence->GetMovieScene()->GetPlaybackRange()),
		TickResolution);

	const FFrameRate OuterFrameRate = SubTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
	const int32      OuterDuration  = InnerDuration.ConvertTo(OuterFrameRate).FrameNumber.Value;

	UMovieSceneSubSection* NewSection = SubTrack->AddSequenceOnRow(Sequence, KeyTime, OuterDuration, RowIndex);
	KeyPropertyResult.bTrackModified = true;
	KeyPropertyResult.SectionsCreated.Add(NewSection);

	GetSequencer()->EmptySelection();
	GetSequencer()->SelectSection(NewSection);
	GetSequencer()->ThrobSectionSelection();

	if (TickResolution != OuterFrameRate)
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("TickResolutionMismatch", "The parent sequence has a different tick resolution {0} than the newly added sequence {1}"), OuterFrameRate.ToPrettyText(), TickResolution.ToPrettyText()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	return KeyPropertyResult;
}

#undef LOCTEXT_NAMESPACE
