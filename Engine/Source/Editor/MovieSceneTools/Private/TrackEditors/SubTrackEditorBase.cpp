// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/SubTrackEditorBase.h"
#include "Fonts/FontCache.h"
#include "FrameNumberDisplayFormat.h"
#include "FrameNumberNumericInterface.h"
#include "MovieScene.h"
#include "LevelSequence.h"
#include "MovieSceneMetaData.h"
#include "SequencerSettings.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "TimeToPixel.h"

#define LOCTEXT_NAMESPACE "FSubTrackEditorBase"

FSubSectionPainterResult FSubSectionPainterUtil::PaintSection(TSharedPtr<const ISequencer> Sequencer, const UMovieSceneSubSection& SectionObject, FSequencerSectionPainter& InPainter, FSubSectionPainterParams Params)
{
	using namespace UE::MovieScene;

	const TRange<FFrameNumber> SectionRange = SectionObject.GetRange();
	if (SectionRange.GetLowerBound().IsOpen() || SectionRange.GetUpperBound().IsOpen())
	{
		return FSSPR_InvalidSection;
	}

	const int32 SectionSize = UE::MovieScene::DiscreteSize(SectionRange);
	if (SectionSize <= 0)
	{
		return FSSPR_InvalidSection;
	}

	UMovieSceneSequence* InnerSequence = SectionObject.GetSequence();
	if (InnerSequence == nullptr || InnerSequence->GetMovieScene() == nullptr)
	{
		return FSSPR_NoInnerSequence;
	}

	const ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	PaintSectionBounds(Sequencer, SectionObject, *InnerSequence, InPainter, DrawEffects);

	UMovieScene* MovieScene = InnerSequence->GetMovieScene();
	const int32 NumTracks = MovieScene->GetPossessableCount() + MovieScene->GetSpawnableCount() + MovieScene->GetTracks().Num();

	FVector2D TopLeft = InPainter.SectionGeometry.AbsoluteToLocal(InPainter.SectionClippingRect.GetTopLeft()) + FVector2D(1.f, -1.f);

	FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");

	TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

	auto GetFontHeight = [&]
	{
		return FontCache->GetMaxCharacterHeight(FontInfo, 1.f) + FontCache->GetBaseline(FontInfo, 1.f);
	};
	while (GetFontHeight() > InPainter.SectionGeometry.Size.Y && FontInfo.Size > 11)
	{
		FontInfo.Size = FMath::Max(FMath::FloorToInt(FontInfo.Size - 6.f), 11);
	}

	uint32 LayerId = InPainter.LayerId;
	FMargin ContentPadding = Params.ContentPadding;

	FText TrackNumText, NetworkText;
	if (Params.bShowTrackNum)
	{
		TrackNumText = FText::Format(LOCTEXT("NumTracksFormat", "{0} track(s)"), FText::AsNumber(NumTracks));
	}

	EMovieSceneServerClientMask NetworkMask = SectionObject.GetNetworkMask();
	if (NetworkMask == EMovieSceneServerClientMask::Client)
	{
		NetworkText = LOCTEXT("SubSectionClientOnlyText", "(client only)");
	}
	else if (NetworkMask == EMovieSceneServerClientMask::Server)
	{
		NetworkText = LOCTEXT("SubSectionServerOnlyText", "(server only)");
	}

	FText SectionText;
	if (!TrackNumText.IsEmpty() || !NetworkText.IsEmpty())
	{
		SectionText = FText::Format(LOCTEXT("SectionTextFormat", "{0} {1}"), TrackNumText, NetworkText);
	}
	else if (!TrackNumText.IsEmpty())
	{
		SectionText = TrackNumText;
	}
	else
	{
		SectionText = NetworkText;
	}


	if (!SectionText.IsEmpty())
	{
		FSlateClippingZone ClippingZone(InPainter.SectionClippingRect.InsetBy(FMargin(1.0f)));

		InPainter.DrawElements.PushClip(ClippingZone);
		
		FSlateDrawElement::MakeText(
			InPainter.DrawElements,
			++LayerId,
			InPainter.SectionGeometry.ToPaintGeometry(
				FVector2D(InPainter.SectionGeometry.Size.X, GetFontHeight()),
				FSlateLayoutTransform(TopLeft + FVector2D(ContentPadding.Left, ContentPadding.Top) + FVector2D(11.f, GetFontHeight()*2.f))
			),
			SectionText,
			FontInfo,
			DrawEffects,
			FColor(200, 200, 200, static_cast<uint8>(255 * InPainter.GhostAlpha))
		);

		InPainter.DrawElements.PopClip();
	}

	InPainter.LayerId = LayerId;

	return FSSPR_Success;
}

void FSubSectionPainterUtil::PaintSectionBounds(TSharedPtr<const ISequencer> Sequencer, const UMovieSceneSubSection& SectionObject, const UMovieSceneSequence& InnerSequence, FSequencerSectionPainter& InPainter, ESlateDrawEffect DrawEffects)
{
	const uint8        GhostAlpha        = static_cast<uint8>(255 * InPainter.GhostAlpha);
	const FFrameNumber SectionStartFrame = SectionObject.GetInclusiveStartFrame();
	const UMovieScene* MovieScene        = InnerSequence.GetMovieScene();

	const FMovieSceneSequenceTransform        OuterToInnerTransform = SectionObject.OuterToInnerTransform();
	const FMovieSceneInverseSequenceTransform InnerToOuterTransform = OuterToInnerTransform.Inverse();

	const TRange<FFrameTime> SectionTimeRange = UE::MovieScene::ConvertToFrameTimeRange(SectionObject.GetRange());
	const int32              SectionSize      = UE::MovieScene::DiscreteSize(SectionObject.GetRange());

	int32 LayerId = InPainter.LayerId;
	const int32 LineLayerIdOffset = 10; // Draw the boundary lines on a layer after the dimmed boxes
	const int32 BoxLayerIdOffset = 5; // Draw the dimmed boxes on a layer before the boundary lines
	InPainter.LayerId = LayerId + LineLayerIdOffset + BoxLayerIdOffset + 1;

	TOptional<FFrameTime> FirstTime, LastTime;
	auto PaintDashedLine = [&](FFrameTime InTime, FColor Tint, float LineNudge)
	{
		float Offset = InPainter.GetTimeConverter().FrameToPixel(InTime) + LineNudge;

		if (!FirstTime)
		{
			FirstTime = InTime;
		}
		LastTime = InTime;

		// Don't actually draw it if it is out of bounds
		if (Offset < 0 || Offset >= InPainter.SectionGeometry.Size.X)
		{
			return;
		}

		TArray<FVector2f> NewVector;
		NewVector.Reserve(2);

		NewVector.Add(FVector2f(Offset, 0.f));
		NewVector.Add(FVector2f(Offset, InPainter.SectionGeometry.Size.Y));

		constexpr float Thickness = 1.f;
		constexpr float DashLengthPx = 3.f;
		FSlateDrawElement::MakeDashedLines(
			InPainter.DrawElements,
			LayerId + LineLayerIdOffset,
			InPainter.SectionGeometry.ToPaintGeometry(),
			MoveTemp(NewVector),
			DrawEffects,
			Tint,
			Thickness,
			DashLengthPx
		);
	};

	auto PaintLine = [&](FFrameTime InTime, FColor Tint, float LineNudge)
	{
		float Offset = InPainter.GetTimeConverter().FrameToPixel(InTime) + LineNudge;

		if (!FirstTime)
		{
			FirstTime = InTime;
		}
		LastTime = InTime;

		// Don't actually draw it if it is out of bounds
		if (Offset < 0 || Offset >= InPainter.SectionGeometry.Size.X)
		{
			return;
		}

		// add green line for playback start
		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			LayerId + LineLayerIdOffset,
			InPainter.SectionGeometry.ToPaintGeometry(
				FVector2f(1.0f, InPainter.SectionGeometry.Size.Y),
				FSlateLayoutTransform(FVector2f(Offset, 0.f))
			),
			FAppStyle::GetBrush("WhiteBrush"),
			DrawEffects,
			Tint
		);
	};


	FLinearColor PlaybackRangeStartColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeStartColor();
	FLinearColor PlaybackRangeEndColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeEndColor();

	FColor RedTint(192, 48, 48, GhostAlpha);	// 0,   75, 75 (HSV)
	FLinearColor SectionTint = InPainter.GetSectionColor().LinearRGBToHSV();
	SectionTint.B *= 0.1f;
	SectionTint = SectionTint.HSVToLinearRGB();

	auto PaintBoundary = [PaintDashedLine, SectionTint, LayerId](FFrameTime Time)
	{
		PaintDashedLine(Time, SectionTint.ToFColor(true), 0.f);
		return true;
	};

	if (!OuterToInnerTransform.ExtractBoundariesWithinRange(SectionStartFrame, SectionStartFrame + SectionSize, PaintBoundary))
	{
		// Just use the playback range
		TOptional<FFrameTime> StartBound = InnerToOuterTransform.TryTransformTime(MovieScene->GetPlaybackRange().GetLowerBoundValue());
		TOptional<FFrameTime> EndBound = InnerToOuterTransform.TryTransformTime(MovieScene->GetPlaybackRange().GetUpperBoundValue());

		if (StartBound)
		{
			PaintLine(StartBound.GetValue(), PlaybackRangeStartColor.CopyWithNewOpacity(GhostAlpha).ToFColor(true), 0.f);
		}
		if (EndBound)
		{
			PaintLine(EndBound.GetValue(), PlaybackRangeEndColor.CopyWithNewOpacity(GhostAlpha).ToFColor(true), 1.f);
		}
	}

	// Paint the dimmed section range before the first valid time
	if (FirstTime && FirstTime.GetValue() > SectionTimeRange.GetLowerBoundValue())
	{
		TOptional<FFrameTime> PreceedingInvalidTime;
		auto SetPreceedingInvalidTime = [&PreceedingInvalidTime](FFrameTime InTime)
		{
			PreceedingInvalidTime = InTime;
			return false;
		};

		OuterToInnerTransform.ExtractBoundariesWithinRange(MIN_int32, SectionTimeRange.GetLowerBoundValue(), SetPreceedingInvalidTime);
		if (!PreceedingInvalidTime)
		{
			const float WidthFactor = FMath::Min(1.f, static_cast<float>((FirstTime.GetValue().AsDecimal() - SectionTimeRange.GetLowerBoundValue().AsDecimal()) / SectionTimeRange.Size<FFrameTime>().AsDecimal()));

			FSlateDrawElement::MakeBox(
				InPainter.DrawElements,
				LayerId + BoxLayerIdOffset,
				InPainter.SectionGeometry.ToPaintGeometry(
					FVector2f(InPainter.SectionGeometry.Size.X * WidthFactor-1, InPainter.SectionGeometry.Size.Y),
					FSlateLayoutTransform()
				),
				FAppStyle::GetBrush("WhiteBrush"),
				DrawEffects,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);
		}
	}

	// Paint the dimmed section range after the last valid time
	if (LastTime && LastTime.GetValue() < SectionTimeRange.GetUpperBoundValue())
	{
		TOptional<FFrameTime> ProceedingInvalidTime;
		auto SetProceedingInvalidTime = [&ProceedingInvalidTime](FFrameTime InTime)
		{
			ProceedingInvalidTime = InTime;
			return false;
		};

		OuterToInnerTransform.ExtractBoundariesWithinRange(SectionTimeRange.GetUpperBoundValue(), MAX_int32, SetProceedingInvalidTime);
		if (!ProceedingInvalidTime)
		{
			const float Offset      = FMath::Max(0.f, InPainter.GetTimeConverter().FrameToPixel(LastTime.GetValue()) + 1);
			const float WidthFactor = FMath::Min(1.f, 1.f - static_cast<float>((LastTime.GetValue().AsDecimal() - SectionTimeRange.GetLowerBoundValue().AsDecimal()) / SectionTimeRange.Size<FFrameTime>().AsDecimal()));

			FSlateDrawElement::MakeBox(
				InPainter.DrawElements,
				LayerId + BoxLayerIdOffset,
				InPainter.SectionGeometry.ToPaintGeometry(
					FVector2f(InPainter.SectionGeometry.Size.X * WidthFactor-1, InPainter.SectionGeometry.Size.Y),
					FSlateLayoutTransform(FVector2f(Offset, 0.f))
				),
				FAppStyle::GetBrush("WhiteBrush"),
				DrawEffects,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);
		}
	}
}


FSubSectionEditorUtil::FSubSectionEditorUtil(UMovieSceneSubSection& InSection)
	: SectionObject(InSection)
	, PreDilateTimeScale(1.f)
{
}

FSubSectionEditorUtil::~FSubSectionEditorUtil()
{}

void FSubSectionEditorUtil::BeginResizeSection()
{
	InitialDragTransform = MakeUnique<FMovieSceneSequenceTransform>(SectionObject.OuterToInnerTransform());
}

FFrameNumber FSubSectionEditorUtil::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	UMovieSceneSequence* InnerSequence = SectionObject.GetSequence();
	UMovieScene* InnerMovieScene = InnerSequence ? InnerSequence->GetMovieScene() : nullptr;

	if (SectionObject.Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		return ResizeTime;
	}

	if (ResizeMode == SSRM_LeadingEdge && InnerMovieScene != nullptr)
	{
		TRange<FFrameNumber> Range = InnerMovieScene->GetPlaybackRange();

		// Find the new inner offset as an absolute time
		FFrameNumber NewStartTime = InitialDragTransform->TransformTime(ResizeTime).RoundToFrame();

		// Inner offset must be relative to playback start
		NewStartTime = NewStartTime - Range.GetLowerBoundValue();

		if (SectionObject.Parameters.bCanLoop)
		{
			SectionObject.Parameters.FirstLoopStartFrameOffset = NewStartTime;
		}
		else
		{
			SectionObject.Parameters.StartFrameOffset = NewStartTime;
		}
	}

	return ResizeTime;
}

void FSubSectionEditorUtil::BeginSlipSection()
{
	// Cache the same values as when resizing.
	BeginResizeSection();
}

FFrameNumber FSubSectionEditorUtil::SlipSection(FFrameNumber SlipTime)
{
	UMovieSceneSequence* InnerSequence = SectionObject.GetSequence();
	UMovieScene* InnerMovieScene = InnerSequence ? InnerSequence->GetMovieScene() : nullptr;

	if (InnerMovieScene != nullptr)
	{
		TRange<FFrameNumber> Range = InnerMovieScene->GetPlaybackRange();

		// Find the new inner offset as an absolute time
		FFrameNumber NewStartTime = InitialDragTransform->TransformTime(SlipTime).RoundToFrame();

		// Inner offset must be relative to playback start
		NewStartTime = NewStartTime - Range.GetLowerBoundValue();

		if (SectionObject.Parameters.bCanLoop)
		{
			SectionObject.Parameters.FirstLoopStartFrameOffset = NewStartTime;
		}
		else
		{
			SectionObject.Parameters.StartFrameOffset = NewStartTime;
		}
	}

	return SlipTime;
}

void FSubSectionEditorUtil::BeginDilateSection()
{
	if (SectionObject.Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		PreDilateTimeScale = SectionObject.Parameters.TimeScale.AsFixedPlayRate(); //make sure to cache the play rate
	}
	else if (SectionObject.Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		FMovieSceneTimeWarpChannel* Channel = SectionObject.GetChannelProxy().GetChannel<FMovieSceneTimeWarpChannel>(0);
		if (Channel)
		{
			SectionObject.Parameters.TimeScale.AsCustom()->Modify();
			PreDilateChannel = MakeUnique<FMovieSceneTimeWarpChannel>(*Channel);
		}
	}
}

void FSubSectionEditorUtil::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	if (SectionObject.Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		// Clamp dilation to a 'sensible' range
		double NewDilation = FMath::Clamp(PreDilateTimeScale / DilationFactor, -1000000.0, 1000000);
		SectionObject.Parameters.TimeScale.Set(NewDilation);
	}
	else if (SectionObject.Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		FMovieSceneTimeWarpChannel* Channel = SectionObject.GetChannelProxy().GetChannel<FMovieSceneTimeWarpChannel>(0);
		if (Channel)
		{
			*Channel = *PreDilateChannel;

			// Dilate the times
			Dilate(Channel, FFrameNumber(0), DilationFactor);

			SectionObject.Parameters.TimeScale.AsCustom()->MarkAsChanged();
		}
	}

	SectionObject.SetRange(NewRange);
}


bool FSubTrackEditorUtil::CanAddSubSequence(const UMovieSceneSequence* CurrentSequence, const UMovieSceneSequence& SubSequence)
{
	// Prevent adding ourselves, check the SubSequence is a compatible subsequence and ensure we have a valid movie scene.
	if ((CurrentSequence == nullptr) || (CurrentSequence == &SubSequence) || (CurrentSequence->GetMovieScene() == nullptr))
	{
		return false;
	}

	if (!CurrentSequence->IsSubSequenceCompatible(SubSequence))
	{
		return false;
	}

	// ensure that the other sequence has a valid movie scene
	UMovieScene* SequenceMovieScene = SubSequence.GetMovieScene();

	if (SequenceMovieScene == nullptr)
	{
		return false;
	}

	// make sure we are not contained in the other sequence (circular dependency)
	// @todo sequencer: this check is not sufficient (does not prevent circular dependencies of 2+ levels)
	UMovieSceneSubTrack* SequenceSubTrack = SequenceMovieScene->FindTrack<UMovieSceneSubTrack>();
	if (SequenceSubTrack && SequenceSubTrack->ContainsSequence(*CurrentSequence, true))
	{
		return false;
	}

	UMovieSceneCinematicShotTrack* SequenceCinematicTrack = SequenceMovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (SequenceCinematicTrack && SequenceCinematicTrack->ContainsSequence(*CurrentSequence, true))
	{
		return false;
	}

	return true;
}

UMovieSceneMetaData* FSubTrackEditorUtil::FindOrAddMetaData(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		return nullptr;
	}

	ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequence);
	return LevelSequence ? LevelSequence->FindOrAddMetaData<UMovieSceneMetaData>() : nullptr;
}

FText FSubTrackEditorUtil::GetMetaDataText(const UMovieSceneSequence* Sequence)
{
	const ULevelSequence* LevelSequence = Cast<const ULevelSequence>(Sequence);
	if (!LevelSequence)
	{
		return FText::GetEmpty();
	}

	const UMovieSceneMetaData* MetaData = LevelSequence->FindMetaData<const UMovieSceneMetaData>();
	if (!MetaData)
	{
		return FText::GetEmpty();
	}

	if (MetaData->IsEmpty())
	{
		return FText::GetEmpty();
	}

	return FText::Format(LOCTEXT("MetaDataContentFormat", "Author: {0}\nCreated: {1}\nNotes: {2}"),
		FText::FromString(MetaData->GetAuthor()),
		FText::AsDateTime(MetaData->GetCreated()),
		FText::FromString(MetaData->GetNotes())
	);
}

#undef LOCTEXT_NAMESPACE
