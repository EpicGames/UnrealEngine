// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/TimeWarpTrackEditor.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerEditTool.h"
#include "SequencerSectionPainter.h"
#include "SequencerUtilities.h"
#include "SequencerSettings.h"
#include "TimeSliderArgs.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "MovieSceneSequence.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Sections/MovieSceneTimeWarpSection.h"
#include "Channels/MovieSceneTimeWarpChannel.h"

#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Views/ITrackAreaHotspot.h"

#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "TimeWarpTrackEditor"

namespace UE::Sequencer
{

struct FScrubberHotspot : ITrackAreaHotspot
{
	TWeakPtr<ISequencer> WeakSequencer;

	FScrubberHotspot(TWeakPtr<ISequencer> InWeakSequencer)
		: WeakSequencer(InWeakSequencer)
	{}

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
	{
		InTrackArea.AttemptToActivateTool("Movement");
	}

	virtual TOptional<FFrameNumber> GetTime() const
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer)
		{
			return Sequencer->GetLocalTime().Time.FrameNumber;
		}
		return TOptional<FFrameNumber>();
	}

	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent)
	{
		struct FScrubLocalTime : ISequencerEditToolDragOperation
		{
			TWeakPtr<ISequencer> WeakSequencer;
			FScrubLocalTime(TWeakPtr<ISequencer> InWeakSequencer)
				: WeakSequencer(InWeakSequencer)
			{}

			void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
			{
				if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
				{
					Sequencer->OnBeginScrubbing();
				}
			}
			void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
			{
				FFrameTime ScrubTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);
				if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
				{
					USequencerSettings* Settings = Sequencer->GetSequencerSettings();
					if (Settings->GetForceWholeFrames())
					{
						ScrubTime = FFrameRate::Snap(ScrubTime, Sequencer->GetFocusedTickResolution(), Sequencer->GetFocusedDisplayRate());
					}

					ENearestKeyOption NearestKeyOption = MouseEvent.IsShiftDown()
						? ENearestKeyOption::NKO_SearchKeys | ENearestKeyOption::NKO_SearchSections | ENearestKeyOption::NKO_SearchMarkers
						: ENearestKeyOption::NKO_None;

					if (Settings->GetIsSnapEnabled() || MouseEvent.IsShiftDown())
					{
						if (Settings->GetSnapPlayTimeToKeys())
						{
							EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys);
						}
						if (Settings->GetSnapPlayTimeToSections())
						{
							EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchSections);
						}
						if (Settings->GetSnapPlayTimeToMarkers())
						{
							EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchMarkers);
						}

						FFrameNumber NearestKey = Sequencer->OnGetNearestKey(ScrubTime, NearestKeyOption);

						static float MouseTolerance = 20.f;
						if (FMath::IsNearlyEqual(VirtualTrackArea.FrameToPixel(NearestKey), LocalMousePos.X, MouseTolerance))
						{
							ScrubTime = NearestKey;
						}
					}

					// @todo: Autoscroll goes wild when scrubbing warped time.
					//        That is an intricate system that needs updating to handle warped times, but for now
					//        we just hack it off when scrubbing.
					if (Settings->GetAutoScrollEnabled())
					{
						Settings->SetAutoScrollEnabled(false);
						Sequencer->OnScrubPositionChanged(ScrubTime, true, true);
						Settings->SetAutoScrollEnabled(true);
					}
					else
					{
						Sequencer->OnScrubPositionChanged(ScrubTime, true, true);
					}
				}
			}
			void OnEndDrag( const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
			{
				if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
				{
					Sequencer->OnEndScrubbing();
				}
			}
			FCursorReply GetCursor() const override
			{
				return FCursorReply::Cursor(EMouseCursor::Default);
			}
			int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
			{
				return LayerId;
			}
		};

		return MakeShared<FScrubLocalTime>(WeakSequencer);
	}

	virtual FCursorReply GetCursor() const
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}

	virtual int32 Priority() const
	{
		return 10000;
	}
};

struct STimeWarpScrubber : public SLeafWidget
{
	SLATE_BEGIN_ARGS(STimeWarpScrubber){}
	SLATE_END_ARGS()

	static constexpr float HalfScrubberWidthPx = 7.f;
	static constexpr float ScrubberWidthPx = HalfScrubberWidthPx * 2.f;

	void Construct(const FArguments& InArgs, UMovieSceneTimeWarpSection* Section, TSharedPtr<FTimeToPixel> InTimeToPixel, TWeakPtr<STrackAreaView> InWeakTrackAreaView, TWeakPtr<ISequencer> InWeakSequencer)
	{
		WeakSection = Section;
		TimeToPixel = InTimeToPixel;
		WeakSequencer = InWeakSequencer;
		WeakTrackAreaView = InWeakTrackAreaView;

		SetVisibility(MakeAttributeSP(this, &STimeWarpScrubber::GetVisibility));
	}

	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		using namespace UE::MovieScene;

		UMovieSceneTimeWarpSection* TimeWarpSection = WeakSection.Get();
		UMovieSceneTimeWarpTrack*   TimeWarpTrack   = TimeWarpSection ? TimeWarpSection->GetTypedOuter<UMovieSceneTimeWarpTrack>() : nullptr;

		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

		if (Sequencer && TimeWarpTrack && TimeWarpTrack->IsTimeWarpActive())
		{
			FLinearColor TimeWarpColor = FStyleColors::AccentOrange.GetSpecifiedColor();
			if (IsDirectlyHovered())
			{
				FLinearColor HSV = TimeWarpColor.LinearRGBToHSV();
				HSV.B = .6f;
				HSV.G = .6f;
				TimeWarpColor = HSV.HSVToLinearRGB();
			}

			const FSlateBrush* Brush = FAppStyle::GetBrush("Sequencer.Timeline.ScrubHandle");
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(AllottedGeometry.Size - FVector2f(0.f, 1.f), FSlateLayoutTransform(FVector2f(0.f, 1.f))),
				Brush,
				ESlateDrawEffect::None,
				TimeWarpColor
			);
		}

		return LayerId;
	}

	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		TSharedPtr<STrackAreaView> TrackAreaView = WeakTrackAreaView.Pin();
		if (TrackAreaView)
		{
			TrackAreaView->GetViewModel()->SetHotspot(MakeShared<FScrubberHotspot>(WeakSequencer));
		}
	}

	void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		TSharedPtr<STrackAreaView> TrackAreaView = WeakTrackAreaView.Pin();
		if (TrackAreaView)
		{
			TrackAreaView->GetViewModel()->SetHotspot(nullptr);
		}
	}

	FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// Hack to prevent the section from being able to handle this
		return FReply::Handled();
	}

	FVector2D ComputeDesiredSize(float) const
	{
		return FVector2D(ScrubberWidthPx, 100.f);
	}

	EVisibility GetVisibility() const
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		UMovieSceneTimeWarpSection* TimeWarpSection = WeakSection.Get();
		UMovieSceneTimeWarpTrack*   TimeWarpTrack   = TimeWarpSection ? TimeWarpSection->GetTypedOuter<UMovieSceneTimeWarpTrack>() : nullptr;

		const bool bIsVisible = (Sequencer && Sequencer->GetSequencerSettings()->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::Both)
			&& TimeWarpTrack && TimeWarpTrack->IsTimeWarpActive();

		return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:
	TWeakObjectPtr<UMovieSceneTimeWarpSection> WeakSection;
	TSharedPtr<FTimeToPixel> TimeToPixel;
	TWeakPtr<STrackAreaView> WeakTrackAreaView;
	TWeakPtr<ISequencer> WeakSequencer;
};


struct FTimeWarpSection : FSequencerSection
{
	TWeakPtr<ISequencer> WeakSequencer;

	FTimeWarpSection(UMovieSceneSection& InSection, TSharedPtr<ISequencer> InSequencer)
		: FSequencerSection(InSection)
		, WeakSequencer(InSequencer)
	{
	}

	void CreateViewWidgets(const FCreateSectionViewWidgetParams& Params) override
	{
		if (UMovieSceneTimeWarpSection* TimeWarpSection = Cast<UMovieSceneTimeWarpSection>(GetSectionObject()))
		{
			auto ScrubPosition = [WeakSequencer = this->WeakSequencer, TimeToPixel = Params.SectionView->GetTimeToPixel()]
			{
				FMargin Margin;
				if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
				{
					Margin.Left = TimeToPixel->SecondsToPixel(Sequencer->GetLocalTime().AsSeconds()) - STimeWarpScrubber::HalfScrubberWidthPx;
				}
				return Margin;
			};

			// Add our widget above everything
			Params.Overlay->AddSlot(FCreateSectionViewWidgetParams::ChannelViewOrder + 10)
			.HAlign(HAlign_Left)
			.Padding(MakeAttributeLambda(ScrubPosition))
			[
				SNew(STimeWarpScrubber, TimeWarpSection, Params.SectionView->GetTimeToPixel(), Params.TrackAreaView, WeakSequencer)
			];
		}
	}
};


UE_SEQUENCER_DEFINE_CASTABLE(FTimeWarpTrackModel)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FTimeWarpTrackExtension)


bool FTimeWarpTrackModel::IsActiveTimeWarp() const
{
	UMovieSceneTimeWarpTrack* Track = Cast<UMovieSceneTimeWarpTrack>(GetTrack());
	return Track && Track->IsTimeWarpActive();
}

void FTimeWarpTrackModel::OnConstruct()
{
	FTrackModel::OnConstruct();

	TViewModelPtr<FEditorSharedViewModelData> Shared = GetSharedData()->CastThisShared<FEditorSharedViewModelData>();
	if (Shared)
	{
		FTimeWarpTrackExtension& TrackExtension = Shared->AddDynamicExtension<FTimeWarpTrackExtension>();
		TrackExtension.WeakTimeWarpModels.Add(SharedThis(this));
	}
}

const FTimeWarpTrackModel* FTimeWarpTrackExtension::GetActiveTimeWarpTrack() const
{
	for (TWeakViewModelPtr<FTimeWarpTrackModel> WeakTimeWarpTrack : WeakTimeWarpModels)
	{
		TViewModelPtr<FTimeWarpTrackModel> TimeWarpTrack = WeakTimeWarpTrack.Pin();
		if (TimeWarpTrack && TimeWarpTrack->IsActiveTimeWarp())
		{
			return TimeWarpTrack.Get();
		}
	}
	return nullptr;
}


} // namespace UE::Sequencer

TSharedPtr<UE::Sequencer::FTrackModel> FTimeWarpTrackEditor::CreateTrackModel(UMovieSceneTrack* Track)
{
	using namespace UE::Sequencer;

	if (UMovieSceneTimeWarpTrack* TimeWarpTrack = Cast<UMovieSceneTimeWarpTrack>(Track))
	{
		return MakeShared<FTimeWarpTrackModel>(TimeWarpTrack);
	}
	return nullptr;
}

void FTimeWarpTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	Operation.ApplyDefault(InKeyTime, InSequencer, OutResults);
}

FText FTimeWarpTrackEditor::GetDisplayName() const
{
	return LOCTEXT("TimeWarpTrackEditor_DisplayName", "Time Warp");
}

void FTimeWarpTrackEditor::BuildPinnedAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly() || FocusedMovieScene->FindTrack<UMovieSceneTimeWarpTrack>() != nullptr)
	{
		return;
	}


	auto HandleAddTimeWarp = [this](TSubclassOf<UMovieSceneTimeWarpGetter> InClass)
	{
		this->HandleAddTimeWarpTrack(InClass);
	};

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddTimeWarpTrack", "Time Warp"),
		LOCTEXT("AddTimeWarpTrackTooltip", "Adds a new track that manipulates the time of the current sequence."),
		FNewMenuDelegate::CreateStatic(FSequencerUtilities::PopulateTimeWarpSubMenu, TFunction<void(TSubclassOf<UMovieSceneTimeWarpGetter>)>(HandleAddTimeWarp)),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.TimeWarp")
	);
}

TSharedRef<ISequencerSection> FTimeWarpTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<UE::Sequencer::FTimeWarpSection>(SectionObject, GetSequencer());
}

bool FTimeWarpTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneTimeWarpTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

void FTimeWarpTrackEditor::HandleAddTimeWarpTrack(TSubclassOf<UMovieSceneTimeWarpGetter> ClassType)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly() || FocusedMovieScene->FindTrack<UMovieSceneTimeWarpTrack>() != nullptr)
	{
		return;
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddTimeWarpTrack_Transaction", "Add Time Warp Track"));

	FocusedMovieScene->Modify();

	UMovieSceneTimeWarpTrack*   NewTrack   = NewObject<UMovieSceneTimeWarpTrack>(FocusedMovieScene, NAME_None, RF_Transactional);
	UMovieSceneSection*         NewSection = NewTrack->CreateNewSection();
	FMovieSceneTimeWarpVariant* TimeWarp   = NewSection->GetTimeWarp();

	check(TimeWarp);

	UObject* GetterOuter = FocusedMovieScene->GetTypedOuter<UMovieSceneSequence>();
	UMovieSceneTimeWarpGetter* NewGetter = NewObject<UMovieSceneTimeWarpGetter>(GetterOuter ? GetterOuter : NewSection, ClassType.Get(), NAME_None, RF_Transactional);
	NewGetter->InitializeDefaults(NewSection);
	TimeWarp->Set(NewGetter);

	NewTrack->AddSection(*NewSection);

	FocusedMovieScene->AddGivenTrack(NewTrack);
	SequencerPtr->OnAddTrack(NewTrack, FGuid());
}

#undef LOCTEXT_NAMESPACE
