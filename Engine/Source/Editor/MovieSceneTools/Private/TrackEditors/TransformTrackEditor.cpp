// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/TransformTrackEditor.h"
#include "GameFramework/Actor.h"
#include "Framework/Commands/Commands.h"
#include "Animation/AnimSequence.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Components/SceneComponent.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/Character.h"
#include "Engine/Selection.h"
#include "LevelEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "ISectionLayoutBuilder.h"
#include "IKeyArea.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Sections/TransformPropertySection.h"
#include "SequencerUtilities.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MovieSceneToolHelpers.h"
#include "Animation/AnimData/IAnimationDataModel.h"

#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Tracks/IMovieSceneTransformOrigin.h"
#include "IMovieScenePlaybackClient.h"
#include "Animation/AnimData/AnimDataModel.h"

#include "Editor.h"
#include "LevelEditorViewport.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformConstraintUtil.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "Misc/TransactionObjectEvent.h"
#include "Engine/Selection.h"
#include "PropertyHandle.h"
#include "IDetailKeyframeHandler.h"
#include "ISequencerObjectChangeListener.h"
#include "ISequencerPropertyKeyedStatus.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "IMovieSceneRootMotionOffsetProvider.h"

#define LOCTEXT_NAMESPACE "MovieScene_TransformTrack"

void GetActorAndSceneComponentFromObject( UObject* Object, AActor*& OutActor, USceneComponent*& OutSceneComponent )
{
	OutActor = Cast<AActor>( Object );
	if ( OutActor != nullptr && OutActor->GetRootComponent() )
	{
		OutSceneComponent = OutActor->GetRootComponent();
	}
	else
	{
		// If the object wasn't an actor attempt to get it directly as a scene component and then get the actor from there.
		OutSceneComponent = Cast<USceneComponent>( Object );
		if ( OutSceneComponent != nullptr )
		{
			OutActor = Cast<AActor>( OutSceneComponent->GetOuter() );
		}
	}
}

F3DTransformTrackEditor* F3DTransformTrackEditor::CurrentEditor = nullptr;
FName F3DTransformTrackEditor::TransformPropertyName("Transform");

F3DTransformTrackEditor::F3DTransformTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FKeyframeTrackEditor<UMovieScene3DTransformTrack>( InSequencer ) 
{
	// Listen for actor/component movement
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &F3DTransformTrackEditor::OnPrePropertyChanged);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &F3DTransformTrackEditor::OnPostPropertyChanged);
	if (GEditor != nullptr)
	{
		GEditor->RegisterForUndo(this);
	}

	if (TSharedPtr<ISequencer> SequencerPtr = FMovieSceneTrackEditor::GetSequencer())
	{
	    const FProperty* LocationProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
		const FProperty* RotationProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());
		const FProperty* Scale3DProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());

		ISequencerObjectChangeListener& ObjectChangeListener = SequencerPtr->GetObjectChangeListener();
		ISequencerPropertyKeyedStatusHandler& PropertyKeyedStatusHandler = SequencerPtr->GetPropertyKeyedStatusHandler();

		auto AddTransformProperty = [this, &ObjectChangeListener, &PropertyKeyedStatusHandler](const FProperty* Property, EMovieSceneTransformChannel TransformChannel)
		{
			if (Property)
			{
				TransformProperties.Add({ Property, TransformChannel });
				ObjectChangeListener.GetOnAnimatablePropertyChanged(Property)
					.AddRaw(this, &F3DTransformTrackEditor::OnTransformPropertyChanged, TransformChannel);

				PropertyKeyedStatusHandler.GetExternalHandler(Property)
					.BindRaw(this, &F3DTransformTrackEditor::GetPropertyKeyedStatus, TransformChannel);
			}
		};

		AddTransformProperty(LocationProperty, EMovieSceneTransformChannel::Translation);
		AddTransformProperty(RotationProperty, EMovieSceneTransformChannel::Rotation);
		AddTransformProperty(Scale3DProperty, EMovieSceneTransformChannel::Scale);
	}
	F3DTransformTrackEditor::CurrentEditor = this;

}

F3DTransformTrackEditor::~F3DTransformTrackEditor()
{
	F3DTransformTrackEditor::CurrentEditor = nullptr;

	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}

	if (TSharedPtr<ISequencer> SequencerPtr = FMovieSceneTrackEditor::GetSequencer())
	{
		ISequencerObjectChangeListener& ObjectChangeListener = SequencerPtr->GetObjectChangeListener();
		ISequencerPropertyKeyedStatusHandler& PropertyKeyedStatusHandler = SequencerPtr->GetPropertyKeyedStatusHandler();
		for (const FTransformPropertyInfo& TransformProperty : TransformProperties)
		{
			ObjectChangeListener.GetOnAnimatablePropertyChanged(TransformProperty.Property).RemoveAll(this);
			PropertyKeyedStatusHandler.GetExternalHandler(TransformProperty.Property).Unbind();
		}
	}
}
//for 5.2 we will move this over to the header
static TSet<TWeakObjectPtr<UMovieScene3DTransformSection>> SectionsToClear;

void F3DTransformTrackEditor::OnRelease()
{
	ClearOutConstraintDelegates();
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (OnSceneComponentConstrainedHandle.IsValid())
	{
		FConstraintsManagerController::OnSceneComponentConstrained().Remove(OnSceneComponentConstrainedHandle);
		OnSceneComponentConstrainedHandle.Reset();
	}

	for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsPerspective() && LevelVC->GetViewMode() != VMI_Unknown)
		{
			LevelVC->ViewFOV = LevelVC->FOVAngle;
		}
	}
	F3DTransformTrackEditor::CurrentEditor = nullptr;

}

TSharedRef<ISequencerTrackEditor> F3DTransformTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new F3DTransformTrackEditor( InSequencer ) );
}


bool F3DTransformTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	// We support animatable transforms
	return Type == UMovieScene3DTransformTrack::StaticClass();
}


void F3DTransformTrackEditor::BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track )
{
	UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>( Track );

	MenuBuilder.AddSubMenu(
		NSLOCTEXT( "Sequencer", "ImportTransforms", "Import From Animation Root" ),
		NSLOCTEXT( "Sequencer", "ImportTransformsTooltip", "Import transform keys from an animation sequence's root motion." ),
		FNewMenuDelegate::CreateSP(this, &F3DTransformTrackEditor::BuildAssetPickerSubMenu, TransformTrack)
	);

	MenuBuilder.AddMenuSeparator();

	FKeyframeTrackEditor::BuildTrackContextMenu(MenuBuilder, Track);
}

void F3DTransformTrackEditor::BuildAssetPickerSubMenu(FMenuBuilder& InMenuBuilder, UMovieScene3DTransformTrack* InTransformTrack)
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}

	TWeakObjectPtr<UMovieScene3DTransformTrack> TransformTrackWeak = InTransformTrack;

	UMovieSceneSequence* Sequence = SequencerPtr->GetFocusedMovieSceneSequence();

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(
		[this, TransformTrackWeak](const FAssetData& InAssetData)
		{
			if (const TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
			{
				MovieSceneToolHelpers::ImportAnimSequenceTransforms(SequencerPtr.ToSharedRef(), InAssetData, TransformTrackWeak.Get());
			}
		});
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateLambda(
		[this, TransformTrackWeak](const TArray<FAssetData>& InSelectedAssets)
		{
			if (const TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
			{
				MovieSceneToolHelpers::ImportAnimSequenceTransformsEnterPressed(SequencerPtr.ToSharedRef(), InSelectedAssets, TransformTrackWeak.Get());
			}
		});
	AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
	AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	InMenuBuilder.AddWidget(
		SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		], 
		FText(), true, false);
}


TSharedRef<ISequencerSection> F3DTransformTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(&SectionObject);
	if (Section)
	{
		if (!Section->ConstraintChannelAdded().IsBoundToObject(this))
		{
			Section->ConstraintChannelAdded().AddRaw(this, &F3DTransformTrackEditor::HandleOnConstraintAdded);
		}
	}
	//if there are channels already we need to act like they were added
	TArray<FConstraintAndActiveChannel>& Channels = Section->GetConstraintsChannels();
	for (FConstraintAndActiveChannel& Channel : Channels)
	{
		HandleOnConstraintAdded(Section, &Channel.ActiveChannel);
	}
	return MakeShared<FTransformSection>(SectionObject, GetSequencer());
}

bool F3DTransformTrackEditor::HasTransformTrack(UObject& InObject) const
{
	FGuid Binding = GetSequencer()->FindObjectId(InObject, GetSequencer()->GetFocusedTemplateID());
	if (Binding.IsValid())
	{
		if (GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(Binding, TransformPropertyName))
		{
			return true;
		}
	}

	return false;
}


void F3DTransformTrackEditor::OnPreTransformChanged( UObject& InObject )
{
	if (!GetSequencer()->IsAllowedToChange())
	{
		return;
	}

	AActor* Actor = Cast<AActor>(&InObject);
	// If Sequencer is allowed to autokey and we are clicking on an Actor that can be autokeyed
	if(Actor && !Actor->IsEditorOnly())
	{
		AActor* ActorThatChanged = nullptr;
		USceneComponent* SceneComponentThatChanged = nullptr;
		GetActorAndSceneComponentFromObject(&InObject, ActorThatChanged, SceneComponentThatChanged);

		if( SceneComponentThatChanged )
		{
			// Cache off the existing transform so we can detect which components have changed
			// and keys only when something has changed
			FTransformData Transform( SceneComponentThatChanged );

			{
				// switch to constraint space if needed so that the optional LastTransform passed to F3DTransformTrackEditor::GetTransformKeys
				// is expressed in the right space to compare values
				const FTransform Relative(Transform.Rotation, Transform.Translation, Transform.Scale);
				FTransform ConstraintRelative(Relative);
				UE::TransformConstraintUtil::UpdateTransformBasedOnConstraint(ConstraintRelative, SceneComponentThatChanged);
				if (!ConstraintRelative.Equals(Relative, UE_SMALL_NUMBER))
				{
					Transform.Translation = ConstraintRelative.GetTranslation();
					Transform.Rotation = ConstraintRelative.Rotator();
					Transform.Scale = ConstraintRelative.GetScale3D();
				}
			}
			
			ObjectToExistingTransform.Add(&InObject, Transform);
			
			bool bObjectHasTransformTrack = HasTransformTrack(InObject);
			bool bComponentHasTransformTrack = HasTransformTrack(*SceneComponentThatChanged);

			// If there's no existing track, key the existing transform on pre-change so that the current transform before interaction is stored as the default state. 
			// If keying only happens at the end of interaction, the transform after interaction would end up incorrectly as the default state.
			if (!bObjectHasTransformTrack && !bComponentHasTransformTrack)
			{
				TOptional<FTransformData> LastTransform;

				UObject* ObjectToKey = nullptr;
				if (bComponentHasTransformTrack)
				{
					ObjectToKey = SceneComponentThatChanged;
				}
				// If the root component broadcasts a change, we want to key the actor instead
				else if (ActorThatChanged && ActorThatChanged->GetRootComponent() == &InObject)
				{
					ObjectToKey = ActorThatChanged;
				}
				else
				{
					ObjectToKey = &InObject;
				}

				AddTransformKeys(ObjectToKey, LastTransform, Transform, EMovieSceneTransformChannel::All, ESequencerKeyMode::AutoKey);
			}
		}
	}
}


void F3DTransformTrackEditor::OnTransformChanged( UObject& InObject )
{
	if (!GetSequencer()->IsAllowedToChange())
	{
		return;
	}

	AActor* Actor = nullptr;
	USceneComponent* SceneComponentThatChanged = nullptr;
	GetActorAndSceneComponentFromObject(&InObject, Actor, SceneComponentThatChanged);

	// If the Actor that just finished transforming doesn't have autokey disabled
	if( SceneComponentThatChanged != nullptr && (Actor && !Actor->IsEditorOnly()))
	{
		// Find an existing transform if possible.  If one exists we will compare against the new one to decide what components of the transform need keys
		TOptional<FTransformData> ExistingTransform;
		if (const FTransformData* Found = ObjectToExistingTransform.Find( &InObject ))
		{
			ExistingTransform = *Found;
		}

		// Remove it from the list of cached transforms. 
		// @todo sequencer livecapture: This can be made much for efficient by not removing cached state during live capture situation
		ObjectToExistingTransform.Remove( &InObject );

		// Build new transform data
		FTransformData NewTransformData( SceneComponentThatChanged );

		bool bComponentHasTransformTrack = HasTransformTrack(*SceneComponentThatChanged);

		UObject* ObjectToKey = nullptr;
		if (bComponentHasTransformTrack)
		{
			ObjectToKey = SceneComponentThatChanged;
		}
		// If the root component broadcasts a change, we want to key the actor instead
		else if (Actor && Actor->GetRootComponent() == &InObject)
		{
			ObjectToKey = Actor;
		}
		else
		{
			ObjectToKey = &InObject;
		}

		AddTransformKeys(ObjectToKey, ExistingTransform, NewTransformData, EMovieSceneTransformChannel::All, ESequencerKeyMode::AutoKey);
	}
}

void F3DTransformTrackEditor::OnPrePropertyChanged(UObject* InObject, const FEditPropertyChain& InPropertyChain)
{
	FProperty* PropertyAboutToChange = InPropertyChain.GetActiveMemberNode() ? InPropertyChain.GetActiveMemberNode()->GetValue() : nullptr;
	const FName MemberPropertyName = PropertyAboutToChange != nullptr ? PropertyAboutToChange->GetFName() : NAME_None;
	const bool bTransformationToChange =
		(MemberPropertyName == USceneComponent::GetRelativeLocationPropertyName() ||
		 MemberPropertyName == USceneComponent::GetRelativeRotationPropertyName() ||
		 MemberPropertyName == USceneComponent::GetRelativeScale3DPropertyName());

	if (InObject && bTransformationToChange)
	{
		UE::MovieScene::FScopedSignedObjectModifyDefer ForceFlush(true);
		OnPreTransformChanged(*InObject);
	}
}

void F3DTransformTrackEditor::OnPostPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName MemberPropertyName = InPropertyChangedEvent.MemberProperty != nullptr ? InPropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const bool bTransformationChanged =
		(MemberPropertyName == USceneComponent::GetRelativeLocationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeRotationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeScale3DPropertyName());

	if (InObject && bTransformationChanged)
	{
		UE::MovieScene::FScopedSignedObjectModifyDefer ForceFlush(true);
		OnTransformChanged(*InObject);
	}
}

void F3DTransformTrackEditor::OnPreSaveWorld(UWorld* World)
{
	LockedCameraBindings.Reset();

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	// Get the camera binding GUIDs.
	TArray<FGuid> CameraBindingIDs;
	SequencerPtr->GetCameraObjectBindings(CameraBindingIDs);

	// Match the camera binding GUIDs with what actor they are bound to.
	TArray<AActor*> CameraBindingActors;
	for (const FGuid& CameraBindingID : CameraBindingIDs)
	{
		AActor* BoundActor = nullptr;
		for (auto Object : SequencerPtr->FindObjectsInCurrentSequence(CameraBindingID))
		{
			AActor* Actor = Cast<AActor>(Object.Get());
			if (Actor != nullptr)
			{
				BoundActor = Actor;
				break;
			}
		}
		CameraBindingActors.Add(BoundActor);
	}

	// Look at all the editor viewports and see if they're locked to one of our
	// bound camera actors. If they are, associate them with the binding GUID.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->GetViewMode() != VMI_Unknown && LevelVC->AllowsCinematicControl())
		{
			AActor* ActorLock = LevelVC->GetActiveActorLock().Get();
			if (!ActorLock)
			{
				continue;
			}

			int32 Index = CameraBindingActors.Find(ActorLock);
			if (Index != INDEX_NONE)
			{
				LockedCameraBindings.Add(LevelVC, CameraBindingIDs[Index]);
			}
		}
	}
}

void F3DTransformTrackEditor::OnPostSaveWorld(UWorld* World)
{
	for (const TPair<FLevelEditorViewportClient*, FGuid> Pair : LockedCameraBindings)
	{
		LockCameraBinding(true, Pair.Value, Pair.Key, false);
	}

	LockedCameraBindings.Reset();
}

bool F3DTransformTrackEditor::CanAddTransformKeysForSelectedObjects() const
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s). 
	// If the viewport is moving, disregard setting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return false;
		}
	}
	TArray<UObject*> SelectedObjects;
	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(*It);
		if (SceneComponent)
		{
			return true;
		}
	}

	if (SelectedObjects.Num() == 0)
	{
		USelection* CurrentSelection = GEditor->GetSelectedActors();
		CurrentSelection->GetSelectedObjects(AActor::StaticClass(), SelectedObjects);
	}
	return SelectedObjects.Num() > 0;
}

void F3DTransformTrackEditor::OnAddTransformKeysForSelectedObjects( EMovieSceneTransformChannel Channel )
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s). 
	// If the viewport is moving, disregard setting keyframes.
	for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return;
		}
	}

	TArray<UObject*> SelectedObjects;
	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(*It);
		if (SceneComponent)
		{
			SelectedObjects.Add(SceneComponent);
		}
	}
	
	if (SelectedObjects.Num() == 0)
	{
		USelection* CurrentSelection = GEditor->GetSelectedActors();
		CurrentSelection->GetSelectedObjects( AActor::StaticClass(), SelectedObjects );
	}

	for (TArray<UObject*>::TIterator It(SelectedObjects); It; ++It)
	{
		AddTransformKeysForObject(*It, Channel, ESequencerKeyMode::ManualKeyForced);
	}
}

void F3DTransformTrackEditor::OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel)
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s).
	// If the viewport is moving, disregard deleting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}

	FFrameNumber CurrentFrame = SequencerPtr->GetLocalTime().Time.FrameNumber;
	UMovieScene* MovieScene = GetFocusedMovieScene();
	if (!MovieScene)
	{
		return;
	}

	TArray<UObject*> SelectedObjects;
	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(*It);
		if (SceneComponent)
		{
			SelectedObjects.Add(SceneComponent);
		}
	}
	if (SelectedObjects.Num() == 0)
	{
		USelection* CurrentSelection = GEditor->GetSelectedActors();
		CurrentSelection->GetSelectedObjects(AActor::StaticClass(), SelectedObjects);
	}

	// Channel index order in UMovieScene3DTransformSection: Translation X=0, Y=1, Z=2, Rotation X=3, Y=4, Z=5, Scale X=6, Y=7, Z=8
	const TTuple<EMovieSceneTransformChannel, int32> ChannelMap[] =
	{
		MakeTuple(EMovieSceneTransformChannel::TranslationX, 0),
		MakeTuple(EMovieSceneTransformChannel::TranslationY, 1),
		MakeTuple(EMovieSceneTransformChannel::TranslationZ, 2),
		MakeTuple(EMovieSceneTransformChannel::RotationX,    3),
		MakeTuple(EMovieSceneTransformChannel::RotationY,    4),
		MakeTuple(EMovieSceneTransformChannel::RotationZ,    5),
		MakeTuple(EMovieSceneTransformChannel::ScaleX,       6),
		MakeTuple(EMovieSceneTransformChannel::ScaleY,       7),
		MakeTuple(EMovieSceneTransformChannel::ScaleZ,       8),
	};

	TRange<FFrameNumber> FrameRange(
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame),
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame));

	for (UObject* SelectedObject : SelectedObjects)
	{
		FGuid ObjectGuid = SequencerPtr->GetHandleToObject(SelectedObject, false);
		if (!ObjectGuid.IsValid())
		{
			if (USceneComponent* SceneComp = Cast<USceneComponent>(SelectedObject))
			{
				ObjectGuid = SequencerPtr->GetHandleToObject(SceneComp->GetOwner(), false);
			}
		}
		if (!ObjectGuid.IsValid())
		{
			continue;
		}

		UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectGuid, TransformPropertyName);
		if (!TransformTrack)
		{
			continue;
		}

		for (UMovieSceneSection* Section : TransformTrack->GetAllSections())
		{
			UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
			if (!TransformSection)
			{
				continue;
			}

			EMovieSceneTransformChannel SectionMask = TransformSection->GetMask().GetChannels();
			EMovieSceneTransformChannel ChannelsToDelete = Channel & SectionMask;

			TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

			bool bModified = false;
			for (const TTuple<EMovieSceneTransformChannel, int32>& ChannelPair : ChannelMap)
			{
				if (!EnumHasAnyFlags(ChannelsToDelete, ChannelPair.Key))
				{
					continue;
				}
				if (ChannelPair.Value >= Channels.Num())
				{
					continue;
				}

				FMovieSceneDoubleChannel* DoubleChannel = Channels[ChannelPair.Value];
				TArray<FFrameNumber> KeyTimes;
				TArray<FKeyHandle> KeyHandles;
				DoubleChannel->GetData().GetKeys(FrameRange, &KeyTimes, &KeyHandles);
				if (KeyHandles.Num() > 0)
				{
					if (!bModified)
					{
						TransformSection->Modify();
						bModified = true;
					}
					DoubleChannel->DeleteKeys(MakeArrayView(KeyHandles));
				}
			}
		}
	}
}

void F3DTransformTrackEditor::BuildObjectBindingColumnWidgets(TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding, const UE::Sequencer::FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	using namespace UE::Sequencer;

	bool bAddCameraLock = false;
	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		bAddCameraLock = true;
	}
	else if (InColumnName == FCommonOutlinerNames::KeyFrame)
	{
		// Add the camera lock button to the keyframe column if Nav is disabled
		bAddCameraLock = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false;
	}
	else if (InColumnName == FCommonOutlinerNames::Edit)
	{
		// Add the camera lock button to the edit column if both Nav and KeyFrame are disabled
		bAddCameraLock = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::Nav) == false &&
			InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame) == false;
	}

	if (bAddCameraLock)
	{
		const bool bEditColumn = InColumnName == FCommonOutlinerNames::Edit;
		FGuid ObjectGuid = ObjectBinding->GetObjectGuid();
		GetEditBox()->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(bEditColumn ? HAlign_Left : HAlign_Center)
		.Padding(bEditColumn ? FMargin(4.f, 0.f) : FMargin(0.f))
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "Sequencer.Outliner.ToggleButton")
			.Type(ESlateCheckBoxType::ToggleButton)
			.IsFocusable(false)
			.Visibility(this, &F3DTransformTrackEditor::IsCameraVisible, ObjectGuid)
			.IsChecked(this, &F3DTransformTrackEditor::IsCameraLocked, ObjectGuid)
			.OnCheckStateChanged(this, &F3DTransformTrackEditor::OnLockCameraClicked, ObjectGuid)
			.ToolTipText(this, &F3DTransformTrackEditor::GetLockCameraToolTip, ObjectGuid)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Sequencer.Outliner.CameraLock"))
			]
		];
	}
};
void F3DTransformTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass != nullptr && (ObjectClass->IsChildOf(AActor::StaticClass()) || ObjectClass->IsChildOf(USceneComponent::StaticClass())))
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "AddTransform", "Transform"),
			NSLOCTEXT("Sequencer", "AddTransformTooltip", "Adds a transform track."),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateSP( this, &F3DTransformTrackEditor::AddTransformKeysForHandle, ObjectBindings, EMovieSceneTransformChannel::All, ESequencerKeyMode::ManualKey )
			)
		);
	}
}

EVisibility F3DTransformTrackEditor::IsCameraVisible(FGuid ObjectGuid) const
{
	for (auto Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectGuid))
	{
		AActor* Actor = Cast<AActor>(Object.Get());
		
		if (Actor != nullptr)
		{
			UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(Actor);
			if (CameraComponent)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Hidden;
}

ECheckBoxState F3DTransformTrackEditor::IsCameraLocked(FGuid ObjectGuid) const
{
	return IsCameraBindingLocked(ObjectGuid) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool F3DTransformTrackEditor::IsCameraBindingLocked(FGuid ObjectGuid) const
{
	TWeakObjectPtr<AActor> CameraActor;

	for (auto Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectGuid))
	{
		AActor* Actor = Cast<AActor>(Object.Get());
		
		if (Actor != nullptr)
		{
			CameraActor = Actor;
			break;
		}
	}

	if (CameraActor.IsValid())
	{
		// First, check the active viewport
		FViewport* ActiveViewport = GEditor->GetActiveViewport();

		for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{
			if (LevelVC && LevelVC->GetViewMode() != VMI_Unknown)
			{
				if (LevelVC->Viewport == ActiveViewport)
				{
					return (CameraActor.IsValid() && LevelVC->IsActorLocked(CameraActor.Get()));
				}
			}
		}

		// Otherwise check all other viewports
		for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{
			if (LevelVC && LevelVC->GetViewMode() != VMI_Unknown && CameraActor.IsValid() && LevelVC->IsActorLocked(CameraActor.Get()))
			{
				return true;
			}
		}
	}

	return false;
}

void F3DTransformTrackEditor::OnLockCameraClicked(ECheckBoxState CheckBoxState, FGuid ObjectGuid)
{
	LockCameraBinding((CheckBoxState == ECheckBoxState::Checked), ObjectGuid);
}

void F3DTransformTrackEditor::LockCameraBinding(bool bLock, FGuid ObjectGuid, FLevelEditorViewportClient* ViewportClient, bool bRemoveCinematicLock)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	// Find the actor bound to the given object binding ID.
	AActor* CameraActor = nullptr;
	for (auto Object : SequencerPtr->FindObjectsInCurrentSequence(ObjectGuid))
	{
		AActor* Actor = Cast<AActor>(Object.Get());

		if (Actor != nullptr)
		{
			CameraActor = Actor;
			break;
		}
	}

	// Bail out if we didn't find the actor to lock the viewport to.
	if (!CameraActor)
	{
		return;
	}

	// Find the active viewport if no viewport was provided. If no viewport is active, or if the active viewport doesn't
	// match our requirements, use the first one we find that fits.
	if (ViewportClient == nullptr)
	{
		FViewport* ActiveViewport = GEditor->GetActiveViewport();

		for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{		
			if (LevelVC && 
					LevelVC->GetViewMode() != VMI_Unknown && 
					(LevelVC->Viewport == ActiveViewport || ActiveViewport == nullptr))
			{
				ViewportClient = LevelVC;
				break;
			}
		}
	}

	// Bail out if we didn't find any acceptable viewport.
	if (!ViewportClient)
	{
		return;
	}

	if (bLock)
	{
		// Lock the given/active/found viewport to the camera.
		UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(CameraActor);
		if (CameraComponent && CameraComponent->ProjectionMode == ECameraProjectionMode::Type::Perspective)
		{
			if (ViewportClient->GetViewportType() != LVT_Perspective)
			{
				ViewportClient->SetViewportType(LVT_Perspective);
			}
		}

		if (bRemoveCinematicLock)
		{
			GetSequencer()->SetPerspectiveViewportCameraCutEnabled(false);
			ViewportClient->SetCinematicActorLock(nullptr);
		}

		ViewportClient->SetActorLock(CameraActor);
		ViewportClient->bLockedCameraView = true;
		ViewportClient->UpdateViewForLockedActor();
		ViewportClient->Invalidate();
	}
	else
	{
		// Clear the lock to this camera on the given/active/found viewport.
		if (ViewportClient->IsActorLocked(CameraActor))
		{
			if (bRemoveCinematicLock)
			{
				ViewportClient->SetCinematicActorLock(nullptr);
			}

			ViewportClient->SetActorLock(nullptr);
			ViewportClient->bLockedCameraView = false;
			ViewportClient->ViewFOV = ViewportClient->FOVAngle;
			ViewportClient->RemoveCameraRoll();
			ViewportClient->UpdateViewForLockedActor();
			ViewportClient->Invalidate();
		}
	}
}


FText F3DTransformTrackEditor::GetLockCameraToolTip(FGuid ObjectGuid) const
{
	TWeakObjectPtr<AActor> CameraActor;

	for (auto Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectGuid))
	{
		AActor* Actor = Cast<AActor>(Object.Get());

		if (Actor != nullptr)
		{
			CameraActor = Actor;
			break;
		}
	}

	if (CameraActor.IsValid())
	{
		return IsCameraLocked(ObjectGuid) == ECheckBoxState::Checked ?
			FText::Format(LOCTEXT("UnlockCamera", "Unlock {0} from Viewport"), FText::FromString(CameraActor.Get()->GetActorLabel())) :
			FText::Format(LOCTEXT("LockCamera", "Lock {0} to Selected Viewport"), FText::FromString(CameraActor.Get()->GetActorLabel()));
	}
	return FText();
}

double UnwindChannel(const double& OldValue, double NewValue)
{
	while( NewValue - OldValue > 180.0f )
	{
		NewValue -= 360.0f;
	}
	while( NewValue - OldValue < -180.0f )
	{
		NewValue += 360.0f;
	}
	return NewValue;
}
FRotator UnwindRotator(const FRotator& InOld, const FRotator& InNew)
{
	FRotator Result;
	Result.Pitch = UnwindChannel(InOld.Pitch, InNew.Pitch);
	Result.Yaw   = UnwindChannel(InOld.Yaw, InNew.Yaw);
	Result.Roll  = UnwindChannel(InOld.Roll, InNew.Roll);
	return Result;
}


void F3DTransformTrackEditor::GetTransformKeys( const TOptional<FTransformData>& LastTransform, const FTransformData& CurrentTransform, EMovieSceneTransformChannel ChannelsToKey, UObject* Object, UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys)
{
	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
	EMovieSceneTransformChannel TransformMask = TransformSection->GetMask().GetChannels();

	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	// If key all is enabled, for a key on all the channels
	bool bLastVectorIsValid = LastTransform.IsSet();
	if (SequencerPtr->GetKeyGroupMode() == EKeyGroupMode::KeyAll)
	{
		bLastVectorIsValid = false;
		ChannelsToKey = EMovieSceneTransformChannel::All;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FTransformData RecomposedTransform = RecomposeTransform(CurrentTransform, Object, Section);

	// Get the channel indices for our curve channels.
	// We will get invalid handles for any channels that are anything else than FMovieSceneDoubleChannel (such as when
	// a channel has been overriden by a procedural channel). This means that the value setters below will get null
	// channels and will simply ignore those non-double channels. This is what we want, since we assume (perhaps
	// incorrectly) that overriden channels are most probably not keyable.
	FMovieSceneChannelProxy& SectionChannelProxy = Section->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> ChannelHandles[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
	};

	USceneComponent* SceneComponent = MovieSceneHelpers::SceneComponentFromRuntimeObject(Object);
	USceneComponent* AttachParent = SceneComponent ? SceneComponent->GetAttachParent() : nullptr;
	bool bHasNeutralRotationSpace = IsValid(AttachParent) || (GetTransformOrigin().GetRotation() == FQuat::Identity && GetRootMotionOffset(Object).Equals(FTransform::Identity));

	// Set translation keys/defaults
	{
		bool bKeyX = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationX);
		bool bKeyY = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationY);
		bool bKeyZ = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::TranslationZ);

		if (bLastVectorIsValid && bHasNeutralRotationSpace)
		{
			bKeyX &= !FMath::IsNearlyEqual(LastTransform->Translation.X, CurrentTransform.Translation.X);
			bKeyY &= !FMath::IsNearlyEqual(LastTransform->Translation.Y, CurrentTransform.Translation.Y);
			bKeyZ &= !FMath::IsNearlyEqual(LastTransform->Translation.Z, CurrentTransform.Translation.Z);
		}

		if (GetSequencer()->GetKeyGroupMode() == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ))
		{
			bKeyX = bKeyY = bKeyZ = true;
		}

		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationX))
		{
			bKeyX = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationY))
		{
			bKeyY = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationZ))
		{
			bKeyZ = false;
		}

		FVector KeyVector = RecomposedTransform.Translation;

		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[0].GetChannelIndex(), (double)KeyVector.X, bKeyX));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[1].GetChannelIndex(), (double)KeyVector.Y, bKeyY));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[2].GetChannelIndex(), (double)KeyVector.Z, bKeyZ));
	}

	// Set rotation keys/defaults
	{
		bool bKeyX = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationX);
		bool bKeyY = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationY);
		bool bKeyZ = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::RotationZ);

		FRotator KeyRotator = CurrentTransform.Rotation;
		if (bLastVectorIsValid)
		{
			KeyRotator = UnwindRotator(LastTransform->Rotation, CurrentTransform.Rotation);

			if (bHasNeutralRotationSpace)
			{
				bKeyX &= !FMath::IsNearlyEqual(LastTransform->Rotation.Roll,  KeyRotator.Roll);
				bKeyY &= !FMath::IsNearlyEqual(LastTransform->Rotation.Pitch, KeyRotator.Pitch);
				bKeyZ &= !FMath::IsNearlyEqual(LastTransform->Rotation.Yaw,   KeyRotator.Yaw);
			}
		}

		if (GetSequencer()->GetKeyGroupMode() == EKeyGroupMode::KeyGroup && ( bKeyX || bKeyY || bKeyZ) )
		{
			bKeyX = bKeyY = bKeyZ = true;
		}

		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationX))
		{
			bKeyX = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationY))
		{
			bKeyY = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationZ))
		{
			bKeyZ = false;
		}

		// Do we need to unwind re-composed rotations?
		KeyRotator = (Section->GetBlendType() != EMovieSceneBlendType::Additive && Section->GetBlendType() != EMovieSceneBlendType::AdditiveFromBase) ?
			UnwindRotator(CurrentTransform.Rotation, RecomposedTransform.Rotation) : RecomposedTransform.Rotation;
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[3].GetChannelIndex(), (double)KeyRotator.Roll, bKeyX));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[4].GetChannelIndex(), (double)KeyRotator.Pitch, bKeyY));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[5].GetChannelIndex(), (double)KeyRotator.Yaw, bKeyZ));
	}

	// Set scale keys/defaults
	{
		bool bKeyX = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::ScaleX);
		bool bKeyY = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::ScaleY);
		bool bKeyZ = EnumHasAnyFlags(ChannelsToKey, EMovieSceneTransformChannel::ScaleZ);

		if (bLastVectorIsValid)
		{
			bKeyX &= !FMath::IsNearlyEqual(LastTransform->Scale.X, CurrentTransform.Scale.X);
			bKeyY &= !FMath::IsNearlyEqual(LastTransform->Scale.Y, CurrentTransform.Scale.Y);
			bKeyZ &= !FMath::IsNearlyEqual(LastTransform->Scale.Z, CurrentTransform.Scale.Z);
		}

		if (GetSequencer()->GetKeyGroupMode() == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ))
		{
			bKeyX = bKeyY = bKeyZ = true;
		}

		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleX))
		{
			bKeyX = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleY))
		{
			bKeyY = false;
		}
		if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleZ))
		{
			bKeyZ = false;
		}

		FVector KeyVector = RecomposedTransform.Scale;
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[6].GetChannelIndex(), (double)KeyVector.X, bKeyX));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[7].GetChannelIndex(), (double)KeyVector.Y, bKeyY));
		OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneDoubleChannel>(ChannelHandles[8].GetChannelIndex(), (double)KeyVector.Z, bKeyZ));
	}
}

FTransform F3DTransformTrackEditor::GetTransformOrigin() const
{
	return MovieSceneToolHelpers::GetTransformOriginForFocusedSequence(GetSequencer());
}

FTransform F3DTransformTrackEditor::GetRootMotionOffset(UObject* AnimatedObject) const
{
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();
	const UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if (!EntityLinker)
	{
		return FTransform::Identity;
	}

	// Each provider returns Identity for objects it doesn't manage, so the
	// first non-identity result is the one that owns this object.
	TArray<IMovieSceneRootMotionOffsetProvider*> Providers =
		IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneRootMotionOffsetProvider>(
			IMovieSceneRootMotionOffsetProvider::GetModularFeatureName());
	for (IMovieSceneRootMotionOffsetProvider* Provider : Providers)
	{
		FTransform Offset = Provider->GetRootMotionOffset(EntityLinker, AnimatedObject);
		if (!Offset.Equals(FTransform::Identity))
		{
			return Offset;
		}
	}

	return FTransform::Identity;
}

void F3DTransformTrackEditor::UpdateRootMotionOffset(UObject* AnimatedObject)
{
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();
	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if (!EntityLinker)
	{
		return;
	}

	TArray<IMovieSceneRootMotionOffsetProvider*> Providers =
		IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneRootMotionOffsetProvider>(
			IMovieSceneRootMotionOffsetProvider::GetModularFeatureName());
	for (IMovieSceneRootMotionOffsetProvider* Provider : Providers)
	{
		Provider->UpdateRootMotionOffset(EntityLinker, AnimatedObject);
	}
}

void F3DTransformTrackEditor::AddTransformKeysForHandle(TArray<FGuid> ObjectHandles, EMovieSceneTransformChannel ChannelToKey, ESequencerKeyMode KeyMode)
{
	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddTransformTrack", "Add Transform Track"));
	
	for (FGuid ObjectHandle : ObjectHandles)
	{
		for (TWeakObjectPtr<UObject> Object : GetSequencer()->FindObjectsInCurrentSequence(ObjectHandle))
		{
			AddTransformKeysForObject(Object.Get(), ChannelToKey, KeyMode);
		}
	}
}


void F3DTransformTrackEditor::AddTransformKeysForObject( UObject* Object, EMovieSceneTransformChannel ChannelToKey, ESequencerKeyMode KeyMode )
{
	USceneComponent* SceneComponent = MovieSceneHelpers::SceneComponentFromRuntimeObject(Object);
	if ( SceneComponent != nullptr )
	{
		FTransformData CurrentTransform( SceneComponent );
		AddTransformKeys( Object, TOptional<FTransformData>(), CurrentTransform, ChannelToKey, KeyMode );
	}
}

void F3DTransformTrackEditor::AddTransformKeys( UObject* ObjectToKey, const TOptional<FTransformData>& LastTransform, const FTransformData& CurrentTransform, EMovieSceneTransformChannel ChannelsToKey, ESequencerKeyMode KeyMode )
{
	if (!GetSequencer()->IsAllowedToChange())
	{
		return;
	}

	auto InitializeNewTrack = [](UMovieScene3DTransformTrack* NewTrack)
	{
		NewTrack->SetPropertyNameAndPath(TransformPropertyName, TransformPropertyName.ToString());
	};
	auto GenerateKeys = [=, this](UMovieSceneSection* Section, FGeneratedTrackKeys& GeneratedKeys)
	{
		UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(Section);

		// Ensure that the Transform Channel is masked in
		EMovieSceneTransformChannel ExistingChannels = TransformSection->GetMask().GetChannels();
		if (!EnumHasAnyFlags(ExistingChannels, ChannelsToKey))
		{
			TransformSection->Modify();
			TransformSection->SetMask(TransformSection->GetMask().GetChannels() | ChannelsToKey);
		}

		this->GetTransformKeys(LastTransform, CurrentTransform, ChannelsToKey, ObjectToKey, Section, GeneratedKeys);
	};
	auto OnKeyProperty = [=, this](FFrameNumber Time) -> FKeyPropertyResult
	{
		FKeyPropertyResult KeyPropertyResult = this->AddKeysToObjects(MakeArrayView(&ObjectToKey, 1), Time,  KeyMode, UMovieScene3DTransformTrack::StaticClass(), TransformPropertyName, InitializeNewTrack, GenerateKeys);
		if (KeyPropertyResult.SectionsKeyed.Num() > 0)
		{
			for (TWeakObjectPtr<UMovieSceneSection>& WeakSection : KeyPropertyResult.SectionsKeyed)
			{
				if (UMovieScene3DTransformSection* Section = Cast< UMovieScene3DTransformSection>(WeakSection.Get()))
				{
					FMovieSceneConstraintChannelHelper::CompensateIfNeeded(GetSequencer(), Section, Time, true /*bCompPreviousTick*/);
				}
			}
		}

		// Update root motion if necessary so it doesn't double-compensate in the case we don't force update the sequencer frame
		UpdateRootMotionOffset(ObjectToKey);

		return KeyPropertyResult;
	};

	AnimatablePropertyChanged( FOnKeyProperty::CreateLambda(OnKeyProperty) );
}

FTransformData F3DTransformTrackEditor::RecomposeTransform(const FTransformData& InTransformData, UObject* AnimatedObject, UMovieSceneSection* Section)
{
	using namespace UE::MovieScene;

	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = GetSequencer()->GetEvaluationTemplate();

	UMovieSceneEntitySystemLinker* EntityLinker = EvaluationTemplate.GetEntitySystemLinker();
	if (!EntityLinker)
	{
		return InTransformData;
	}

	USceneComponent* SceneComponent = MovieSceneHelpers::SceneComponentFromRuntimeObject(AnimatedObject);

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &EntityLinker->EntityManager);

	// We want the transform value contributed to by the given transform section.
	//
	// In most cases, the section only has one ECS entity for it specifiy all 9 channel curves and resulting values, but if
	// some of those channels have been overriden (e.g. with some noise channel), then there will be one extra ECS entity
	// for each overriden channel, and the "main" entity will be missing that channel. We therefore gather up all entities
	// related to the given section and recompose their contribution, and then join them all together by matching which one
	// handles which channel.
	//
	TArray<FMovieSceneEntityID> ImportedEntityIDs;
	EvaluationTemplate.FindEntitiesFromOwner(Section, GetSequencer()->GetFocusedTemplateID(), ImportedEntityIDs);

	FTransform CurrentTransform = FTransform(InTransformData.Rotation, InTransformData.Translation, InTransformData.Scale);

	// Subtract any root motion first (both from the 'CurrentTransform' and the InTransformData)
	if (!SceneComponent->GetAttachParent() && Section->GetBlendType() == EMovieSceneBlendType::Absolute)
	{
		CurrentTransform = GetRootMotionOffset(AnimatedObject).Inverse() * CurrentTransform;
	}

	if (ImportedEntityIDs.Num())
	{
		UMovieScenePropertyInstantiatorSystem* System = EntityLinker->FindSystem<UMovieScenePropertyInstantiatorSystem>();
		if (System)
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
			FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

			TArray<FMovieSceneEntityID> EntityIDs;
			{
				// In order to check for the result channels later, we need to look up the children entities that are
				// bound to the given animated object. Imported entities generally don't have the result channels.
				FEntityTaskBuilder()
				.ReadEntityIDs()
				.Read(BuiltInComponents->ParentEntity)
				.Read(BuiltInComponents->BoundObject)
				.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
				.Iterate_PerEntity(
					&EntityLinker->EntityManager, 
					[SceneComponent, ImportedEntityIDs, &EntityIDs](FMovieSceneEntityID EntityID, FMovieSceneEntityID ParentEntityID, UObject* BoundObject)
					{
						if (SceneComponent == BoundObject && ImportedEntityIDs.Contains(ParentEntityID))
						{
							EntityIDs.Add(EntityID);
						}
					});
			}

			FDecompositionQuery Query;
			Query.Entities = MakeArrayView(EntityIDs);
			Query.Object   = SceneComponent;
			Query.bConvertFromSourceEntityIDs = false;  // We already pass the children entity IDs

			FIntermediate3DTransform CurrentValue(CurrentTransform.GetTranslation(), CurrentTransform.GetRotation().Rotator(), CurrentTransform.GetScale3D());

			TRecompositionResult<FIntermediate3DTransform> TransformData = System->RecomposeBlendOperational(TrackComponents->ComponentTransform, Query, CurrentValue);

			double CurrentTransformChannels[9] = { 0, 0, 0, 0, 0, 0, 0, 0, 0 };
			CurrentTransformChannels[0] = CurrentValue.GetTranslation()[0];
			CurrentTransformChannels[1] = CurrentValue.GetTranslation()[1];
			CurrentTransformChannels[2] = CurrentValue.GetTranslation()[2];
			CurrentTransformChannels[3] = CurrentValue.GetRotation().Roll;
			CurrentTransformChannels[4] = CurrentValue.GetRotation().Pitch;
			CurrentTransformChannels[5] = CurrentValue.GetRotation().Yaw;
			CurrentTransformChannels[6] = CurrentValue.GetScale()[0];
			CurrentTransformChannels[7] = CurrentValue.GetScale()[1];
			CurrentTransformChannels[8] = CurrentValue.GetScale()[2];

			EMovieSceneTransformChannel ChannelsObtained(EMovieSceneTransformChannel::None);
			check(EntityIDs.Num() == TransformData.Values.Num());
			for (int32 EntityIndex = 0; EntityIndex < TransformData.Values.Num(); ++EntityIndex)
			{
				// For each entity, find which channel they contribute to by checking the result channel.
				// We don't (yet) handle the case where two entities contribute to the same channel -- in theory the entities
				// should be mutually exclusive in that regard, at least as far as overriden channels are concerned.
				FMovieSceneEntityID EntityID = EntityIDs[EntityIndex];
				FIntermediate3DTransform EntityTransformData = TransformData.Values[EntityIndex];
				FComponentMask EntityType = EntityLinker->EntityManager.GetEntityType(EntityID);
				for (int32 CompositeIndex = 0; CompositeIndex < 9; ++CompositeIndex)
				{
					EMovieSceneTransformChannel ChannelMask = (EMovieSceneTransformChannel)(1 << CompositeIndex);
					if (!EnumHasAnyFlags(ChannelsObtained, ChannelMask) && EntityType.Contains(BuiltInComponents->DoubleResult[CompositeIndex]))
					{
						EnumAddFlags(ChannelsObtained, (EMovieSceneTransformChannel)(1 << CompositeIndex));
						CurrentTransformChannels[CompositeIndex] = EntityTransformData[CompositeIndex];
					}
				}
			}

			CurrentTransform = FTransform(
				FRotator(CurrentTransformChannels[4], CurrentTransformChannels[5], CurrentTransformChannels[3]), // pitch yaw roll
				FVector(CurrentTransformChannels[0], CurrentTransformChannels[1], CurrentTransformChannels[2]),
				FVector(CurrentTransformChannels[6], CurrentTransformChannels[7], CurrentTransformChannels[8]));			
		}
	}

	// Account for the transform origin only if this is not parented because the transform origin is already being applied to the parent.
	if (!SceneComponent->GetAttachParent() && Section->GetBlendType() == EMovieSceneBlendType::Absolute)
	{
		CurrentTransform *= GetTransformOrigin().Inverse();
	}

	UE::TransformConstraintUtil::UpdateTransformBasedOnConstraint(CurrentTransform, SceneComponent);
	
	return FTransformData(CurrentTransform.GetLocation(), CurrentTransform.GetRotation().Rotator(), CurrentTransform.GetScale3D());
}

FText F3DTransformTrackEditor::GetDisplayName() const
{
	return LOCTEXT("3DTransformTrackEditor_DisplayName", "3D Transform");
}

void F3DTransformTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::Sequencer;

	auto Iterator = [this, InKeyTime, &Operation, &InSequencer, OutResults](UMovieSceneTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> Operations)
	{
		FGuid ObjectBinding = Track->FindObjectBindingGuid();
		if (ObjectBinding.IsValid())
		{
			for (TWeakObjectPtr<> WeakObject : InSequencer.FindBoundObjects(ObjectBinding, InSequencer.GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					this->ProcessKeyOperation(Object, Operations, InSequencer, InKeyTime, OutResults);
					return;
				}
			}
		}

		// Default behavior
		FKeyOperation::ApplyOperations(InKeyTime, Operations, ObjectBinding, InSequencer, OutResults);
	};

	Operation.IterateOperations(Iterator);
}

int32 GetPreviousKey(FMovieSceneDoubleChannel& Channel, FFrameNumber Time)
{
	TArray<FFrameNumber> KeyTimes;
	TArray<FKeyHandle> KeyHandles;

	TRange<FFrameNumber> Range;
	Range.SetLowerBound(TRangeBound<FFrameNumber>::Open());
	Range.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(Time));
	Channel.GetData().GetKeys(Range, &KeyTimes, &KeyHandles);

	if (KeyHandles.Num() <= 0)
	{
		return INDEX_NONE;
	}

	int32 Index = Channel.GetData().GetIndex(KeyHandles[KeyHandles.Num() - 1]);
	return Index;
}

EPropertyKeyedStatus F3DTransformTrackEditor::GetKeyedStatusInSection(const UMovieScene3DTransformSection& Section, const TRange<FFrameNumber>& Range, EMovieSceneTransformChannel TransformChannel, TConstArrayView<int32> ChannelIndices) const
{
	EPropertyKeyedStatus SectionKeyedStatus = EPropertyKeyedStatus::NotKeyed;

	// Skip Section if the Transform Channel is completely masked out
	if (!EnumHasAnyFlags(Section.GetMask().GetChannels(), TransformChannel))
	{
		return SectionKeyedStatus;
	}

	int32 EmptyChannelCount = 0;

	FMovieSceneChannelProxy& ChannelProxy = Section.GetChannelProxy();
	for (int32 ChannelIndex : ChannelIndices)
	{
		FMovieSceneDoubleChannel* Channel = ChannelProxy.GetChannel<FMovieSceneDoubleChannel>(ChannelIndex);
		if (!Channel)
		{
			continue;
		}

		if (Channel->GetNumKeys() == 0)
		{
			++EmptyChannelCount;
			continue;
		}

		SectionKeyedStatus = FMath::Max(SectionKeyedStatus, EPropertyKeyedStatus::KeyedInOtherFrame);

		TArray<FFrameNumber> KeyTimes;
		Channel->GetKeys(Range, &KeyTimes, nullptr);
		if (KeyTimes.IsEmpty())
		{
			++EmptyChannelCount;
		}
		else
		{
			SectionKeyedStatus = FMath::Max(SectionKeyedStatus, EPropertyKeyedStatus::PartiallyKeyed);
		}
	}

	if (EmptyChannelCount == 0 && SectionKeyedStatus == EPropertyKeyedStatus::PartiallyKeyed)
	{
		SectionKeyedStatus = EPropertyKeyedStatus::KeyedInFrame;
	}
	return SectionKeyedStatus;
}

EPropertyKeyedStatus F3DTransformTrackEditor::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle, EMovieSceneTransformChannel TransformChannel) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(F3DTransformTrackEditor::GetPropertyKeyedStatus);

	TSharedPtr<ISequencer> SequencerPtr = FMovieSceneTrackEditor::GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	UMovieSceneSequence* Sequence = SequencerPtr->GetFocusedMovieSceneSequence();

	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle.GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	TSet<const UMovieScene3DTransformSection*> ProcessedSections;
	ProcessedSections.Reserve(OuterObjects.Num());

	TArray<int32, TFixedAllocator<3>> ChannelIndices;
	switch (TransformChannel)
	{
	case EMovieSceneTransformChannel::Translation:
		ChannelIndices = { 0, 1, 2 };
		break;

	case EMovieSceneTransformChannel::Rotation:
		ChannelIndices = { 3, 4, 5 };
		break;

	case EMovieSceneTransformChannel::Scale:
		ChannelIndices = { 6, 7, 8 };
		break;
	}

	const TRange<FFrameNumber> FrameRange = TRange<FFrameNumber>(SequencerPtr->GetLocalTime().Time.FrameNumber);

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	// List of Tracks that had no sections at the current frame that require an additional check by going through all its sections and whether there's a key in any of them.
	// Used only to determine whether to return "Not Keyed" or "Keyed In Other Frame".
	TArray<const UMovieScene3DTransformTrack*> TransformTracksToCheck;
	TransformTracksToCheck.Reserve(OuterObjects.Num());

	for (UObject* Object : OuterObjects)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(Object);
		if (!SceneComponent)
		{
			continue;
		}

		TArray<UMovieScene3DTransformTrack*> TransformTracks;
		{
			constexpr bool bCreateHandleIfMissing = false;

			// Include Owner Actor Transform Track if Scene Component is Root
			AActor* OwningActor = SceneComponent->GetOwner();
			if (OwningActor && OwningActor->GetRootComponent() == SceneComponent)
			{
				FGuid OwningActorHandle = SequencerPtr->GetHandleToObject(OwningActor, bCreateHandleIfMissing);
				if (OwningActorHandle.IsValid())
				{
					TransformTracks.Add(MovieScene->FindTrack<UMovieScene3DTransformTrack>(OwningActorHandle, TransformPropertyName));
				}
			}

			FGuid SceneComponentHandle = SequencerPtr->GetHandleToObject(SceneComponent, bCreateHandleIfMissing);
			if (SceneComponentHandle.IsValid())
			{
				TransformTracks.Add(MovieScene->FindTrack<UMovieScene3DTransformTrack>(SceneComponentHandle, TransformPropertyName));
			}
		}

		for (UMovieScene3DTransformTrack* TransformTrack : TransformTracks)
		{
			if (!TransformTrack || TransformTrack->IsEmpty())
			{
				continue;
			}

			TArray<UMovieSceneSection*, TInlineAllocator<4>> Sections = TransformTrack->FindAllSections(FrameRange.GetLowerBoundValue());
			for (const UMovieSceneSection* Section : Sections)
			{
				const UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
				if (!TransformSection)
				{
					continue;
				}

				ProcessedSections.Add(TransformSection);

				EPropertyKeyedStatus NewKeyedStatus = GetKeyedStatusInSection(*TransformSection, FrameRange, TransformChannel, ChannelIndices);
				KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);

				// Maximum Status Reached no need to iterate further
				if (KeyedStatus == EPropertyKeyedStatus::KeyedInFrame)
				{
					return KeyedStatus;
				}
			}

			if (KeyedStatus == EPropertyKeyedStatus::NotKeyed)
			{
				TransformTracksToCheck.Add(TransformTrack);
			}
		}		
	}

	// If there's no key in the provided sections look through all sections of the tracks
	// And return "KeyedInOtherFrame" as soon as there's a keyed section
	if (KeyedStatus == EPropertyKeyedStatus::NotKeyed)
	{
		for (const UMovieScene3DTransformTrack* TransformTrack : TransformTracksToCheck)
		{
			if (!TransformTrack)
			{
				continue;
			}

			for (const UMovieSceneSection* Section : TransformTrack->GetAllSections())
			{
				const UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);

				// Skip sections that were already processed
				if (TransformSection && !ProcessedSections.Contains(TransformSection))
				{
					EPropertyKeyedStatus NewKeyedStatus = GetKeyedStatusInSection(*TransformSection, FrameRange, TransformChannel, ChannelIndices);
					KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);

					// Maximum Status Reached no need to iterate further
					if (KeyedStatus >= EPropertyKeyedStatus::KeyedInOtherFrame)
					{
						return KeyedStatus;
					}
				}
			}
		}
	}

	return KeyedStatus;
}

void F3DTransformTrackEditor::OnTransformPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, EMovieSceneTransformChannel TransformChannel)
{
	// Key Property sends in one object at a time
	USceneComponent* SceneComponent = MovieSceneHelpers::SceneComponentFromRuntimeObject(PropertyChangedParams.ObjectsThatChanged[0]);
	if (!SceneComponent)
	{
		return;
	}

	UObject* ObjectToKey = SceneComponent;

	// Set Owning Actor to key instead of Scene Component if it's the root
	AActor* OwningActor = SceneComponent->GetOwner();
	if (OwningActor && SceneComponent == OwningActor->GetRootComponent())
	{
		ObjectToKey = OwningActor;
	}

	auto InitializeNewTrack = [](UMovieScene3DTransformTrack* NewTrack)
	{
		NewTrack->SetPropertyNameAndPath(TransformPropertyName, TransformPropertyName.ToString());
	};

	auto GenerateKeys = [this, SceneComponent, TransformChannel](UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys)
	{
		UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(Section);

		// Ensure that the Transform Channel is masked in
		EMovieSceneTransformChannel ExistingChannels = TransformSection->GetMask().GetChannels();
		if (!EnumHasAnyFlags(ExistingChannels, TransformChannel))
		{
			TransformSection->Modify();
			TransformSection->SetMask(TransformSection->GetMask().GetChannels() | TransformChannel);
		}

		GetTransformKeys(TOptional<FTransformData>(), FTransformData(SceneComponent), TransformChannel, SceneComponent, Section, OutGeneratedKeys);
	};

	auto KeyProperty = [this, ObjectToKey, PropertyChangedParams, InitializeNewTrack, GenerateKeys, TransformChannel](FFrameNumber KeyTime)
	{
		FKeyPropertyResult KeyPropertyResult = this->AddKeysToObjects({ ObjectToKey },
			KeyTime,
			PropertyChangedParams.KeyMode,
			UMovieScene3DTransformTrack::StaticClass(),
			TransformPropertyName,
			InitializeNewTrack,
			GenerateKeys);

		for (TWeakObjectPtr<UMovieSceneSection>& WeakSection : KeyPropertyResult.SectionsKeyed)
		{
			if (UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(WeakSection.Get()))
			{
				FMovieSceneConstraintChannelHelper::CompensateIfNeeded(GetSequencer(), Section, KeyTime, true /*bCompPreviousTick*/);
			}
		}

		// For new sections created, mask so only the relevant transform channel appears
		for (TWeakObjectPtr<UMovieSceneSection>& WeakSection : KeyPropertyResult.SectionsCreated)
		{
			if (UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(WeakSection.Get()))
			{
				Section->SetMask(TransformChannel);
			}
		}

		return KeyPropertyResult;
	};

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(KeyProperty));
}

void F3DTransformTrackEditor::ProcessKeyOperation(UObject* ObjectToKey, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	USceneComponent* Component = MovieSceneHelpers::SceneComponentFromRuntimeObject(ObjectToKey);
	if (!Component)
	{
		return;
	}

	FTransform CurrentTransform(Component->GetRelativeRotation(), Component->GetRelativeLocation(), Component->GetRelativeScale3D());

	for (int32 Index = 0; Index < SectionsToKey.Num(); ++Index)
	{
		FTransformData RecomposedTransform = RecomposeTransform(CurrentTransform, ObjectToKey, SectionsToKey[Index].Section->GetSectionObject());

		for (TSharedPtr<IKeyArea> KeyArea : SectionsToKey[Index].KeyAreas)
		{
			FMovieSceneChannelHandle Handle  = KeyArea->GetChannel();
			if (Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
			{
				FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(Handle.Get());

				if (ensureAlwaysMsgf(Channel, TEXT("Channel: %s for Key Area %s does not exist. Keying may not function properly"), *Handle.GetChannelTypeName().ToString(), *KeyArea->GetName().ToString()))
				{
					double Value =
						Handle.GetChannelIndex() == 0 ? RecomposedTransform.Translation[0] :
						Handle.GetChannelIndex() == 1 ? RecomposedTransform.Translation[1] :
						Handle.GetChannelIndex() == 2 ? RecomposedTransform.Translation[2] :
						Handle.GetChannelIndex() == 3 ? RecomposedTransform.Rotation.Roll :
						Handle.GetChannelIndex() == 4 ? RecomposedTransform.Rotation.Pitch :
						Handle.GetChannelIndex() == 5 ? RecomposedTransform.Rotation.Yaw :
						Handle.GetChannelIndex() == 6 ? RecomposedTransform.Scale[0] :
						Handle.GetChannelIndex() == 7 ? RecomposedTransform.Scale[1] :
						Handle.GetChannelIndex() == 8 ? RecomposedTransform.Scale[2] : 0.f;

					if (KeyArea->GetName() == "Rotation.X" ||
						KeyArea->GetName() == "Rotation.Y" ||
						KeyArea->GetName() == "Rotation.Z")
					{
						int32 PreviousKey = GetPreviousKey(*Channel, KeyTime);
						if (PreviousKey != INDEX_NONE && PreviousKey < Channel->GetData().GetValues().Num())
						{
							double OldValue = Channel->GetData().GetValues()[PreviousKey].Value;
							Value = UnwindChannel(OldValue, Value);
						}
					}

					EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(Channel, KeyTime, InSequencer.GetKeyInterpolation());
					FKeyHandle KeyHandle = AddKeyToChannel(Channel, KeyTime, Value, Interpolation);

					if (OutResults)
					{
						FAddKeyResult Result;
						Result.KeyArea = KeyArea;
						Result.KeyHandle = KeyHandle;
						OutResults->Add(Result);
					}
				}
			}
			else
			{
				FKeyHandle KeyHandle = KeyArea->AddOrUpdateKey(KeyTime, FGuid(), InSequencer);

				if (OutResults)
				{
					FAddKeyResult Result;
					Result.KeyArea = KeyArea;
					Result.KeyHandle = KeyHandle;
					OutResults->Add(Result);
				}
			}
		}
	}
}

bool F3DTransformTrackEditor::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const
{
	SectionsGettingUndone.SetNum(0);
	// Check if we care about the undo/redo
	bool bGettingUndone = false;
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjects)
	{

		UObject* Object = TransactionObjectPair.Key;
		while (Object != nullptr)
		{
			const UClass* ObjectClass = Object->GetClass();
			if (ObjectClass && ObjectClass->IsChildOf(UMovieScene3DTransformSection::StaticClass()))
			{
				UMovieScene3DTransformSection* Section = Cast< UMovieScene3DTransformSection>(Object);
				if (Section)
				{
					SectionsGettingUndone.Add(Section);
				}
				bGettingUndone = true;
				break;
			}
			Object = Object->GetOuter();
		}
	}

	return bGettingUndone;
}

void F3DTransformTrackEditor::PostUndo(bool bSuccess)
{
	for (TWeakObjectPtr<UMovieScene3DTransformSection> &Section : SectionsGettingUndone)
	{
		if (Section.IsValid())
		{
			TArray<FConstraintAndActiveChannel>& ConstraintChannels = Section->GetConstraintsChannels();
			for (FConstraintAndActiveChannel& Channel : ConstraintChannels)
			{
				HandleOnConstraintAdded(Section.Get(), &(Channel.ActiveChannel));
			}
		}
	}
}

void F3DTransformTrackEditor::HandleOnConstraintAdded(IMovieSceneConstrainedSection* InSection, FMovieSceneConstraintChannel* InConstraintChannel)
{
	if (!InConstraintChannel)
	{
		return;
	}
	// handle scene component changes so we can update the gizmo location when it changes
	if (!FConstraintsManagerController::OnSceneComponentConstrained().IsBound())
	{
		OnSceneComponentConstrainedHandle =
			FConstraintsManagerController::OnSceneComponentConstrained().AddLambda([](USceneComponent* InSceneComponent)
				{
					//only update gizmo if it or it's actor is seleced, otherwise can be wasteful
					GUnrealEd->UpdatePivotLocationForSelection();
					AActor* Actor = InSceneComponent->GetTypedOuter<AActor>();
					if (USelection* SelectedActors = GEditor->GetSelectedActors())
					{
						if (SelectedActors->IsSelected(Actor))
						{
							GUnrealEd->UpdatePivotLocationForSelection();
							return;
						}
					}
					else if (USelection* SelectedComponents = GEditor->GetSelectedComponents())
					{
						if (SelectedComponents->IsSelected(InSceneComponent))
						{
							GUnrealEd->UpdatePivotLocationForSelection();
						}
					}
				});
	}

	// Store Section so we can remove these delegates
	UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(InSection);
	SectionsToClear.Add(Section);

	// handle key moved
	if (!InConstraintChannel->OnKeyMovedEvent().IsBound())
	{
		InConstraintChannel->OnKeyMovedEvent().AddLambda([this, InSection](
			FMovieSceneChannel* InChannel, const TArray<FKeyMoveEventItem>& InMovedItems)
			{
				const FMovieSceneConstraintChannel* ConstraintChannel = static_cast<FMovieSceneConstraintChannel*>(InChannel);
				HandleConstraintKeyMoved(InSection, ConstraintChannel, InMovedItems);
			});
	}

	// handle key deleted
	if (!InConstraintChannel->OnKeyDeletedEvent().IsBound())
	{
		InConstraintChannel->OnKeyDeletedEvent().AddLambda([this, InSection](
			FMovieSceneChannel* InChannel, const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems)
			{
				const FMovieSceneConstraintChannel* ConstraintChannel = static_cast<FMovieSceneConstraintChannel*>(InChannel);
				HandleConstraintKeyDeleted(InSection, ConstraintChannel, InDeletedItems);
			});
	}

	// handle constraint deleted
	if (InSection)
	{
		HandleConstraintRemoved(InSection);
	}

	if (!UTickableTransformConstraint::GetOnConstraintChanged().IsBoundToObject(this))
	{
		UTickableTransformConstraint::GetOnConstraintChanged().AddRaw(this, &F3DTransformTrackEditor::HandleConstraintPropertyChanged);
	}
}

static UTickableTransformConstraint* GetTickableTransformConstraint(IMovieSceneConstrainedSection* InSection, const FMovieSceneConstraintChannel* InConstraintChannel)
{
	UTickableTransformConstraint* Constraint = nullptr;
	// get constraint channel
	TArray<FConstraintAndActiveChannel>& ConstraintChannels = InSection->GetConstraintsChannels();
	const int32 Index = ConstraintChannels.IndexOfByPredicate([InConstraintChannel](const FConstraintAndActiveChannel& InChannel)
		{
			return &(InChannel.ActiveChannel) == InConstraintChannel;
		});

	if (Index != INDEX_NONE)
	{
		Constraint = Cast<UTickableTransformConstraint>(ConstraintChannels[Index].GetConstraint().Get());
	}
	return Constraint;
}

void F3DTransformTrackEditor::HandleConstraintKeyDeleted(IMovieSceneConstrainedSection* InSection, const FMovieSceneConstraintChannel* InConstraintChannel,
	const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems) const
{
	if (FMovieSceneConstraintChannelHelper::bDoNotCompensate)
	{
		return;
	}
	
	if (!InConstraintChannel)
	{
		return;
	}
	UTickableTransformConstraint* Constraint = GetTickableTransformConstraint(InSection, InConstraintChannel);
	if (!Constraint)
	{
		return;
	}

    if(UTickableTransformConstraint* Constrain = GetTickableTransformConstraint(InSection,InConstraintChannel))
	{
		for (const FKeyAddOrDeleteEventItem& EventItem : InDeletedItems)
		{
			UMovieSceneSection* Section = Cast<UMovieSceneSection>(InSection);
			FMovieSceneConstraintChannelHelper::HandleConstraintKeyDeleted(
				Constraint, InConstraintChannel,
				GetSequencer(), Section,
				EventItem.Frame);
			
		}
	}
}

void F3DTransformTrackEditor::HandleConstraintKeyMoved(IMovieSceneConstrainedSection* InSection, const FMovieSceneConstraintChannel* InConstraintChannel,
	const TArray<FKeyMoveEventItem>& InMovedItems)
{
	if (!InConstraintChannel)
	{
		return;
	}
	UTickableTransformConstraint* Constraint = GetTickableTransformConstraint(InSection, InConstraintChannel);
	if (!Constraint)
	{
		return;
	}
	UMovieSceneSection* Section = Cast<UMovieSceneSection>(InSection);

	for (const FKeyMoveEventItem& MoveEventItem : InMovedItems)
	{
		FMovieSceneConstraintChannelHelper::HandleConstraintKeyMoved(
			Constraint, InConstraintChannel, Section,
			MoveEventItem.Frame, MoveEventItem.NewFrame);
	}
}

void F3DTransformTrackEditor::HandleConstraintRemoved(IMovieSceneConstrainedSection* InSection) 
{
	UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(InSection);
	if (!Section)
	{
		return;
	}

	if (!Section->OnConstraintRemovedHandle.IsValid())
	{
		Section->OnConstraintRemovedHandle =
			FConstraintsManagerController::GetNotifyDelegate().AddLambda([WeakSection = MakeWeakObjectPtr(Section), this](EConstraintsManagerNotifyType InNotifyType, UObject *InObject)
				{
					switch (InNotifyType)
					{
						case EConstraintsManagerNotifyType::ConstraintAdded:
							break;
						case EConstraintsManagerNotifyType::ConstraintRemoved:
						case EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation:
							{
								const UTickableConstraint* Constraint = Cast<UTickableConstraint>(InObject);
								UMovieScene3DTransformSection* Section = WeakSection.IsValid() ? WeakSection.Get() : nullptr;
								if (!IsValid(Constraint) || !Section)
								{
									return;
								}

								const FConstraintAndActiveChannel* ConstraintChannel = Section->GetConstraintChannel(Constraint->ConstraintID);
								if (!ConstraintChannel || ConstraintChannel->GetConstraint().Get() != Constraint)
								{
									return;
								}

								TSharedPtr<ISequencer> Sequencer = GetSequencer();
								
								const bool bCompensate = (InNotifyType == EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation);
								if (bCompensate && ConstraintChannel->GetConstraint().Get())
								{
									FMovieSceneConstraintChannelHelper::HandleConstraintRemoved(
										ConstraintChannel->GetConstraint().Get(),
										&ConstraintChannel->ActiveChannel,
										Sequencer,
										Section);
								}

								Section->RemoveConstraintChannel(Constraint);

								if (Sequencer)
								{
									Sequencer->RecreateCurveEditor();
								}
							}
							break;
						case EConstraintsManagerNotifyType::ManagerUpdated:
							if (UMovieScene3DTransformSection* Section = WeakSection.IsValid() ? WeakSection.Get() : nullptr)
							{
								Section->OnConstraintsChanged();
							}
							break;
						case EConstraintsManagerNotifyType::GraphUpdated:
							break;
					}
				});
		
		ConstraintHandlesToClear.Add(Section->OnConstraintRemovedHandle);
	}
}

void F3DTransformTrackEditor::HandleConstraintPropertyChanged(UTickableTransformConstraint* InConstraint, const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	if (!IsValid(InConstraint))
	{
		return;
	}

	// find constraint section
	const UTransformableComponentHandle* Handle = Cast<UTransformableComponentHandle>(InConstraint->ChildTRSHandle);
	if (!IsValid(Handle) || !Handle->IsValid())
	{
		return;
	}

	const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();	
	ITransformConstraintChannelInterface* Interface = InterfaceRegistry.FindConstraintChannelInterface(Handle->GetClass());
	if (!Interface)
	{
		return;
	}
	
	UMovieSceneSection* Section = Interface->GetHandleConstraintSection(Handle, GetSequencer());
	IMovieSceneConstrainedSection* ConstraintSection = Cast<IMovieSceneConstrainedSection>(Section);
	if (!ConstraintSection)
	{
		return;
	}

	// find corresponding channel
	const TArray<FConstraintAndActiveChannel>& ConstraintChannels = ConstraintSection->GetConstraintsChannels();
	const FConstraintAndActiveChannel* Channel = ConstraintChannels.FindByPredicate([InConstraint](const FConstraintAndActiveChannel& Channel)
	{
		return Channel.GetConstraint() == InConstraint;
	});

	if (!Channel)
	{
		return;
	}

	FMovieSceneConstraintChannelHelper::HandleConstraintPropertyChanged(
			InConstraint, Channel->ActiveChannel, InPropertyChangedEvent, GetSequencer(), Section);
}

void F3DTransformTrackEditor::ClearOutConstraintDelegates() 
{
	for (FDelegateHandle& Handle : ConstraintHandlesToClear)
	{
		if (Handle.IsValid())
		{
			FConstraintsManagerController::GetNotifyDelegate().Remove(Handle);
		}
	}
	ConstraintHandlesToClear.Reset();

	for (TWeakObjectPtr<UMovieScene3DTransformSection>& Section : SectionsToClear)
	{
		if (IMovieSceneConstrainedSection* CRSection = Section.Get())
		{
			// clear constraint channels
			TArray<FConstraintAndActiveChannel>& ConstraintChannels = CRSection->GetConstraintsChannels();
			for (FConstraintAndActiveChannel& Channel : ConstraintChannels)
			{
				Channel.ActiveChannel.OnKeyMovedEvent().Clear();
				Channel.ActiveChannel.OnKeyDeletedEvent().Clear();
			}

			if (CRSection->OnConstraintRemovedHandle.IsValid())
			{
				CRSection->OnConstraintRemovedHandle.Reset();
			}

			CRSection->ConstraintChannelAdded().RemoveAll(this);
		}
	}
	SectionsToClear.Reset();

	UTickableTransformConstraint::GetOnConstraintChanged().RemoveAll(this);
}
#undef LOCTEXT_NAMESPACE

