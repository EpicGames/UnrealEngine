// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Layout/Visibility.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "Framework/Commands/UICommandList.h"
#include "ISequencerTrackEditor.h"
#include "KeyframeTrackEditor.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "SequencerKeyParams.h"
#include "EditorUndoClient.h"

class AActor;
struct FAssetData;
class FLevelEditorViewportClient;
class SHorizontalBox;
class UTickableTransformConstraint;

namespace UE::Sequencer { struct FAddKeyResult; }

/**
 * Tools for animatable transforms
 */
class F3DTransformTrackEditor
	: public FKeyframeTrackEditor<UMovieScene3DTransformTrack>, public FEditorUndoClient
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer	The sequencer instance to be used by this tool
	 */
	F3DTransformTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~F3DTransformTrackEditor();

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );


	/**
	*
	* Cached version of the current track editor, that lets other animaiton tools call specific functions. in particularly setting keys on
	* current values
	*
	**/
	static F3DTransformTrackEditor* CurrentEditor;

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) override;
	virtual void BuildObjectBindingColumnWidgets(TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding, const UE::Sequencer::FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindinsg, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual void OnRelease() override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual void BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track ) override;
	virtual bool HasTransformKeyBindings() const override { return true; }
	virtual bool CanAddTransformKeysForSelectedObjects() const override;
	virtual void OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) override;
	virtual void OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) override;
	virtual void OnPreSaveWorld(UWorld* World) override;
	virtual void OnPostSaveWorld(UWorld* World) override;

	//FEditorUndoClient Interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	EPropertyKeyedStatus GetKeyedStatusInSection(const UMovieScene3DTransformSection& Section, const TRange<FFrameNumber>& Range, EMovieSceneTransformChannel TransformChannel, TConstArrayView<int32> ChannelIndices) const;

	EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle, EMovieSceneTransformChannel TransformChannel) const;

	void OnTransformPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, EMovieSceneTransformChannel TransformChannel);

	void ProcessKeyOperation(UObject* ObjectToKey, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr);

	/** Whether the object has an existing transform track */
	bool HasTransformTrack( UObject& InObject ) const;

	/**
	 * Called before an actor or component transform changes
	 *
	 * @param Object The object whose transform is about to change
	 */
	void OnPreTransformChanged( UObject& InObject );

	/**
	 * Called when an actor or component transform changes
	 *
	 * @param Object The object whose transform has changed
	 */
	void OnTransformChanged( UObject& InObject );

	/** 
	 * Called before an actor or component property changes.
	 * Forward to OnPreTransformChanged if the property is transform related.
	 *
	 * @param InObject The object whose property is about to change
	 * @param InPropertyChain the property that is about to change
	 */
	void OnPrePropertyChanged(UObject* InObject, const class FEditPropertyChain& InPropertyChain);

	/** 
	 * Called before an actor or component property changes.
	 * Forward to OnTransformChanged if the property is transform related.
	 *
	 * @param InObject The object whose property is about to change
	 * @param InPropertyChangedEvent the property that changed
	 */
	void OnPostPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);


	/** Delegate for camera button visible state */
	EVisibility IsCameraVisible(FGuid ObjectGuid) const;

	/** Delegate for camera button lock state */
	ECheckBoxState IsCameraLocked(FGuid ObjectGuid) const; 

	/** Delegate for locked camera button */
	void OnLockCameraClicked(ECheckBoxState CheckBoxState, FGuid ObjectGuid);

	/** Delegate for camera button lock tooltip */
	FText GetLockCameraToolTip(FGuid ObjectGuid) const; 

	/** Implementation of checking if a camera is locked */
	bool IsCameraBindingLocked(FGuid ObjectGuid) const; 

	/** Toggle whether a camera is locked in the given viewport (or the active viewport if not provided) */
	void LockCameraBinding(bool bLock, FGuid ObjectGuid, FLevelEditorViewportClient* ViewportClient = nullptr, bool bRemoveCinematicLock = true);

	/** Generates transform keys based on the last transform, the current transform, and other options. 
		One transform key is generated for each individual key to be added to the section. */
	void GetTransformKeys( const TOptional<FTransformData>& LastTransform, const FTransformData& CurrentTransform, EMovieSceneTransformChannel ChannelsToKey, UObject* Object, UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys );

	/** Transform origin which may be set for the current level sequence */
	FTransform GetTransformOrigin() const;

	/*In the case the object is being animated with root motion, return the offset from the keyed transform value that root motion is currently applying.*/
	FTransform GetRootMotionOffset(UObject* AnimatedObject) const;

	/*In the case the object is being animated with root motion, updates the offset based on the current object position.*/
	void UpdateRootMotionOffset(UObject* AnimatedObject);

	/** 
	 * Adds transform keys to an object represented by a handle.

	 * @param ObjectHandle The handle to the object to add keys to.
	 * @param ChannelToKey The channels to add keys to.
	 * @param KeyParams Parameters which control how the keys are added. 
	 */
	void AddTransformKeysForHandle( TArray<FGuid> ObjectHandles, EMovieSceneTransformChannel ChannelToKey, ESequencerKeyMode KeyMode );


	/**
	* Adds keys to a specific actor.

	* @param LastTransform The last known transform of the actor if any.
	* @param CurrentTransform The current transform of the actor.
	* @param ChannelToKey The channels to add keys to.
	* @param KeyParams Parameters which control how the keys are added.
	*/
	void AddTransformKeys( UObject* ObjectToKey, const TOptional<FTransformData>& LastTransform, const FTransformData& CurrentTransform, EMovieSceneTransformChannel ChannelsToKey, ESequencerKeyMode KeyMode );

	FTransformData RecomposeTransform(const FTransformData& InTransformData, UObject* AnimatedObject, UMovieSceneSection* Section);

public:
	/**
	* Adds transform keys to a specific object.

	* @param Object The object to add keys to.
	* @param ChannelToKey The channels to add keys to.
	* @param KeyParams Parameters which control how the keys are added.
	*/
	void AddTransformKeysForObject(UObject* Object, EMovieSceneTransformChannel ChannelToKey, ESequencerKeyMode KeyMode);
private:

	/** ConstraintChannel Delegates*/
	FDelegateHandle OnSceneComponentConstrainedHandle;
	void HandleOnConstraintAdded(IMovieSceneConstrainedSection* InSection, FMovieSceneConstraintChannel* InConstraintChannel);
	void HandleConstraintKeyDeleted(IMovieSceneConstrainedSection* InSection, const FMovieSceneConstraintChannel* InConstraintChannel,
		const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems) const;
	void HandleConstraintKeyMoved(IMovieSceneConstrainedSection* InSection, const FMovieSceneConstraintChannel* InConstraintChannel,
		const TArray<FKeyMoveEventItem>& InMovedItems);
	void HandleConstraintRemoved(IMovieSceneConstrainedSection* InSection);
	void HandleConstraintPropertyChanged(UTickableTransformConstraint* InConstraint, const FPropertyChangedEvent& InPropertyChangedEvent) const;
	
	void ClearOutConstraintDelegates();

	void BuildAssetPickerSubMenu(FMenuBuilder& InMenuBuilder, UMovieScene3DTransformTrack* InTransformTrack);

private:

	static FName TransformPropertyName;

	/** Mapping of objects to their existing transform data (for comparing against new transform data) */
	TMap< TWeakObjectPtr<UObject>, FTransformData > ObjectToExistingTransform;

	struct FTransformPropertyInfo
	{
		const FProperty* Property;
		EMovieSceneTransformChannel TransformChannel;
	};
	/** Array of transform property info for the scene component transform properties for explicit support */
	TArray<FTransformPropertyInfo, TFixedAllocator<3>> TransformProperties;

	/** Command Bindings added by the Transform Track Editor to Sequencer and curve editor. */
	TSharedPtr<FUICommandList> CommandBindings;

	/** List of locked cameras to restore after save */
	TMap<FLevelEditorViewportClient*, FGuid> LockedCameraBindings;

	/** Array of sections that are getting undone, we need to recreate any constraint channel add, move key delegates to them*/
	mutable TArray<TWeakObjectPtr<UMovieScene3DTransformSection>> SectionsGettingUndone;

	/** Set of delegate handles we have added delegate's too, need to clear them*/
	TSet<FDelegateHandle> ConstraintHandlesToClear;
};
