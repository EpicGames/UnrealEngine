// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "LevelSequencePlayer.h"
#include "MovieSceneCapture.h"
#include "AutomatedLevelSequenceCapture.generated.h"

#define UE_API MOVIESCENETOOLS_API

class ALevelSequenceActor;
class FJsonObject;
class FSceneViewport;
class ULevelSequenceBurnInOptions;
struct FMovieSceneTimeController_FrameStep;

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, PerObjectConfig, BlueprintType)
class UAutomatedLevelSequenceCapture : public UMovieSceneCapture
{
public:
	UE_API UAutomatedLevelSequenceCapture(const FObjectInitializer&);

	GENERATED_BODY()

	/** This name is used by the UI to save/load a specific instance of the settings from config that doesn't affect the CDO which would affect scripting environments. */
	static UE_API const FName AutomatedLevelSequenceCaptureUIName;

	/** A level sequence asset to playback at runtime - used where the level sequence does not already exist in the world. */
	UPROPERTY(BlueprintReadWrite, Category=Animation)
	FSoftObjectPath LevelSequenceAsset;

	/** Optional shot name to render. The frame range to render will be set to the shot frame range. */
	UPROPERTY(BlueprintReadWrite, Category=Animation)
	FString ShotName;

	/** When enabled, the StartFrame setting will override the default starting frame number */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay)
	bool bUseCustomStartFrame;

	/** Frame number to start capturing. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay, meta=(EditCondition="bUseCustomStartFrame", DisplayName="Start Frame"))
	FFrameNumber CustomStartFrame;

	/** When enabled, the EndFrame setting will override the default ending frame number */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay)
	bool bUseCustomEndFrame;

	/** Frame number to end capturing. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay, meta=(EditCondition="bUseCustomEndFrame", DisplayName="End Frame"))
	FFrameNumber CustomEndFrame;

	/** The number of extra frames to play before the sequence's start frame, to "warm up" the animation.  This is useful if your
	    animation contains particles or other runtime effects that are spawned into the scene earlier than your capture start frame */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay)
	int32 WarmUpFrameCount;

	/** The number of seconds to wait (in real-time) before we start playing back the warm up frames.  Useful for allowing post processing effects to settle down before capturing the animation. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Animation, AdvancedDisplay, meta=(Units=Seconds, ClampMin=0))
	float DelayBeforeWarmUp;

	/** The number of seconds to wait (in real-time) at shot boundaries.  Useful for allowing post processing effects to settle down before capturing the animation. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Animation, AdvancedDisplay, meta = (Units = Seconds, ClampMin = 0))
	float DelayBeforeShotWarmUp;

	/** The number of seconds to wait (in real-time) at every frame.  Useful for allowing post processing effects to settle down before capturing the animation. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Animation, AdvancedDisplay, meta = (Units = Seconds, ClampMin = 0))
	float DelayEveryFrame;

	UPROPERTY(Transient, EditAnywhere, BlueprintReadWrite, Category=CaptureSettings, AdvancedDisplay, meta=(EditInline))
	TObjectPtr<ULevelSequenceBurnInOptions> BurnInOptions;

	/** Whether to write edit decision lists (EDLs) if the sequence contains shots */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Sequence)
	bool bWriteEditDecisionList;

	/** Whether to write Final Cut Pro XML files (XMLs) if the sequence contains shots */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=Sequence)
	bool bWriteFinalCutProXML;

public:
	// UMovieSceneCapture interface
	UE_API virtual void Initialize(TSharedPtr<FSceneViewport> InViewport, int32 PIEInstance = -1) override;

	UE_API virtual void LoadFromConfig() override;

	UE_API virtual void SaveToConfig() override;

	UE_API virtual void Close() override;

	UE_API double GetEstimatedCaptureDurationSeconds() const override;
	
	/** Override the render frames with the given start/end frames. Restore the values when done rendering. */
	UE_API void SetFrameOverrides(FFrameNumber InStartFrame, FFrameNumber InEndFrame);

protected:

	UE_API virtual void AddFormatMappings(TMap<FString, FStringFormatArg>& OutFormatMappings, const FFrameMetrics& FrameMetrics) const override;

	/** Custom, additional json serialization */
	UE_API virtual void SerializeAdditionalJson(FJsonObject& Object);

	/** Custom, additional json deserialization */
	UE_API virtual void DeserializeAdditionalJson(const FJsonObject& Object);

private:

	/** Update any cached information we need from the level sequence actor */
	UE_API void UpdateFrameState();

	/** Called when the level sequence has updated the world */
	UE_API void SequenceUpdated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime);

	/** Called to set up the player's playback range */
	UE_API void SetupFrameRange();

	/** Enable cinematic mode override */
	UE_API void EnableCinematicMode();

	/** Export EDL if requested */
	UE_API void ExportEDL();

	/** Export FCPXML if requested */
	UE_API void ExportFCPXML();

	/** Delegate binding for the above callback */
	FDelegateHandle OnPlayerUpdatedBinding;

private:

	UE_API virtual void OnTick(float DeltaSeconds) override;

	/** Initialize all the shots to be recorded, ie. expand section ranges with handle frames */
	UE_API bool InitializeShots();

	/** Set up the current shot to be recorded, ie. expand playback range to the section range */
	UE_API bool SetupShot(FFrameNumber& StartTime, FFrameNumber& EndTime);

	/** Restore any modification to shots */
	UE_API void RestoreShots();

	/** Restore frame settings from overridden shot frames */
	UE_API bool RestoreFrameOverrides();

	UE_API void DelayBeforeWarmupFinished();

	UE_API void PauseFinished();


	/** The pre-existing level sequence actor to use for capture that specifies playback settings */
	UPROPERTY()
	TWeakObjectPtr<ALevelSequenceActor> LevelSequenceActor;

	/** The viewport being captured. */
	TWeakPtr<FSceneViewport> Viewport;

	/** Which state we're in right now */
	enum class ELevelSequenceCaptureState
	{
		Setup,
		DelayBeforeWarmUp,
		ReadyToWarmUp,
		WarmingUp,
		FinishedWarmUp,
		Paused,
		FinishedPause,
	} CaptureState;

	/** The number of warm up frames left before we actually start saving out images */
	int32 RemainingWarmUpFrames;
	
	/** The number of individual shot movies to render */
	int32 NumShots;

	/** The current shot movie that is rendering */
	int32 ShotIndex;

	FLevelSequencePlayerSnapshot CachedState;

	FTimerHandle DelayTimer;

	struct FCinematicShotCache
	{
		FCinematicShotCache(bool bInActive, bool bInLocked, const TRange<FFrameNumber>& InShotRange, const TRange<FFrameNumber>& InMovieSceneRange) : 
			  bActive(bInActive)
			, bLocked(bInLocked)
			, ShotRange(InShotRange)
			, MovieSceneRange(InMovieSceneRange)
		{
		}

		bool bActive;
		bool bLocked;
		TRange<FFrameNumber> ShotRange;
		TRange<FFrameNumber> MovieSceneRange;
	};

	TArray<FCinematicShotCache> CachedShotStates;
	TRange<FFrameNumber> CachedPlaybackRange;

	TOptional<FFrameNumber> CachedStartFrame;
	TOptional<FFrameNumber> CachedEndFrame;
	TOptional<bool> bCachedUseCustomStartFrame;
	TOptional<bool> bCachedUseCustomEndFrame;

	TSharedPtr<FMovieSceneTimeController_FrameStep> TimeController;

	// We cache these off on initialization so we can restore to them after running an audio pass.
	int32 CachedWarmUpFrameCount;
	int32 CachedDelayBeforeWarmUp;
	int32 CachedDelayBeforeShotWarmUp;
};

#undef UE_API
