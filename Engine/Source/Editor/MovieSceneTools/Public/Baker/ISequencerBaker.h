// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "Math/Range.h"
#include "EditorSubsystem.h"
#include "MovieSceneTimeUnit.h"
#include "UObject/Interface.h"
#include "ISequencerBaker.generated.h"

#define UE_API MOVIESCENETOOLS_API

namespace UE::AIE
{
	/**
	* FBakeTimeRange
	*
	* Object that contains a range of frame numbers that specify an interval to bake over
	*/
	struct FBakeTimeRange
	{
		FBakeTimeRange() :
			StartFrame(0), EndFrame(0), DisplayRate(1, 1), TickResolution(1, 1)
		{
			IncomingRange = TRange<FFrameNumber>{ 0 };
		};
		~FBakeTimeRange() = default;
		FFrameNumber StartFrame;
		FFrameNumber EndFrame;
		FFrameNumber FrameStep;
		FFrameRate DisplayRate;
		FFrameRate TickResolution;
		int32 NumFrames = 0;
		TRange<FFrameNumber> IncomingRange;

		UE_API bool UpdateIfNeeded(const TRange<FFrameNumber>& InRange, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution);
		UE_API int32 CalculateIndex(const FFrameNumber& InCurrentFrame) const;
	};

	/**
	* FBakeTimeIndex
	*
	* A specific index in the FBakeTimeRange, used to eaily keep track of an index and how it relates to a frame number or a second.
	* This is passed into the recorders on each frame baked.
	*/
	struct FBakeTimeIndex
	{
		int32 Index;
		FFrameNumber FrameNumber;
		double DeltaTime;
	};
}

UINTERFACE(MinimalAPI)
class USequencerBakeRecorder : public UInterface
{
	GENERATED_BODY()
};

/**
* ISequencerBakeRecorder
*
* Main interface for an object that records a specific set of data when the bake happens.  For eample an AnimationSequenceBaker would record a specific
* animation sequence for a specific skel mesh component that you want to record.
*/
class ISequencerBakeRecorder
{
	GENERATED_BODY()

public:
	//virtual ~ISequencerBakeRecorder() is defined above
	//unique ID
	virtual uint32 GetHash() = 0;
	virtual void BakeStarted(const UE::AIE::FBakeTimeRange& InRange) = 0;
	virtual void BakeFrame(const UE::AIE::FBakeTimeIndex& InIndex) = 0;
	virtual void BakeCancelled() = 0;
	virtual void BakeFinished() = 0;
	virtual bool HasSequencerBinding(FGuid InGuid) = 0;
	virtual bool IsolateBakeResult(bool bIsolate) = 0;
	virtual void SetRecordingEnabled(bool bInEnabled) = 0;
	virtual bool GetRecordingEnabled() = 0;
	virtual void ReadyToBake() = 0;
	virtual void RemovedFromBake() = 0;
};

namespace UE::AIE
{

/**
* ISequencerBaker
*
* Main interface for the object performs a bake on a set of ISequencerBakeRecorders
*/

class ISequencerBaker 
{
public:
	struct FRecorderOptions
	{
		FFrameNumber WarmupFrames = FFrameNumber(0);
		FFrameNumber DelayBeforeStart = FFrameNumber(0);
	};
	virtual ~ISequencerBaker() {};
	virtual void AddRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder, const FRecorderOptions& InOptions) = 0;
	virtual void RemoveRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder) = 0;
	virtual FBakeTimeRange GetTimeRange() = 0;

	virtual bool IsBakeRunning(TOptional<float>& OutPercentageDone) = 0;
	virtual void CancelBake() = 0;

};

};

#undef UE_API

