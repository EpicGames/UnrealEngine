// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "ISequencerBaker.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "TickableEditorObject.h"

class FSequencerBakeRecorder;
class ISequencer;
class UMovieSceneTrack;
struct FGuid;
class USkeletalMeshComponent;
class IMovieSceneToolsAnimationBakeHelper;
class UWorld;

#define UE_API MOVIESCENETOOLS_API

namespace UE::Sequencer
{
	//delegates fired when we add or remove a bake recorder
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddBakeRecorder, ISequencerBakeRecorder*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRemoveBakeRecorder, ISequencerBakeRecorder*);

	/**
	* FSequencerBaker
	*
	* Main object that Bakes within ISequencer, using Sequencer evaluation over the range.
	* It keeps track of track signatures for each recorder to seek if they need to get baked when we switch over to using the baked track
	*/
	class FSequencerBaker
		: public UE::AIE::ISequencerBaker
		, public FTickableEditorObject
		, public TSharedFromThis<FSequencerBaker>
	{

	public:
		FSequencerBaker();
		virtual ~FSequencerBaker() {};

		void Initialize(TSharedPtr<ISequencer>& InSequencer);
		void Release();

		//~ ISequencerBaker
		virtual UE::AIE::FBakeTimeRange GetTimeRange() override;
		virtual bool IsBakeRunning(TOptional<float>& OutPercentageDone) override;
		virtual void CancelBake() override;
		virtual  void AddRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder, const UE::AIE::ISequencerBaker::FRecorderOptions& InOptions) override;
		virtual  void RemoveRecorder(TSharedPtr<ISequencerBakeRecorder>& InRecorder) override;

		//~ FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FSequencerBaker, STATGROUP_Tickables); }

	public:
	
		//add all existing recorders to get baked
		UE_API void AddAllRecordersToBake();
		//add extra recorders to get baked just once on next bake
		UE_API void AddExtraRecordersToBake(const TArray<TSharedPtr<ISequencerBakeRecorder>>& InExtraRecorders);
		//bake everything and will remove recorders from bake queue
		UE_API void BakeFullRange();
	
		//this adds that recorder that was already added to be baked on next bake time
		UE_API bool AddRecorderToBake(TSharedPtr<ISequencerBakeRecorder>& InRecorder);

		UE_API void IsolateRecorderWithGuid(FGuid InGuid, bool bIsolate);

		UE_API void AddSkelMeshCompsToTick(TArray<USkeletalMeshComponent*>& InSkelMeshCompArray);

		UE_API void AddSignedObjectsToTrack(TArray<TWeakObjectPtr<UMovieSceneSignedObject>>& InSignedObjects, TSharedPtr<ISequencerBakeRecorder>& Owner);
		UE_API void RemoveSignedObjectsFromTrack(TArray<TWeakObjectPtr<UMovieSceneSignedObject>>& InSignedObjects, TSharedPtr<ISequencerBakeRecorder>& Owner);
	
		//ignore the last change, used mostly for when we isolate the tracks we don't want to trigger a re-baking
		UE_API void IgnoreLastChange();
		
		TWeakPtr<ISequencer> GetWeakSequencer() const { return WeakSequencer; }

		//finds the recorder by it's signature of arguments
		template<typename... Args>
		TSharedPtr<ISequencerBakeRecorder>* FindRecorder(Args&&... InArgs) {

			uint32 HashValue = (GetTypeHash(InArgs) + ...);

			return Recorders.Find(HashValue);
		}

	public:

		//public delegates folks can subscribe for when recorders are added or removed
		FOnAddBakeRecorder OnAddBakeRecorder;
		FOnRemoveBakeRecorder OnRemoveBakeRecorder;
	private:

		//delegates we subscribe
		void OnPreNavigateToSequenceChanged();
		void OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID);
		void OnMovieSceneBindingsChanged();
		FDelegateHandle OnPreNavigateToSequenceChangedHandle;
		FDelegateHandle OnActivateSequenceChangedHandle;
		FDelegateHandle OnMovieSceneBindingsChangedHandle;

		//called when we activate a new sequence
		void SetUpBakeRecorders();

	private:

		//internal struct to set up time range based upon sequencer frame range
		struct FBakeInterval
		{
			FBakeInterval() = default;

			bool SetFromSequencer(ISequencer* InSequencer);
			void Clear();
			bool IsFrameCalcuated(int32 Index);
			void SetFrameCalculated(int32 Index);

			//will return INDEX_NONE if no more to calculate
			int32 NextFrameToCalculate();
			bool IsDoneCalculating() const;

			UE::AIE::FBakeTimeRange TimeRange;
			FFrameNumber DelayBeforeStart = FFrameNumber(0);
			FFrameNumber WarmupFrames = FFrameNumber(0);

		private:
			TBitArray<> CalculatedFrames;

		};
		
	private:
		//internal objects that contains a movie scene signed object(track) last guid and the bakers that reference
		//we keep of these guids and if the LastValidMovieSceneGuid changes we see which of these track guids have changed.
		//this was we only re-bake what's needed.
		struct FSignedObjectState
		{
			TSet<TSharedPtr<ISequencerBakeRecorder>> Owners;
			FGuid Guid;
			TArray<bool> DisabledArray; //if it's a track we keep track of this since if it get's enabled/disabled we ignore the change
		};
		struct FSignedObjectChanges
		{
			void AddObject(UMovieSceneSignedObject* InObject, TSharedPtr<ISequencerBakeRecorder>& Owner);
			void RemoveObject(UMovieSceneSignedObject* InObject, TSharedPtr<ISequencerBakeRecorder>& Owner);
			void AddIgnoreObject(UMovieSceneSignedObject* InObject) { if (InObject) { IgnoredObjects.Add(InObject); } };
			void RemoveIgnoreObject(UMovieSceneSignedObject* InObject) { if (InObject) { IgnoredObjects.Remove(InObject); } };

			//returns list of recorders that have changed
			TArray<TSharedPtr<ISequencerBakeRecorder>> UpdateSignatures();
			TMap<TWeakObjectPtr<UMovieSceneSignedObject>, FSignedObjectState>  ObjectSignatures;
			TSet <TWeakObjectPtr<UMovieSceneSignedObject>> IgnoredObjects;
		};

		bool CheckForChanges();
		FGuid LastValidMovieSceneGuid;
	
private:

		//sequencer and it's interval to bake
		TWeakPtr<ISequencer> WeakSequencer;
		FBakeInterval  BakeInterval;

		//contains full set of options e.g. if one anim sequence has delay frame others will also
		FRecorderOptions RecorderOptions;

		//Registered Recorders
		//uses hash as key
		TMap<uint32, TSharedPtr<ISequencerBakeRecorder>> Recorders;

		//Recorders that are baking
		TSet<TSharedPtr<ISequencerBakeRecorder>> BakingRecorders;
		TArray<TSharedPtr<ISequencerBakeRecorder>>  LastBakedRecorders;

		//skel meshes to tick that comes from the recorders
		TSet<TWeakObjectPtr<USkeletalMeshComponent>> SkeletalMeshCompsTick;

		//tracked objecs for guid changes
		FSignedObjectChanges  SignedObjectsToTrack;

	private:
		//cached items during the bake
		TArray<IMovieSceneToolsAnimationBakeHelper*> BakeHelpers;
		FMovieSceneInverseSequenceTransform LocalToRootTransform;
		//this is cached during set of functions
		UWorld* World;
	
private:

		//functions used during the Bake
		bool bNeedsBaking = false;
		bool bIsBaking = false;
		void BakeStarted();
		void BakeFrame();
		void BakeCancelled();
		void BakeFinished();
		void BakeFramesTillDone();
		bool TickFrameInternal(const FFrameNumber& FrameNumber, float DeltaTime);

	private:

		//used when we navigate to another sequence, we isolate and turn on the baked items for performance
		void IsolateBakedTracks(TArray<TSharedPtr<ISequencerBakeRecorder>>& InRecorders, bool bIsolate);
	};

};

#undef UE_API




