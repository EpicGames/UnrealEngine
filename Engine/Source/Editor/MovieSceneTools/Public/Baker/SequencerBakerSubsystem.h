// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "Editor/UnrealEdTypes.h"
#include "SequencerBakerSubsystem.generated.h"

class ISequencer;
namespace UE::Sequencer
{
	class FSequencerBaker;
}

#define UE_API MOVIESCENETOOLS_API

/**
* USequencerBakeSubsystem
* 
* Simple Editor Subsystem that acts like a Singleton, that holds a Sequencer Baker object for each open ISequencer object.
*/
UCLASS(MinimalAPI)
class USequencerBakeSubsystem :
	public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UE_API void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);

	UE_API void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	UE_API TSharedPtr<UE::Sequencer::FSequencerBaker> GetSequencerBaker(const TSharedPtr<ISequencer>& InSequencer);
private:

	FDelegateHandle OnSequencerCreatedHandle;

	//list of created sequencers
	TMap<TWeakPtr<ISequencer>, TSharedPtr<UE::Sequencer::FSequencerBaker>> Sequencers;

};

#undef UE_API
