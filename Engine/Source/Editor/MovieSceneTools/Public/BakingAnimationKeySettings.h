// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BakingAnimationKeySettings.generated.h"


UENUM(Blueprintable)
enum class EBakingKeySettings : uint8
{
	KeysOnly UMETA(DisplayName = "Keys Only"),
	AllFrames UMETA(DisplayName = "All Frames"),
};

USTRUCT(BlueprintType)
struct FBakingAnimationKeySettings
{
	GENERATED_BODY();

	FBakingAnimationKeySettings()
	{
		StartFrame = 0;
		EndFrame = 100;
		BakingKeySettings = EBakingKeySettings::KeysOnly;
		FrameIncrement = 1;
		bReduceKeys = false;
		Tolerance = 0.001f;
		bTimeWarp = false;
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake")
	FFrameNumber StartFrame;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake")
	FFrameNumber EndFrame;

	/** Bake on keyed frames only or bake all frames between start and end */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake")
	EBakingKeySettings BakingKeySettings;

	/** Frames to increment when baking */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (ClampMin = "1", UIMin = "1", EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	int32 FrameIncrement;

	/** Reduce keys after baking */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	bool bReduceKeys;

	/** Tolerance to use when reducing keys */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames || bReduceKeys"))
	float Tolerance;

	/** Bake with time warp applied. If there is time warp on the sequence, you will need to disable it manually after baking, otherwise times will be warped twice. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake")
	bool bTimeWarp;
};
