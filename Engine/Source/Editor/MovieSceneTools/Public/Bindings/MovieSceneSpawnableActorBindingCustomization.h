// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "MovieSceneSpawnableBindingCustomization.h"

class IDetailLayoutBuilder;
class ISequencer;
class IPropertyHandle; 
class STextComboBox;
namespace ESelectInfo { enum Type : int; }

class FMovieSceneSpawnableActorBindingBaseCustomization : public FMovieSceneSpawnableBindingCustomization
{
public:

	FMovieSceneSpawnableActorBindingBaseCustomization(TWeakPtr<ISequencer> InSequencer, UMovieScene* InMovieScene, FGuid InBindingGuid)
		: FMovieSceneSpawnableBindingCustomization(InMovieScene, InBindingGuid)
		, SequencerPtr(InSequencer)
	{}
	static MOVIESCENETOOLS_API TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<ISequencer> InSequencer, UMovieScene* InMovieScene, FGuid InBindingGuid);
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	TWeakPtr<ISequencer> SequencerPtr;

	TSharedPtr<IPropertyHandle> SpawnLevelProperty;

	// level names
	TSharedPtr<STextComboBox> LevelNameComboBox;
	TArray< TSharedPtr< FString > > LevelNameComboListItems;
	TArray<FName> LevelNameList;
	FName LevelNameComboSelectedName;

	void OnLevelNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void RefreshComboList();

};
