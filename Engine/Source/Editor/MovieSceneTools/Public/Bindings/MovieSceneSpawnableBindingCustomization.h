// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UMovieScene;

class FMovieSceneSpawnableBindingCustomization : public IDetailCustomization
{
public:

	FMovieSceneSpawnableBindingCustomization(UMovieScene* InMovieScene, FGuid InBindingGuid)
		: MovieScene(InMovieScene)
		, BindingGuid(InBindingGuid)
	{}

	static MOVIESCENETOOLS_API TSharedRef<IDetailCustomization> MakeInstance(UMovieScene* InMovieScene, FGuid InBindingGuid);
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;


private:

	void OnSpawnOwnershipChanged();
	TObjectPtr<UMovieScene> MovieScene = nullptr;
	FGuid BindingGuid;

	TSharedPtr<IPropertyHandle> SpawnOwnershipProperty;
};
