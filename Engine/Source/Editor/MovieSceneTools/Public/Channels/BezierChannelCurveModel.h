// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/ChannelCurveModel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "IBufferedCurveModel.h"
#include "Misc/NotNull.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class ISequencer;
class UMovieSceneSection;
struct FCurveAttributes;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyPosition;

namespace UE::MovieSceneTools { template<typename ChannelType> class FMovieSceneCachedCurve; }

/**
 * Buffered curve implementation for a bezier curve model, stores a copy of the bezier curve channel in order to draw itself.
 */
template<typename ChannelType>
class FBezierChannelBufferedCurveModel : public IBufferedCurveModel
{
public:
	/** Create a copy of the channel while keeping the reference to the section */
	FBezierChannelBufferedCurveModel(const ChannelType* InChannel, TWeakObjectPtr<UMovieSceneSection> InWeakSection,
		TArray<FKeyPosition>&& InKeyPositions, TArray<FKeyAttributes>&& InKeyAttributes, const FString& InLongDisplayName, const double InValueMin, const double InValueMax);

	virtual void DrawCurve(const FCurveEditor& InCurveEditor, const FCurveEditorScreenSpace& InScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override;
	virtual bool Evaluate(double InTime, double& OutValue) const override;

private:
	ChannelType Channel;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};

/**
 * Implementation of a bezier curve model.
 */
template<typename ChannelType, typename ChannelValue, typename KeyType> 
class FBezierChannelCurveModel : public FChannelCurveModel<ChannelType, ChannelValue, KeyType>
{
	using Super = FChannelCurveModel<ChannelType, ChannelValue, KeyType>;

public:
	FBezierChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);
	FBezierChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	virtual UE::CurveEditor::ICurveEditorCurveCachePool* DrawCurveToCachePool(const TSharedRef<FCurveEditor>& CurveEditor, const UE::CurveEditor::FCurveDrawParamsHandle& CurveDrawParamsHandle, const FCurveEditorScreenSpace& ScreenSpace) override;
	virtual bool HasChangedAndResetTest() override;
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;

	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void GetKeyAttributesExcludingAutoComputed(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;

	virtual TPair<ERichCurveInterpMode, ERichCurveTangentMode> GetInterpolationMode(const double& InTime, ERichCurveInterpMode DefaultInterpolationMode, ERichCurveTangentMode DefaultTangentMode) const override;
	virtual void GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const override;
	virtual void SetCurveAttributes(const FCurveAttributes& InCurveAttributes) override;
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	virtual void GetValueRange(double InMinTime, double InMaxTime, double& MinValue, double& MaxValue) const override;

	virtual bool HasDefaultValue() const override;
	virtual double GetDefaultValue() const override;

protected:
	// FChannelCurveModel
	virtual double GetKeyValue(TArrayView<const ChannelValue> Values, int32 Index) const override;
	virtual void SetKeyValue(int32 Index, double KeyValue) const override;

	TSharedPtr<UE::MovieSceneTools::FMovieSceneCachedCurve<ChannelType>> CachedCurve;

	/**
	 * Updates all dependent systems after a change has been made to owning object.
	 * Calls MarkAsChanged, dirties the package, and FCurveModel::OnCurvesModified.
	 */
	void OnPostOwnerChanged(TNotNull<UMovieSceneSignedObject*> InSignedOwner);

private:
	void FeaturePointMethod(double StartTime, double EndTime, double StartValue, double Mu, int Depth, int MaxDepth, double& MaxV, double& MinVal) const;
};

