// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/ChannelCurveModel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "IBufferedCurveModel.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IBufferedCurveModel;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FKeyHandle;
struct FMovieSceneBoolChannel;
template <typename ChannelType> struct TMovieSceneChannelHandle;

class FBoolChannelCurveModel : public FChannelCurveModel<FMovieSceneBoolChannel, bool, bool>
{
public:
	FBoolChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneBoolChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;
	virtual void GetCurveAttributes(FCurveAttributes& OutAttributes) const override;
	virtual void SetCurveAttributes(const FCurveAttributes& InAttributes) override;

protected:

	// FChannelCurveModel
	virtual double GetKeyValue(TArrayView<const bool> Values, int32 Index) const override;
	virtual void SetKeyValue(int32 Index, double KeyValue) const override;
};