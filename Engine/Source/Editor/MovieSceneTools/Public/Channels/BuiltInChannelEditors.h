// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"

#include "MovieSceneKeyStruct.h"
#include "SequencerChannelTraits.h"
#include "Channels/MovieSceneChannelHandle.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneEventChannel.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Sections/MovieSceneStringSection.h"
#include "Channels/MovieSceneTextChannel.h"
#include "Sections/MovieSceneParticleSection.h"
#include "Sections/MovieSceneActorReferenceSection.h"

struct FKeyHandle;
struct FKeyDrawParams;

class SWidget;
class ISequencer;
class FMenuBuilder;
class FStructOnScope;
class ISectionLayoutBuilder;

/** Overrides for adding or updating a key for non-standard channels */
MOVIESCENETOOLS_API FKeyHandle AddOrUpdateKey(FMovieSceneDoubleChannel* Channel, UMovieSceneSection* SectionToKey,  const TMovieSceneExternalValue<double>& EditorData, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings);
MOVIESCENETOOLS_API FKeyHandle AddOrUpdateKey(FMovieSceneFloatChannel* Channel, UMovieSceneSection* SectionToKey,  const TMovieSceneExternalValue<float>& EditorData, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings);
MOVIESCENETOOLS_API FKeyHandle AddOrUpdateKey(FMovieSceneActorReferenceChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings);
MOVIESCENETOOLS_API FKeyHandle AddOrUpdateKey(FMovieSceneTimeWarpChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings);

/** Key editor overrides */
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneBoolChannel*       Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneByteChannel*       Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneIntegerChannel*    Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneDoubleChannel*      Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneFloatChannel*      Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneStringChannel*     Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneTextChannel*       Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneObjectPathChannel* Channel);
MOVIESCENETOOLS_API bool CanCreateKeyEditor(const FMovieSceneActorReferenceChannel* Channel);

MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneBoolChannel>&        Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneByteChannel>&        Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneIntegerChannel>&     Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>&      Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneFloatChannel>&       Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneStringChannel>&      Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneTextChannel>&        Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneObjectPathChannel>&  Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneActorReferenceChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);
MOVIESCENETOOLS_API TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>&    Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);

MOVIESCENETOOLS_API UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneByteChannel* Channel,       FSequencerKeyStructGenerator* Generator);
MOVIESCENETOOLS_API UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneTimeWarpChannel* Channel,   FSequencerKeyStructGenerator* Generator);
MOVIESCENETOOLS_API UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneTextChannel* Channel,       FSequencerKeyStructGenerator* Generator);
MOVIESCENETOOLS_API UMovieSceneKeyStructType* InstanceGeneratedStruct(FMovieSceneObjectPathChannel* Channel, FSequencerKeyStructGenerator* Generator);

MOVIESCENETOOLS_API void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct);
MOVIESCENETOOLS_API void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneObjectPathChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct);
MOVIESCENETOOLS_API void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct);
MOVIESCENETOOLS_API void PostConstructKeyInstance(const TMovieSceneChannelHandle<FMovieSceneTextChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct);

/** Key drawing overrides */
MOVIESCENETOOLS_API void DrawKeys(FMovieSceneDoubleChannel*   Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);
MOVIESCENETOOLS_API void DrawKeys(FMovieSceneFloatChannel*    Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);
MOVIESCENETOOLS_API void DrawKeys(FMovieSceneParticleChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);
MOVIESCENETOOLS_API void DrawKeys(FMovieSceneEventChannel*    Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);

/** Context menu overrides */
MOVIESCENETOOLS_API void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneDoubleChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneFloatChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneIntegerChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneBoolChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneByteChannel>>&& Channels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);

MOVIESCENETOOLS_API TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneDoubleChannel>>&& InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneFloatChannel>>&& InChannels,const  TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneIntegerChannel>>&& InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneBoolChannel>>&& InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);
MOVIESCENETOOLS_API TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TMovieSceneChannelHandle<FMovieSceneByteChannel>>&& InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer);

MOVIESCENETOOLS_API void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneDoubleChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer);
MOVIESCENETOOLS_API void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneFloatChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer);
MOVIESCENETOOLS_API void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<FMovieSceneTimeWarpChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer);

/** Curve editor models */
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>& DoubleChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneFloatChannel>& FloatChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneIntegerChannel>& IntegerChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneBoolChannel>& BoolChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneByteChannel>& ByteChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneEventChannel>& EventChannel) { return true; }
MOVIESCENETOOLS_API inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& TimeWarpChannel) { return true; }

MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneDoubleChannel>& DoubleChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneFloatChannel>& FloatChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneIntegerChannel>& IntegerChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneBoolChannel>& BoolChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneByteChannel>& ByteChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneEventChannel>& EventChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);
MOVIESCENETOOLS_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& TimeWarpChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);

MOVIESCENETOOLS_API bool ShouldShowCurve(const FMovieSceneFloatChannel* Channel, UMovieSceneSection* InSection);
MOVIESCENETOOLS_API bool ShouldShowCurve(const FMovieSceneDoubleChannel* Channel, UMovieSceneSection* InSection);

MOVIESCENETOOLS_API TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel(const TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel>& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName);
