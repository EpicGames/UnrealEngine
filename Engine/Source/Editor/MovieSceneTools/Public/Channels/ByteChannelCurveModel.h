// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/ChannelCurveModel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "IBufferedCurveModel.h"
#include "Templates/UniquePtr.h"

class IBufferedCurveModel;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FKeyHandle;
struct FMovieSceneByteChannel;
template <typename ChannelType> struct TMovieSceneChannelHandle;

class FByteChannelCurveModel : public FChannelCurveModel<FMovieSceneByteChannel, uint8, uint8>
{
public:
	FByteChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneByteChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;
	virtual void GetCurveAttributes(FCurveAttributes& OutAttributes) const override;
	virtual void SetCurveAttributes(const FCurveAttributes& InAttributes) override;
protected:

	// FChannelCurveModel
	virtual double GetKeyValue(TArrayView<const uint8> Values, int32 Index) const override;
	virtual void SetKeyValue(int32 Index, double KeyValue) const override;

};