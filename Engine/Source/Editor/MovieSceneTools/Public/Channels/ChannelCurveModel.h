// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Curves/KeyHandle.h"
#include "Delegates/IDelegateInstance.h"
#include "IBufferedCurveModel.h"
#include "Misc/NotNull.h"
#include "Misc/OptionalFwd.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Misc/Guid.h"

class FCurveEditor;
class FString;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FCurveAttributes;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyPosition;

template <class ChannelType, class ChannelValue, class KeyType>
class FChannelCurveModel : public FCurveModel
{
public:
	FChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);
	FChannelCurveModel(TMovieSceneChannelHandle<ChannelType> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer);

	~FChannelCurveModel();

	virtual const void* GetCurve() const override;

	virtual void Modify() override;
	virtual void OnCloseRootChangeScope() override;

	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& OutInterpolatingPoints) const override;
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override;
	virtual void GetAllKeys(TArray<FKeyHandle>& OutKeyHandles) const override;
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;

	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override;
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;

	virtual void GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const override;
	virtual void GetTimeRange(double& MinTime, double& MaxTime) const override;
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	virtual int32 GetNumKeys() const override;
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override;
	virtual bool Evaluate(double ProspectiveTime, double& OutValue) const override;
	
	virtual void AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override;

	virtual void ReplaceKeyHandles(TConstArrayView<FKeyHandle> InCurrentHandles, TConstArrayView<FKeyHandle> InNewHandles) override;

	virtual FTransform2d GetCurveTransform() const override;
	virtual bool IsReadOnly() const override;
	virtual UObject* GetOwningObject() const override
	{
		if (UObject* OwningObject = WeakOwningObject.Get())
		{
			return OwningObject;
		}
		return WeakSection.Get();
	}
	virtual bool HasChangedAndResetTest() override
	{
		UMovieSceneSignedObject* SignedOwner = Cast<UMovieSceneSignedObject>(WeakOwningObject.Get());
		if (!SignedOwner)
		{
			SignedOwner = WeakSection.Get();
		}

		if (SignedOwner)
		{
			if (SignedOwner->GetSignature() != LastSignature)
			{
				LastSignature = SignedOwner->GetSignature();
				return true;
			}
			return false;
		}
		return true;
	}

	virtual void GetCurveColorObjectAndName(UObject** OutObject, FString& OutName) const override;

protected:

	virtual double GetKeyValue(TArrayView<const ChannelValue> Values, int32 Index) const = 0;
	virtual void SetKeyValue(int32 Index, double KeyValue) const = 0;

	/** Performs actions that should take place after any mutating call, like SetKeyPositions, etc. */
	void PostProcessChanges(TNotNull<ChannelType*> Channel, TNotNull<UMovieSceneSection*> Section);
	
	/**
	 * Updates all dependent systems after a change has been made to the channel, namely:
	 *	- Invalidate LastSignature,
	 *	- ChannelType::PostEditChange,
	 *	- ChannelType::OnChannelChanged(),
	 *	- UMovieSceneSection::MarkAsChanged,
	 *	- MarkPackageDirty: this is required so the user is prompted to save the package after editing keys. If currently recording a transaction,
	 *	the generated transaction will undo this flag when the user performs an undo op.
	 *	- FCurveModel::OnCurvesModified
	 */
	void InvokePostChannelChangedEvents(TNotNull<UMovieSceneSection*> Section);

public:

	const TMovieSceneChannelHandle<ChannelType>& GetChannelHandle() const  { return ChannelHandle; } 

private:

	void FixupCurve();

protected:

	TMovieSceneChannelHandle<ChannelType> ChannelHandle;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TWeakObjectPtr<> WeakOwningObject;
	TWeakPtr<ISequencer> WeakSequencer;
	FDelegateHandle OnDestroyHandle;
	FGuid LastSignature;
};