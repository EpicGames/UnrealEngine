// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/BezierChannelCurveModel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Containers/ArrayView.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class IBufferedCurveModel;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FKeyHandle;
struct FMovieSceneDoubleChannel;
struct FMovieSceneDoubleValue;
template <typename ChannelType> struct TMovieSceneChannelHandle;

class FDoubleChannelCurveModel : public FBezierChannelCurveModel<FMovieSceneDoubleChannel, FMovieSceneDoubleValue, double>
{
public:
	FDoubleChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneDoubleChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);
	FDoubleChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneDoubleChannel> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;

	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;
	virtual void GetValueRange(double InMinTime, double InMaxTime, double& MinValue, double& MaxValue) const override;
};
