// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/BezierChannelCurveModel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/ArrayView.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class IBufferedCurveModel;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FKeyHandle;
struct FMovieSceneFloatChannel;
struct FMovieSceneFloatValue;
template <typename ChannelType> struct TMovieSceneChannelHandle;

class FFloatChannelCurveModel : public FBezierChannelCurveModel<FMovieSceneFloatChannel, FMovieSceneFloatValue, float>
{
public:
	FFloatChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneFloatChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;
};