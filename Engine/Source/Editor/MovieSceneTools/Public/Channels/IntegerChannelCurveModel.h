// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/ChannelCurveModel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "IBufferedCurveModel.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IBufferedCurveModel;
class ISequencer;
class UMovieSceneSection;
class UObject;
struct FKeyHandle;
struct FMovieSceneIntegerChannel;
template <typename ChannelType> struct TMovieSceneChannelHandle;

class FIntegerChannelCurveModel : public FChannelCurveModel<FMovieSceneIntegerChannel, int32, int32>
{
public:
	FIntegerChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneIntegerChannel> InChannel, UMovieSceneSection* InOwningSection, TWeakPtr<ISequencer> InWeakSequencer);

	// FCurveModel
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	virtual void CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects) override;
	virtual TUniquePtr<IBufferedCurveModel> CreateBufferedCurveCopy() const override;
	virtual void GetCurveAttributes(FCurveAttributes& OutAttributes) const override;
	virtual void SetCurveAttributes(const FCurveAttributes& InAttributes) override;
protected:

	// FChannelCurveModel
	virtual double GetKeyValue(TArrayView<const int32> Values, int32 Index) const override;
	virtual void SetKeyValue(int32 Index, double KeyValue) const override;

};