// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Channels/MovieSceneDoublePerlinNoiseChannelContainer.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannelContainer.h"
#include "Styling/SlateTypes.h"

class IDetailLayoutBuilder;

class FMovieSceneFloatPerlinNoiseChannelDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static FName GetPerlinNoiseChannelParamsPropertyName();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

class FMovieSceneDoublePerlinNoiseChannelDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static FName GetPerlinNoiseChannelParamsPropertyName();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

