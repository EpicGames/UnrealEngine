// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveKeyEditors/SequencerKeyEditor.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ISequencerChannelInterface.h"
#include "Misc/NotifyHook.h"
#include "MovieSceneSection.h"
#include "TimeToPixel.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class FMenuBuilder;

template<typename ChannelType, typename NumericType>
class SNumericTextBlockKeyEditor : public SCompoundWidget
{
	using FKeyEditor = TSequencerKeyEditor<ChannelType, NumericType>;
	
	FKeyEditor KeyEditor;
	TSharedPtr<INumericTypeInterface<NumericType>> Interface;

	SLATE_BEGIN_ARGS(SNumericTextBlockKeyEditor){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FKeyEditor& InKeyEditor)
	{
		KeyEditor = InKeyEditor;
		Interface = MakeShareable(new TDefaultNumericTypeInterface<NumericType>);

		ChildSlot
		[
			SNew(STextBlock)
			.Text(this, &SNumericTextBlockKeyEditor<ChannelType, NumericType>::GetValueAsText)
		];
	}

	FText GetValueAsText() const
	{
		const FString ValueString = Interface->ToString(KeyEditor.GetCurrentValue());
		return FText::FromString(ValueString);
	}
};

struct FPerlinNoiseChannelSectionMenuExtension : TSharedFromThis<FPerlinNoiseChannelSectionMenuExtension>, ISidebarChannelExtension
{
	FPerlinNoiseChannelSectionMenuExtension(TArrayView<const FMovieSceneChannelHandle> InChannelHandles, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections);
	virtual ~FPerlinNoiseChannelSectionMenuExtension() = default;

	virtual TSharedPtr<ISidebarChannelExtension> ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu) override;

private:

	void Initialize();

	void BuildChannelsMenu(FMenuBuilder& MenuBuilder);
	void BuildParametersMenu(FMenuBuilder& MenuBuilder, int32 ChannelHandleIndex);

private:

	struct FChannelNotifyHook : FNotifyHook
	{
		TWeakObjectPtr<UObject> WeakObjectToModify;

		FChannelNotifyHook(UObject* InObjectToModify) : WeakObjectToModify(InObjectToModify) {}

		virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	};

	TArray<FMovieSceneChannelHandle> ChannelHandles;
	TArray<int32> ChannelHandleSectionIndexes;
	TArray<TWeakObjectPtr<UMovieSceneSection>> WeakSections;
	TArray<FChannelNotifyHook> NotifyHooks;
};

template<typename ChannelContainerType>
struct TPerlinNoiseChannelInterface : ISequencerChannelInterface
{
	using ChannelType = typename ChannelContainerType::ChannelType;

	virtual FKeyHandle AddOrUpdateKey_Raw(FMovieSceneChannel* Channel, UMovieSceneSection* SectionToKey, const void* ExtendedEditorData, FFrameNumber InTime, ISequencer& InSequencer, const FGuid& ObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings) const override
	{
		return FKeyHandle::Invalid();
	}

	virtual void DeleteKeys_Raw(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime) const override
	{
	}

	virtual void CopyKeys_Raw(FMovieSceneChannel* Channel, const UMovieSceneSection* Section, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> KeyMask) const override
	{
	}

	virtual void PasteKeys_Raw(FMovieSceneChannel* Channel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys) const override
	{
	}

	virtual TSharedPtr<FStructOnScope> GetKeyStruct_Raw(FMovieSceneChannelHandle Channel, FKeyHandle KeyHandle) const override
	{
		return nullptr;
	}

	virtual bool CanCreateKeyEditor_Raw(const FMovieSceneChannel* Channel) const override
	{
		return true;
	}

	virtual TSharedRef<SWidget> CreateKeyEditor_Raw(const FMovieSceneChannelHandle& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params) const override
	{
		const TMovieSceneExternalValue<typename ChannelType::CurveValueType>* ExternalValue = Channel.Cast<ChannelType>().GetExtendedEditorData();
		if (!ExternalValue)
		{
			return SNullWidget::NullWidget;
		}

		TSequencerKeyEditor<ChannelType, typename ChannelType::CurveValueType> KeyEditor(
			Params.ObjectBindingID, Channel.Cast<ChannelType>(),
			Params.OwningSection, Params.Sequencer, Params.PropertyBindings, ExternalValue->OnGetExternalValue
			);

		using KeyEditorType = SNumericTextBlockKeyEditor<ChannelType, typename ChannelType::CurveValueType>;
		return SNew(KeyEditorType, KeyEditor);
	}

	virtual void ExtendKeyMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TArrayView<const FExtendKeyMenuParams> Parameters, TWeakPtr<ISequencer> InSequencer) const override
	{
	}

	virtual void ExtendSectionMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArrayView<const FMovieSceneChannelHandle> InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer) const override
	{
		TSharedRef<FPerlinNoiseChannelSectionMenuExtension> Extension = MakeShared<FPerlinNoiseChannelSectionMenuExtension>(InChannels, InWeakSections);

		InMenuExtender->AddMenuExtension(TEXT("SequencerChannels"), EExtensionHook::First, nullptr
			, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& MenuBuilder)
				{
					Extension->ExtendMenu(MenuBuilder, true);
				}));
	}

	virtual TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> InMenuExtender, TArrayView<const FMovieSceneChannelHandle> InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer) const override
	{
		const TSharedRef<FPerlinNoiseChannelSectionMenuExtension> Extension = MakeShared<FPerlinNoiseChannelSectionMenuExtension>(InChannels, InWeakSections);
		Extension->ExtendMenu(MenuBuilder, false);
		return Extension;
	}

	virtual void DrawKeys_Raw(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams) const override
	{
	}

	virtual bool ShouldShowCurve_Raw(const FMovieSceneChannel* Channel, UMovieSceneSection* InSection) const override
	{
		return true;
	}

	virtual bool SupportsCurveEditorModels_Raw(const FMovieSceneChannelHandle& InChannel) const override
	{
		return false;
	}

	virtual TUniquePtr<FCurveModel> CreateCurveEditorModel_Raw(const FMovieSceneChannelHandle& Channel, const UE::Sequencer::FCreateCurveEditorModelParams& Params) const override
	{
		return nullptr;
	}

	virtual int32 DrawExtra_Raw(FMovieSceneChannel* InChannel, const UMovieSceneSection* InOwner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const override
	{
		using namespace UE::Sequencer;

		FLinearColor FillColor(1, 1, 1, 0.334f);
		ChannelType* TypedChannel = static_cast<ChannelType*>(InChannel);

		TArray<FVector2D> CurvePoints;
		CurvePoints.Reserve(PaintArgs.Geometry.Size.X / 2.0);

		const double Amplitude = TypedChannel->PerlinNoiseParams.Amplitude;
		const double YOffset = PaintArgs.Geometry.Size.Y / 2.0;
		const double YScale = (Amplitude != 0) ? (PaintArgs.Geometry.Size.Y / Amplitude / 2.0) : 1;

		for (double X = 0; X < PaintArgs.Geometry.Size.X; X += 2.f)
		{
			double Seconds = PaintArgs.TimeToPixel.PixelToSeconds(X);
			double Y = (double)TypedChannel->Evaluate(Seconds);
			CurvePoints.Add(FVector2D(X, YOffset - (Y * YScale)));
		}

		FSlateDrawElement::MakeLines(
				PaintArgs.DrawElements,
				LayerId,
				PaintArgs.Geometry.ToPaintGeometry(),
				CurvePoints,
				ESlateDrawEffect::PreMultipliedAlpha,
				FillColor,
				true);

		return LayerId + 1;
	}

	virtual TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel_Raw(const FMovieSceneChannelHandle& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName) const override
	{
		return nullptr;
	}
	virtual TSharedPtr<UE::Sequencer::STrackAreaLaneView> CreateChannelView_Raw(const FMovieSceneChannelHandle& InChannelHandle, TWeakPtr<UE::Sequencer::FChannelModel> InWeakModel, const UE::Sequencer::FCreateTrackLaneViewParams& Parameters) const override
	{
		return nullptr;
	}
	virtual EMovieSceneKeyInterpolation GetInterpolationMode_Raw(FMovieSceneChannel* InChannel, FFrameNumber InTime, EMovieSceneKeyInterpolation InDefaultMode) const override
	{
		return InDefaultMode;
	}
};

