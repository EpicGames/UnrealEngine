// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveModel.h"
#include "Misc/Attribute.h"

namespace UE::MovieScene
{

struct FPiecewiseCurve;

class FPiecewiseCurveModel : public FCurveModel
{
public:

	TAttribute<const UE::MovieScene::FPiecewiseCurve*> CurveAttribute;
	TAttribute<FFrameRate> FrameRateAttribute;
	TAttribute<FTransform2d> CurveTransformAttribute;

	FPiecewiseCurveModel()
	{
		Color = FLinearColor(0.1f,0.1f,0.1f);
	}

	// FCurveModel
	const void* GetCurve() const override
	{
		return nullptr;
	}
	void Modify() override
	{
	}
	void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override
	{
	}
	void AddKeys(TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override
	{
	}
	void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override
	{
	}
	void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override
	{
	}
	void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override
	{
	}
	void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override
	{
	}
	int32 GetNumKeys() const override
	{
		return 0;
	}
	void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override
	{
	}

	bool IsReadOnly() const override { return true; }
	FTransform2d GetCurveTransform() const override { return CurveTransformAttribute.Get(); }
	bool Evaluate(double InTime, double& OutValue) const override;
	void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	void GetTimeRange(double& MinTime, double& MaxTime) const override;
	void GetValueRange(double& MinValue, double& MaxValue) const override;
};


} // namespace UE::MovieScene