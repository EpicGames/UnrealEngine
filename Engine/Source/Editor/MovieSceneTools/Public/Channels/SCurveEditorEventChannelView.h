// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Rendering/RenderingCommon.h"
#include "Templates/SharedPointer.h"
#include "Views/SInteractiveCurveEditorView.h"

#define UE_API MOVIESCENETOOLS_API

class FCurveEditor;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
struct FGeometry;

class SCurveEditorEventChannelView : public SInteractiveCurveEditorView
{
public:

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

	UE_API virtual void GetGridLinesY(TSharedRef<const FCurveEditor> CurveEditor, TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const override;

private:

	//~ SWidget Interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	//~ SInteractiveCurveEditorView interface
	UE_API virtual void PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual bool IsValueSnapEnabled() const override { return false; }

	UE_API void DrawLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const;

	static UE_API float TrackHeight;
};

#undef UE_API
