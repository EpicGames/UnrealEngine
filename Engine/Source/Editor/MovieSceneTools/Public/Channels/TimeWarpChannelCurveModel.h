// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/DoubleChannelCurveModel.h"

struct FMovieSceneTimeWarpChannel;

class FTimeWarpChannelCurveModel : public FDoubleChannelCurveModel
{
public:

	FTimeWarpChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneTimeWarpChannel> InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject, TWeakPtr<ISequencer> InWeakSequencer);

	void SetCurveAttributes(const FCurveAttributes& InCurveAttributes) override;

	void GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const override;

	void AllocateAxes(FCurveEditor* InCurveEditor, TSharedPtr<FCurveEditorAxis>& OutHorizontalAxis, TSharedPtr<FCurveEditorAxis>& OutVerticalAxis) const override;

	void MakeChildCurves(TArray<TUniquePtr<FCurveModel>>& OutChildCurves) const override;

	// Time warp getters are parented to UMovieSceneSequence (not a UMovieSceneSection), so the
	// default outer-chain walk fails to find the section. When a caller asks for a section (or
	// any of its bases) we return our directly-stored section instead.
	virtual UObject* GetOwningObjectOrOuter(UClass* Class) const override;
};