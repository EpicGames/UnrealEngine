// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#if WITH_EDITOR

#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneEvent.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "Channels/MovieSceneCameraShakeSourceTriggerChannel.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/MovieSceneActorReferenceSection.h"
#include "MovieSceneClipboard.h"

namespace MovieSceneClipboard
{
	template<> inline FName GetKeyTypeName<uint8>()
	{
		static FName Name("Byte");
		return Name;
	}
	template<> inline FName GetKeyTypeName<int32>()
	{
		static FName Name("Int");
		return Name;
	}
	template<> inline FName GetKeyTypeName<int64>()
	{
		static FName Name("Int64");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FMovieSceneFloatValue>()
	{
		static FName Name("Float");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FMovieSceneDoubleValue>()
	{
		static FName Name("Double");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FName>()
	{
		static FName Name("Name");
		return Name;
	}
	template<> inline FName GetKeyTypeName<bool>()
	{
		static FName Name("Bool");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FString>()
	{
		static FName Name("String");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FText>()
	{
		static FName Name("Text");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FMovieSceneActorReferenceKey>()
	{
		static FName Name("MovieSceneActorReferenceKey");
		return Name;
	}
	template<> inline FName GetKeyTypeName<FEventPayload>()
	{
		return "EventPayload";
	}
	template<> inline FName GetKeyTypeName<FMovieSceneObjectPathChannelKeyValue>()
	{
		return "MovieSceneObjectPathChannelKeyValue";
	}
	template<> inline FName GetKeyTypeName<FMovieSceneEvent>()
	{
		return "MovieSceneEvent";
	}
	template<> inline FName GetKeyTypeName<FMovieSceneCameraShakeSourceTrigger>()
	{
		return "FMovieSceneCameraShakeSourceTrigger";
	}
}

#endif
