// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SComboButton.h"
#include "Conditions/MovieSceneCondition.h"
#include "MovieSceneTrack.h"
#include "MovieSceneConditionCustomization.generated.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UMovieScene;
class FPropertyEditor;
class SWidget;
class FMenuBuilder;
class UMovieSceneSequence;
class UMovieSceneTrack;
class FMovieSceneDirectorBlueprintConditionCustomization;
class IDetailsView;
class ISequencer;

// Helper UObject for editing optional track row metadata not in-place. A UObject instead of a UStruct because we need to support instanced sub objects (conditions)
UCLASS(MinimalAPI, CollapseCategories)
class UMovieSceneTrackRowMetadataHelper : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "General", meta=(ShowOnlyInnerProperties))
	FMovieSceneTrackRowMetadata TrackRowMetadata;

	UPROPERTY()
	TWeakObjectPtr<UMovieSceneTrack> OwnerTrack;
};

class FMovieSceneConditionCustomization : public IPropertyTypeCustomization
{
public:

	static MOVIESCENETOOLS_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static MOVIESCENETOOLS_API TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakObjectPtr<UMovieSceneSequence> Sequence, const TWeakPtr<ISequencer> Sequencer);

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	/**
	* @return The current display value for the combo box as a string
	*/
	FText GetDisplayValueAsString() const;

	/**
	 * @return The current display value's icon, if any. Returns nullptr if we have no valid value.
	 */
	const FSlateBrush* GetDisplayValueIcon() const;

	/**
	 * Wrapper method for determining whether a class is valid for use by this property item input proxy.
	 *
	 * @param	InItem			the property window item that contains this proxy.
	 * @param	CheckClass		the class to verify
	 * @param	bAllowAbstract	true if abstract classes are allowed
	 *
	 * @return	true if CheckClass is valid to be used by this input proxy
	 */
	bool IsClassAllowed(UClass* CheckClass, bool bAllowAbstract) const;

	/**
	 * Generates a condition picker allowing choice of condition class, creation of new class, or director blueprint condition.
	 *
	 * @return The Condition Picker widget.
	 */
	TSharedRef<SWidget> GenerateConditionPicker();

	/**
	 * Callback function from the Class Picker for when a Class is picked.
	 *
	 * @param InClass			The class picked in the Class Picker
	 */
	void OnClassPicked(UClass* InClass);

	/* Fills a sub menu to create a new or pick an existing condition class */
	void FillConditionClassSubMenu(FMenuBuilder& MenuBuilder);

	/* Fills a sub menu to create a new director blueprint condition endpoint or pick an existing one */
	void FillDirectorBlueprintConditionSubMenu(FMenuBuilder& MenuBuilder);

	/* Gets the common sequence for this customization */
	UMovieSceneSequence* GetCommonSequence() const;

	/* Gets the common track for this customization */
	UMovieSceneTrack* GetCommonTrack() const;

	/**
	* Generate the content of the quick bind sub-menu dropdown (shown if the endpoint is not already bound)
	*/
	void PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder);

	/* Used by 'Use Selected' button when clicked to change the condition class to the selected condition class in the asset browser*/
	void OnUseSelected();

	/* Used to enable/disable the selected asset button based on whether a condition class is currently selected */
	bool CanUseSelectedAsset() const;

	/* Used by 'Browse To' button when clicked to browse to the condition class currently in use in the asset browser*/
	void OnBrowseTo();

	/* Used to enable/disable the browse to button based on whether a blueprint condition class is currently in use. */
	bool CanBrowseToAsset() const;

private:

	/** The property handle for the condition container */
	TSharedPtr<IPropertyHandle> ConditionContainerPropertyHandle;

	/* The property handle for the instanced condition property itself*/
	TSharedPtr<IPropertyHandle> ConditionPropertyHandle;

	/** Property utilities for the property we're editing */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/* Property editor for the condition property */
	TSharedPtr<FPropertyEditor> PropertyEditor;

	/* The combo button */
	TSharedPtr<class SComboButton> ComboButton;

	/* Hold a shared ptr to the details view to prevent it from getting destroyed before combo button actions have taken place*/
	TSharedPtr<IDetailsView> DetailsView;

	TSharedPtr<SWidget> OpenMenuWidget;

	TWeakObjectPtr<UMovieSceneSequence> Sequence;
	TWeakObjectPtr<UMovieSceneTrack> Track;

	TWeakPtr<ISequencer> Sequencer;
};
