// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneDirectorBlueprintEndpointCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Types/SlateEnums.h"

#define UE_API MOVIESCENETOOLS_API

class UK2Node;
class UBlueprint;
class UEdGraphPin;
class IPropertyHandle;
class FDetailWidgetRow;
class SWidget;

struct FMovieSceneDirectorBlueprintConditionData;

enum class ECheckBoxState : uint8;

/**
 * Customization for director blueprint condition endpoint picker
 */
class FMovieSceneDirectorBlueprintConditionCustomization : public FMovieSceneDirectorBlueprintEndpointCustomization
{
public:

	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance(UMovieScene* InMovieScene);
	static UE_API TSharedRef<FMovieSceneDirectorBlueprintConditionCustomization> MakeInstance(UMovieScene* InMovieScene, TSharedPtr<IPropertyHandle> InPropertyHandle, TSharedPtr<IPropertyUtilities> InPropertyUtilities);

protected:

	UE_API virtual void GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const override;
	UE_API virtual bool SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewVariableValue) override;
	UE_API virtual UK2Node* FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const override;
	UE_API virtual void GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutWellKnownParameters) const override;
	UE_API virtual void GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const override;
	UE_API virtual bool SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName) override;
	UE_API virtual FMovieSceneDirectorBlueprintEndpointDefinition GenerateEndpointDefinition(UMovieSceneSequence* Sequence) override;
	UE_API virtual void OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) override;
	UE_API virtual void OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) override;
	UE_API virtual void GetEditObjects(TArray<UObject*>& OutObjects) const override;
	UE_API virtual void OnCollectQuickBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder) override;
	virtual bool CreateNewCategoryForPayloadVariables() const override { return false; }
private:

	UE_API void SetEndpointImpl(UMovieScene* MovieScene, FMovieSceneDirectorBlueprintConditionData* DynamicBinding, UBlueprint* Blueprint, UK2Node* NewEndpoint);
	UE_API void EnsureBlueprintExtensionCreated(UMovieScene* MovieScene, UBlueprint* Blueprint);

	UE_API void CollectConditionBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder, bool bIsRebinding);

private:

	UMovieScene* EditedMovieScene;
};

#undef UE_API
