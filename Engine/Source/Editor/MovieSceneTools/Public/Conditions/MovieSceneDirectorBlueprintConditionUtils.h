// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieScene.h"
#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "Editor.h"
#include "MovieSceneCommonHelpers.h"
#include "Conditions/MovieSceneGroupCondition.h"

#include "MovieSceneDirectorBlueprintConditionUtils.generated.h"

#define UE_API MOVIESCENETOOLS_API

/**
 * A utility class for managing director blueprint condition endpoints.
 */
struct FMovieSceneDirectorBlueprintConditionUtils
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	/**
	 * Set an endpoint on the given blueprint condition
	 */
	static UE_API void SetEndpoint(UMovieScene* MovieScene, FMovieSceneDirectorBlueprintConditionData* DirectorBlueprintConditionData, UK2Node* NewEndpoint);

	/**
	 * Ensures that the condition blueprint extension has been added to the given sequence's director blueprint.
	 */
	static UE_API void EnsureBlueprintExtensionCreated(UMovieSceneSequence* MovieSceneSequence, UBlueprint* Blueprint);

	/**
	 * Utility function for iterating all blueprint conditions in a sequence.
	 */
	template<typename Callback>
	static void IterateDirectorBlueprintConditions(UMovieScene* InMovieScene, Callback&& InCallback)
	{
		auto IterateThroughTrack = [&InCallback](UMovieSceneTrack* Track)
		{
			TFunction<void(UMovieSceneCondition*)> IterateThroughCondition;
			IterateThroughCondition = [&InCallback, &IterateThroughCondition](UMovieSceneCondition* Condition)
			{
				if (UMovieSceneDirectorBlueprintCondition* DirectorBlueprintCondition = Cast<UMovieSceneDirectorBlueprintCondition>(Condition))
				{
					InCallback(DirectorBlueprintCondition->DirectorBlueprintConditionData);
				}
				else if (UMovieSceneGroupCondition* GroupCondition = Cast<UMovieSceneGroupCondition>(Condition))
				{
					for (const FMovieSceneConditionContainer& Container : GroupCondition->SubConditions)
					{
						if (Container.Condition)
						{
							IterateThroughCondition(Container.Condition);
						}
					}
				}
			};

			for (UMovieSceneCondition* Condition : Track->GetAllConditions())
			{
				IterateThroughCondition(Condition);
			}
		};

		TArray<UMovieSceneTrack*> Tracks = InMovieScene->GetTracks();
		for (UMovieSceneTrack* Track : Tracks)
		{
			IterateThroughTrack(Track);
		}

		if (UMovieSceneTrack* Track = InMovieScene->GetCameraCutTrack())
		{
			IterateThroughTrack(Track);
		}

		// Add all object binding sections
		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)InMovieScene)->GetBindings())
		{
			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				IterateThroughTrack(Track);
			}
		}
	}
	
	/**
	 * Utility function for gathering all blueprint conditions in a sequence into a container.
	 */
	static void GatherDirectorBlueprintConditions(UMovieScene* InMovieScene, TArray<FMovieSceneDirectorBlueprintConditionData*>& OutDirectorBlueprintConditionData)
	{
		IterateDirectorBlueprintConditions(InMovieScene, [&](FMovieSceneDirectorBlueprintConditionData& Item)
			{
				OutDirectorBlueprintConditionData.Add(&Item);
			});
	}
};

/**
 * Dummy class, used for easily getting a valid UFunction that helps prepare blueprint function graphs.
 */
UCLASS(MinimalAPI)
class UMovieSceneDirectorBlueprintConditionEndpointUtil : public UObject
{
	GENERATED_BODY()

	UFUNCTION()
	bool SampleDirectorBlueprintCondition(const FMovieSceneConditionContext& ConditionContext) const { return true; }
};

#undef UE_API
