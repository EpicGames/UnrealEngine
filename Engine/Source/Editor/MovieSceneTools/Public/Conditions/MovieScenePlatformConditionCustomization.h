// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "PropertyHandle.h"

#define UE_API MOVIESCENETOOLS_API

class IDetailLayoutBuilder;
class SCheckBoxList;

class FMovieScenePlatformConditionCustomization : public IDetailCustomization
{
public:
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	UE_API void OnPlatformCheckChanged(int32 Index);
	UE_API TArray<FName> GetCurrentValidPlatformNames();

	TSharedPtr<IPropertyHandle> ValidPlatformsPropertyHandle;
	TSharedPtr<SCheckBoxList> CheckBoxList;
};

#undef UE_API
