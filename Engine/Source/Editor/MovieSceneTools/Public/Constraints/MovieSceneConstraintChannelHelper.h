// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "MovieSceneSection.h"

#include "MovieSceneConstraintChannelHelper.generated.h"

#define UE_API MOVIESCENETOOLS_API

class IMovieSceneConstrainedSection;
struct ITransformConstraintChannelInterface;
class UWorld;
class UTickableConstraint;
class ISequencer;
class UMovieSceneSection;
class UTickableTransformConstraint;
class UTickableParentConstraint;
class UTransformableHandle;
struct FMovieSceneConstraintChannel;
struct FFrameNumber;
struct FConstraintAndActiveChannel;
enum class EMovieSceneKeyInterpolation : uint8;

UENUM()
enum class ECreationTime : uint8
{
	CurrentFrame,
	FromStart,
	Infinite,
	// FromRange
};

USTRUCT()
struct FSequencerCreationOptions
{
	GENERATED_BODY()

	/** Whether to create the constraint at the current frame, at the beginning of the sequence, or infinitely. */
	UPROPERTY(EditAnywhere, Category=Options)
	ECreationTime CreationTime = ECreationTime::CurrentFrame;
};

struct FCompensationEvaluator
{
public:
	TArray<FTransform> ChildLocals;
	TArray<FTransform> ChildGlobals;
	TArray<FTransform> SpaceGlobals;

	UE_API FCompensationEvaluator(UTickableTransformConstraint* InConstraint);

	struct FEvalParameters
	{
		FEvalParameters(const TSharedPtr<ISequencer>& InSequencer, const TArray<FFrameNumber>& InFrames)
			: Sequencer(InSequencer.Get())
			, Frames(InFrames)
		{}

		bool IsValid() const
		{
			return Sequencer && !Frames.IsEmpty();
		}
		
		ISequencer* Sequencer = nullptr;
		const TConstArrayView<const FFrameNumber> Frames;
		bool bToActive = false;
		bool bKeepCurrent = false;
		bool bInfiniteAtLeft = false;
		TOptional<FFrameNumber> OptCurrentFrame;
		bool bPropagateRelative = true;
	};
	
	UE_API void ComputeLocalTransforms(UWorld* InWorld, const FEvalParameters& InEvalParams);
	UE_API void ComputeLocalTransformsBeforeDeletion(UWorld* InWorld, const FEvalParameters& InEvalParams);
	UE_API void ComputeCompensation(UWorld* InWorld, const TSharedPtr<ISequencer>& InSequencer, const FFrameNumber& InTime);
	UE_API void ComputeLocalTransformsForBaking(UWorld* InWorld, const FEvalParameters& InEvalParams);
	UE_API void CacheTransforms(UWorld* InWorld, const FEvalParameters& InEvalParams);
	UE_API void ComputeCurrentTransforms(UWorld* InWorld);
	
private:

	UE_API const TArray< TWeakObjectPtr<UTickableConstraint> > GetHandleTransformConstraints(UWorld* InWorld) const;

	UTickableTransformConstraint* Constraint = nullptr;
	UTransformableHandle* Handle = nullptr;
};

struct FConstraintSections
{
	UMovieSceneSection* ConstraintSection = nullptr;
	UMovieSceneSection* ChildTransformSection = nullptr;
	UMovieSceneSection* ParentTransformSection = nullptr;
	FConstraintAndActiveChannel* ActiveChannel = nullptr;
	ITransformConstraintChannelInterface* Interface = nullptr;
};
struct FMovieSceneConstraintChannelHelper
{
public:

	static UE_API bool AddConstraintToSequencer(
		const TSharedPtr<ISequencer>& InSequencer,
		UTickableTransformConstraint* InConstraint,
		const FSequencerCreationOptions& InOptions = FSequencerCreationOptions());
	
	/** Adds an active key if needed and does the compensation when switching. Will use the optional active and time if set. 
	Will return true if key is actually set, may not be if the value is the same.*/
	static UE_API bool SmartConstraintKey(
		const TSharedPtr<ISequencer>& InSequencer,
		UTickableTransformConstraint* InConstraint, 
		const TOptional<bool>& InOptActive,
		const TOptional<FFrameNumber>& InOptFrameTime);
	
	/** Compensate transform on handles when a constraint switches state. */
	static UE_API void Compensate(
		const TSharedPtr<ISequencer>& InSequencer,
		const UTickableTransformConstraint* InConstraint,
		const TOptional<FFrameNumber>& InOptTime,
		const bool bCompPreviousTick);
	
	static UE_API void CompensateIfNeeded(
		const TSharedPtr<ISequencer>& InSequencer,
		IMovieSceneConstrainedSection* Section,
		const TOptional<FFrameNumber>& OptionalTime,
		const bool bCompPreviousTick,
		const int32 InChildHash = INDEX_NONE);
		
	static UE_API void HandleConstraintRemoved(
		UTickableConstraint* InConstraint,
		const FMovieSceneConstraintChannel* InChannel,
		const TSharedPtr<ISequencer>& InSequencer,
		UMovieSceneSection* InSection);

	static UE_API void HandleConstraintKeyDeleted(
		UTickableTransformConstraint* InConstraint,
		const FMovieSceneConstraintChannel* InConstraintChannel,
		const TSharedPtr<ISequencer>& InSequencer,
		UMovieSceneSection* InSection,
		const FFrameNumber& InTime);

	static UE_API void HandleConstraintKeyMoved(
		const UTickableTransformConstraint* InConstraint,
		const FMovieSceneConstraintChannel* InConstraintChannel,
		UMovieSceneSection* InSection,
		const FFrameNumber& InCurrentFrame, const FFrameNumber& InNextFrame);

	/* Get the section and the channel for the given constraint, will be nullptr's if it doesn't exist in Sequencer*/
	static UE_API FConstraintSections  GetConstraintSectionAndChannel(
		const UTickableTransformConstraint* InConstraint,
		const TSharedPtr<ISequencer>& InSequencer);

	/* For the given constraint get all of the transform keys for it's child and parent handles*/
	static UE_API void GetTransformFramesForConstraintHandles(
		const UTickableTransformConstraint* InConstraint,
		const TSharedPtr<ISequencer>& InSequencer,
		const FFrameNumber& StartFrame,
		const FFrameNumber& EndFrame,
		TArray<FFrameNumber>& OutFramesToBake);


	/** @todo documentation. */
	template<typename ChannelType>
	static void GetFramesToCompensate(
		const FMovieSceneConstraintChannel& InActiveChannel,
		const bool InActiveValueToBeSet,
		const TOptional<FFrameNumber>& InTime,
		const TArrayView<ChannelType*>& InChannels,
		TArray<FFrameNumber>& OutFrames);

	/** @todo documentation. */
	template< typename ChannelType >
	static void GetFramesAfter(
		const FMovieSceneConstraintChannel& InActiveChannel,
		const FFrameNumber& InTime,
		const TArrayView<ChannelType*>& InChannels,
		TArray<FFrameNumber>& OutFrames);

	/** @todo documentation. */
	template< typename ChannelType >
	static void GetFramesWithinActiveState(
		const FMovieSceneConstraintChannel& InActiveChannel,
		const TArrayView<ChannelType*>& InChannels,
		TArray<FFrameNumber>& OutFrames);

	/** @todo documentation. */
	template< typename ChannelType >
	static void MoveTransformKeys(
		const TArrayView<ChannelType*>& InChannels,
		const FFrameNumber& InCurrentTime,
		const FFrameNumber& InNextTime);

	/** delete transform keys at that time */
	template< typename ChannelType >
	static void DeleteTransformKeys(
		const TArrayView<ChannelType*>& InChannels,
		const FFrameNumber& InTime);

	/** Change key interpolation at the specified time*/
	template< typename ChannelType >
	static void ChangeKeyInterpolation(
		const TArrayView<ChannelType*>& InChannels,
		const FFrameNumber& InTime,
		EMovieSceneKeyInterpolation KeyInterpolation);

	static UE_API void HandleConstraintPropertyChanged(
		UTickableTransformConstraint* InConstraint,
		const FMovieSceneConstraintChannel& InActiveChannel,
		const FPropertyChangedEvent& InPropertyChangedEvent,
		const TSharedPtr<ISequencer>& InSequencer,
		UMovieSceneSection* InSection);

	template< typename ChannelType >
	static TArray<FFrameNumber> GetTransformTimes(
		const TArrayView<ChannelType*>& InChannels,
		const FFrameNumber& StartTime, 
		const FFrameNumber& EndTime);

	template< typename ChannelType >
	static  void DeleteTransformTimes(
		const TArrayView<ChannelType*>& InChannels,
		const FFrameNumber& StartTime,
		const FFrameNumber& EndTime,
		EMovieSceneTransformChannel Channels = EMovieSceneTransformChannel::AllTransform);

	/** this will only set the value son channels with keys at the specified time, reusing tangent time etc. */
	template< typename ChannelType >
	static void SetTransformTimes(
		const TArrayView<ChannelType*>& InChannels,
		const TArray<FFrameNumber>& Frames,
		const TArray<FTransform>& Transforms);

	/** Checks whether the key to be added is prior to the current frame and updates the frames to be compensated accordingly. */
	template<typename ChannelType>
	static TPair<TOptional<FFrameNumber>, bool> IsActiveKeyAddedBeforeCurrentTime(
		const FFrameNumber& InCurrentTime,
		const FFrameNumber& InTime,
		const TArrayView<ChannelType*>& InChannels,
		TArray<FFrameNumber>& InOutFramesToCompensate);
	
	static UE_API bool bDoNotCompensate;

private:
	/** Get the animatable interface for that handle if registered. */
	static UE_API ITransformConstraintChannelInterface* GetHandleInterface(const UTransformableHandle* InHandle);
	
	/** For the given handle create any movie scene binding for it based upon the current sequencer that's open*/
	static UE_API void CreateBindingIDForHandle(const TSharedPtr<ISequencer>& InSequencer, UTransformableHandle* InHandle);

	/** For the given handle, return true if a binding exists and if that binding represents a spawnable. */
	static UE_API bool IsHandleSpawnable(const TSharedPtr<ISequencer>& InSequencer, const UTransformableHandle* InHandle);

	/** Compensate scale keys when enabling/disabling scaling for parent constraints. */
	static UE_API void CompensateScale(
		UTickableParentConstraint* InParentConstraint,
		const FMovieSceneConstraintChannel& InActiveChannel,
		const TSharedPtr<ISequencer>& InSequencer,
		UMovieSceneSection* InSection);

	/** Handle offset modifications so that the child's transform channels are synced. */
	static UE_API void HandleOffsetChanged(
		UTickableTransformConstraint* InConstraint,
		const FMovieSceneConstraintChannel& InActiveChannel,
		const TSharedPtr<ISequencer>& InSequencer);
};

#undef UE_API
