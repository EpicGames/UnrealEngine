// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConstraintChannel.h"
#include "MovieSceneConstraintChannelHelper.h"
#include "Channels/MovieSceneCurveChannelCommon.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSection.h"
#include "Algo/Unique.h"
#include "Containers/SortedMap.h"
#include "KeyParams.h"

template<typename ChannelType>
void FMovieSceneConstraintChannelHelper::GetFramesToCompensate(
	const FMovieSceneConstraintChannel& InActiveChannel,
	const bool InActiveValueToBeSet,
	const TOptional<FFrameNumber>& InTime,
	const TArrayView<ChannelType*>& InChannels,
	TArray<FFrameNumber>& OutFrames)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;

	const bool bHasKeys = (InActiveChannel.GetNumKeys() > 0);
	const bool bInfinite = InActiveChannel.IsInfinite(); 
	
	OutFrames.Reset();

	if (InTime)
	{
		if (bInfinite)
		{
			OutFrames.Add(*InTime);
			
			// add all the frames that need transform compensation
			for (const ChannelType* InChannel: InChannels)
			{
				const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
				const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
				if (!Times.IsEmpty())
				{
					int32 NextTimeIndex = Algo::UpperBound(Times, *InTime);
					while (Times.IsValidIndex(NextTimeIndex))
					{
						OutFrames.Add(Times[NextTimeIndex++]);
					}
				}
			}
			
			// uniqueness
			OutFrames.Sort();
			OutFrames.SetNum(Algo::Unique(OutFrames));
			return;
		}

		const bool bIsActiveByDefault = InActiveChannel.GetDefault() && *InActiveChannel.GetDefault();
		if (bHasKeys && bIsActiveByDefault)
		{
			const TArrayView<const FFrameNumber> ConstraintTimes = InActiveChannel.GetTimes();
			const TArrayView<const bool> Values = InActiveChannel.GetValues();
			if (*InTime <= ConstraintTimes[0] && !Values[0])
			{
				// add all the frames that need transform compensation
				for (const ChannelType* InChannel: InChannels)
				{
					const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
					const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
					if (!Times.IsEmpty())
					{
						int32 Index = 0;
						while (Index < Times.Num() && Times[Index] < *InTime)
						{
							OutFrames.Add(Times[Index++]);
						}
					}
				}
				
				// uniqueness
				OutFrames.Sort();
				OutFrames.SetNum(Algo::Unique(OutFrames));
				
				// insert time upfront after sorting even if greater when dealing with the left side of an infinite constraint
				OutFrames.Insert(*InTime, 0);
				return;
			}
		}
		
		{
			// add the current frame
			OutFrames.Add(*InTime);
			
			// add the next frames that need transform compensation 
			for (const ChannelType* InChannel: InChannels)
			{
				const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
				const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
				if (!Times.IsEmpty())
				{
					// look for the first next key frame for this channel 
					const int32 NextTimeIndex = Algo::UpperBound(Times, *InTime);
					if (Times.IsValidIndex(NextTimeIndex))
					{
						// store the time while the state is different
						for (int32 Index = NextTimeIndex; Index < Times.Num(); ++Index)
						{
							if (!bHasKeys)
							{
								OutFrames.Add(Times[Index]);
							}
							else
							{
								bool NextValue = false; InActiveChannel.Evaluate(Times[Index], NextValue);
								if (NextValue == InActiveValueToBeSet)
								{
									break;
								}
								OutFrames.Add(Times[Index]);
							}
						}
					}
				}
			}	
		}
	}
	else
	{
		for (const ChannelType* InChannel: InChannels)
		{
			const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
			OutFrames.Append(ChannelData.GetTimes());
		}
	}

	// uniqueness
	OutFrames.Sort();
	OutFrames.SetNum(Algo::Unique(OutFrames));
}

template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::GetFramesAfter(
	const FMovieSceneConstraintChannel& InActiveChannel,
	const FFrameNumber& InTime,
	const TArrayView<ChannelType*>& InChannels,
	TArray<FFrameNumber>& OutFrames)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	
	OutFrames.Reset();
	
	const TMovieSceneChannelData<const bool> ConstraintChannelData = InActiveChannel.GetData();
	const int32 KeyIndex = ConstraintChannelData.FindKey(InTime);
	if (!ConstraintChannelData.GetTimes().IsValidIndex(KeyIndex))
	{
		return;
	}

	const bool CurrentValue = ConstraintChannelData.GetValues()[KeyIndex];
	
	// compute last frame to compensate
	auto GetEndOfCompensationTime = [KeyIndex](const TMovieSceneChannelData<const bool>& InData)
	{
		const TArrayView<const bool> Values = InData.GetValues();
		const TArrayView<const FFrameNumber> Times = InData.GetTimes();

		const bool CurrentValue = Values[KeyIndex];
		for (int32 NextIndex = KeyIndex+1; NextIndex < Times.Num(); ++NextIndex)
		{
			if (Values[NextIndex] != CurrentValue)
			{
				return TOptional<FFrameNumber>(Times[NextIndex]);
			}
		}
		return TOptional<FFrameNumber>();
	};
	const TOptional<FFrameNumber> EndOfCompensationTime = GetEndOfCompensationTime(ConstraintChannelData);

	const bool bHasEndTime = EndOfCompensationTime.IsSet();
	
	// add the current frame
	OutFrames.Add(InTime);

	// add the next frames that need transform compensation 
	for (const ChannelType* InChannel: InChannels)
	{
		const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
		const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		if (!Times.IsEmpty())
		{
			// look for the first next key frame for this channel 
			const int32 NextTimeIndex = Algo::UpperBound(Times, InTime);
			if (Times.IsValidIndex(NextTimeIndex))
			{
				// store the time while the state is different
				for (int32 Index = NextTimeIndex; Index < Times.Num(); ++Index)
				{
					if (!bHasEndTime || Times[Index] < EndOfCompensationTime.GetValue() )
					{
						OutFrames.Add(Times[Index]);
					}
				}
			}
		}
	}

	// uniqueness
	OutFrames.Sort();
	OutFrames.SetNum(Algo::Unique(OutFrames));
}

template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::GetFramesWithinActiveState(
	const FMovieSceneConstraintChannel& InActiveChannel,
	const TArrayView<ChannelType*>& InChannels,
	TArray<FFrameNumber>& OutFrames)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	
	OutFrames.Reset();
	
	const TMovieSceneChannelData<const bool> ConstraintChannelData = InActiveChannel.GetData();
	const TArrayView<const FFrameNumber>& ActiveTimes = ConstraintChannelData.GetTimes();
	const bool bInfinite = InActiveChannel.IsInfinite();
	if (ActiveTimes.IsEmpty() && !bInfinite)
	{
		return;
	}

	if (bInfinite)
	{
		for (const ChannelType* InChannel: InChannels)
		{
			const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
			OutFrames.Append(ChannelData.GetTimes());
		}
	}
	else
	{
		const FFrameNumber& FirstTime = ActiveTimes[0];
		const FFrameNumber& LastTime = ActiveTimes.Last();
		
		// add active times
		for (int32 Index = 0; Index < ActiveTimes.Num(); ++Index)
		{
			OutFrames.Add(ActiveTimes[Index]);
		}

		const bool bIsLastStateInactive = ConstraintChannelData.GetValues().Last() == false; 

		// add frames where the constraint is active 
		for (const ChannelType* InChannel: InChannels)
		{
			const TMovieSceneChannelData<const ChannelValueType> ChannelData = InChannel->GetData();
			const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
			if (!Times.IsEmpty())
			{
				// look for the first next key frame for this channel 
				const int32 NextTimeIndex = Algo::UpperBound(Times,FirstTime);
				if (Times.IsValidIndex(NextTimeIndex))
				{
					// store the time is the state is active
					for (int32 Index = NextTimeIndex; Index < Times.Num(); ++Index)
					{
						bool bIsActive = false;
						InActiveChannel.Evaluate(Times[Index], bIsActive);
						if (bIsActive)
						{
							OutFrames.Add(Times[Index]);
						}

						if (bIsLastStateInactive && Times[Index] > LastTime)
						{
							break;
						}
					}
				}
			}
		}
	}

	// uniqueness
	OutFrames.Sort();
	OutFrames.SetNum(Algo::Unique(OutFrames));
}

template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::MoveTransformKeys(
	const TArrayView<ChannelType*>& InChannels,
	const FFrameNumber& InCurrentTime,
	const FFrameNumber& InNextTime)
{
	const FFrameNumber Delta = InNextTime - InCurrentTime;
	if (Delta == 0)
	{
		return;
	}
	
	for (ChannelType* Channel: InChannels)
	{
		TMovieSceneChannelData<typename ChannelType::ChannelValueType> Data = Channel->GetData();
		const TArrayView<const FFrameNumber> Times = Data.GetTimes();
		const int32 NumTimes = Times.Num();

		if (Delta > 0) //if we are moving keys positively in time we start from end frames and move them so we can use indices
		{
			for (int32 KeyIndex = NumTimes - 1; KeyIndex >= 0; --KeyIndex)
			{
				const FFrameNumber& Frame = Times[KeyIndex];
				const FFrameNumber AbsDiff = FMath::Abs(Frame - InCurrentTime);
				if (AbsDiff<= 1)
				{
					Data.MoveKey(KeyIndex, Frame + Delta);
				}
			}
		}
		else
		{
			for (int32 KeyIndex = 0; KeyIndex < NumTimes; ++KeyIndex)
			{
				const FFrameNumber& Frame = Times[KeyIndex];
				const FFrameNumber AbsDiff = FMath::Abs( Frame - InCurrentTime);
				if (AbsDiff <= 1)
				{
					Data.MoveKey(KeyIndex, Frame + Delta);
				}
			}
		}
	}
}

template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::DeleteTransformKeys(
	const TArrayView<ChannelType*>& InChannels,
	const FFrameNumber& InTime)
{
	for (ChannelType* Channel: InChannels)
	{
		TMovieSceneChannelData<typename ChannelType::ChannelValueType> Data = Channel->GetData();
		const TArrayView<const FFrameNumber> Times = Data.GetTimes();
		
		const int32 KeyIndex = Algo::LowerBound(Times, InTime);
		if (Times.IsValidIndex(KeyIndex) && Times[KeyIndex] == InTime)
		{
			Data.RemoveKey(KeyIndex);
		}
	}
}


template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::ChangeKeyInterpolation(
	const TArrayView<ChannelType*>& InChannels,
	const FFrameNumber& InTime,
	EMovieSceneKeyInterpolation KeyInterpolation)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	TEnumAsByte<ERichCurveInterpMode> InterpMode = RCIM_Cubic;
	TEnumAsByte<ERichCurveTangentMode> TangentMode = RCTM_Auto;
	
	switch (KeyInterpolation)
	{
		case EMovieSceneKeyInterpolation::SmartAuto:
		{
			InterpMode = RCIM_Cubic;
			TangentMode = RCTM_SmartAuto;
			break;
		}
		case EMovieSceneKeyInterpolation::Auto:
		{
			InterpMode = RCIM_Cubic;
			TangentMode = RCTM_Auto;
			break;
		}
		case EMovieSceneKeyInterpolation::User:
		{
			InterpMode = RCIM_Cubic;
			TangentMode = RCTM_User;
			break;
		}
		case EMovieSceneKeyInterpolation::Break:
		{
			InterpMode = RCIM_Cubic;
			TangentMode = RCTM_Break;
			break;
		}
		case EMovieSceneKeyInterpolation::Linear:
		{
			InterpMode = RCIM_Linear;
			break;
		}
		case EMovieSceneKeyInterpolation::Constant:
		{
			InterpMode = RCIM_Constant;
			break;
		}
	};
	for (ChannelType* Channel : InChannels)
	{
		TMovieSceneChannelData<ChannelValueType> ChannelInterface = Channel->GetData();
		const TArrayView<const FFrameNumber> Times = ChannelInterface.GetTimes();
		const int32 KeyIndex = Algo::LowerBound(Times, InTime);
		TArrayView<ChannelValueType> Values = ChannelInterface.GetValues();
		if (Times.IsValidIndex(KeyIndex) && Times[KeyIndex] == InTime && Values.IsValidIndex(KeyIndex))
		{
			Values[KeyIndex].InterpMode = InterpMode;
			Values[KeyIndex].TangentMode = TangentMode;
		}
	}
}

template< typename ChannelType >
TArray<FFrameNumber> FMovieSceneConstraintChannelHelper::GetTransformTimes(
	const TArrayView<ChannelType*>& InChannels,
	const FFrameNumber& StartTime,
	const FFrameNumber& EndTime)
{
	TSortedMap<FFrameNumber, FFrameNumber> FrameSet;
	TRange<FFrameNumber> WithinRange(0, 0);
	WithinRange.SetLowerBoundValue(StartTime);
	WithinRange.SetUpperBoundValue(EndTime);
	TArray<FFrameNumber> KeyTimes;
	TArray<FKeyHandle> KeyHandles;
	for (ChannelType* Channel : InChannels)
	{
		TMovieSceneChannelData<typename ChannelType::ChannelValueType> Data = Channel->GetData();
		KeyTimes.SetNum(0);
		KeyHandles.SetNum(0);
		Data.GetKeys(WithinRange, &KeyTimes,&KeyHandles);
		for (const FFrameNumber& FrameNumber : KeyTimes)
		{
			FrameSet.Add(FrameNumber,FrameNumber);
		}
	}
	TArray<FFrameNumber> Frames;
	FrameSet.GenerateKeyArray(Frames);
	return Frames;
}

template< typename ChannelType >
void FMovieSceneConstraintChannelHelper::DeleteTransformTimes(
	const TArrayView<ChannelType*>& InChannels,
	const FFrameNumber& StartTime,
	const FFrameNumber& EndTime,
	EMovieSceneTransformChannel InChannelsToKey)
{
	// nothing to process or invalid channels array 
	const int32 NumChannels = InChannels.Num();
	if (NumChannels == 0 || NumChannels > 9)
	{
		return;
	}
	
	TRange<FFrameNumber> WithinRange(0, 0);
	WithinRange.SetLowerBoundValue(StartTime);
	WithinRange.SetUpperBoundValue(EndTime);
	
	auto ProcessAllChannels = [&WithinRange, InChannels]()
	{
		TArray<FFrameNumber> KeyTimes;
		TArray<FKeyHandle> KeyHandles;
			
		for (ChannelType* Channel : InChannels)
		{
			TMovieSceneChannelData<typename ChannelType::ChannelValueType> Data = Channel->GetData();
			KeyTimes.SetNum(0);
			KeyHandles.SetNum(0);
			Data.GetKeys(WithinRange, &KeyTimes, &KeyHandles);
			Data.DeleteKeys(KeyHandles);
		}
	};
	
	// process all channels
	if (InChannelsToKey == EMovieSceneTransformChannel::AllTransform)
	{
		return ProcessAllChannels();
	}

	const bool bProcessTranslation = EnumHasAllFlags(InChannelsToKey, EMovieSceneTransformChannel::Translation);
	const bool bProcessRotation = EnumHasAllFlags(InChannelsToKey, EMovieSceneTransformChannel::Rotation);
	const bool bProcessScale = EnumHasAllFlags(InChannelsToKey, EMovieSceneTransformChannel::Scale);
	if (!(bProcessTranslation || bProcessRotation || bProcessScale))
	{
		// nothing to process
		return;
	}

	TArray<uint32> ChannelsIndexToProcess;
	if (bProcessTranslation)
	{
		ChannelsIndexToProcess.Append({0, 1, 2});
	}
	if (bProcessRotation)
	{
		ChannelsIndexToProcess.Append({3, 4, 5});
	}
	if (bProcessScale)
	{
		ChannelsIndexToProcess.Append({6, 7, 8});
	}
	const int32 NumChannelToProcess = ChannelsIndexToProcess.Num();
	
	// process all channels
	if (NumChannelToProcess == NumChannels)
	{
		return ProcessAllChannels();
	}
	
	if (!ensureAlways(NumChannels == 9))
	{
		// if NumChannelToProcess != NumChannels and NumChannels is lower than 9 then we don't know what to remove...
		return;
	}
	
	TArray<FFrameNumber> KeyTimes;
	TArray<FKeyHandle> KeyHandles;
	for (const int32 ChannelIndex : ChannelsIndexToProcess)
	{
		if (InChannels.IsValidIndex(ChannelIndex))
		{
			ChannelType* Channel = InChannels[ChannelIndex];
			TMovieSceneChannelData<typename ChannelType::ChannelValueType> Data = Channel->GetData();
			KeyTimes.SetNum(0);
			KeyHandles.SetNum(0);
			Data.GetKeys(WithinRange, &KeyTimes, &KeyHandles);
			Data.DeleteKeys(KeyHandles);
		}
	}
}

template<typename ChannelType>
void EvaluateTangentAtThisTime(int32 ChannelIndex, int32 NumChannels,
	UMovieSceneSection* Section,
	FFrameNumber Time,
	TArray<FMovieSceneTangentData>& OutTangents)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;

	// NOTE this might be moved to FMovieSceneFloatChannel
	auto EvaluateTangent = [](const ChannelType* InChannel, FFrameNumber InTime)
	{
		const TMovieSceneChannelData<const ChannelValueType> ChannelInterface = InChannel->GetData();
		const TArrayView<const FFrameNumber> Times = ChannelInterface.GetTimes();

		// Need at least two keys to evaluate a derivative
		if (Times.Num() < 2)
		{
			static const FMovieSceneTangentData DefaultTangent;
			return DefaultTangent;
		}

		const TArrayView<const ChannelValueType> Values = ChannelInterface.GetValues();

		// look around to get the closest key tangent if fairly close
		const int32 Tolerance = static_cast<int32>(InChannel->GetTickResolution().AsDecimal() * 0.01);
		// NOTE FindKey might return Times.Num() (see AlgoImpl::LowerBoundInternal)
		const int32 ExistingIndex = ChannelInterface.FindKey(InTime, Tolerance);
		if (Times.IsValidIndex(ExistingIndex) && Values.IsValidIndex(ExistingIndex))
		{
			const FFrameNumber& Time = Times[ExistingIndex];
			const int32 DiffToKey = InTime.Value - Time.Value;

			// if the closest key is within a threshold, we use it's tangent directly instead of computing one
			if (FMath::Abs(DiffToKey) <= Tolerance)
			{
				const ChannelValueType Value = Values[ExistingIndex];
				if (DiffToKey == 0)
				{
					return Value.Tangent;
				}

				FMovieSceneTangentData Tangent = Value.Tangent;
				if (DiffToKey < 0) // pretty close to the next key
				{
					Tangent.LeaveTangent = 0.f;
				}
				else // pretty close to the previous key
				{
					Tangent.ArriveTangent = 0.f;
				}
				return Tangent;
			}
		}

		// compute tangent using central difference
		// NOTE we may wanna compute a backward / forward difference instead
		const int32 Delta = static_cast<int32>(InChannel->GetTickResolution().AsDecimal() * 0.1);

		float PrevValue = 0.f; InChannel->Evaluate(InTime - Delta, PrevValue);
		float NextValue = 0.f; InChannel->Evaluate(InTime + Delta, NextValue);
		const float TangentValue = (NextValue - PrevValue) / (2.f * Delta);

		FMovieSceneTangentData TangentData;
		TangentData.ArriveTangent = TangentValue;
		TangentData.LeaveTangent = TangentValue;

		return TangentData;
	};


	// compute and store tangents
	OutTangents.SetNum(NumChannels);

	const TArrayView<ChannelType*> Channels = Section->GetChannelProxy().GetChannels<ChannelType>();
	for (int32 Index = 0; Index < NumChannels; ++Index, ++ChannelIndex)
	{
		OutTangents[Index] = EvaluateTangent(Channels[ChannelIndex], Time);
	}
}

// NOTE we may pass an enum to tell which tangent we wanna set (arrive, leave, both)
template<typename ChannelType>
void SetTangentsAtThisTime(int32 ChannelIndex,
	int32 NumChannels,
	UMovieSceneSection* Section,
	FFrameNumber Time,
	const TArray<FMovieSceneTangentData>& InTangents)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	
	if (!Section)
	{
		return;
	}
	
	if (!InTangents.IsValidIndex(NumChannels-1))
	{
		return;
	}
	
	const TArrayView<ChannelType*> Channels = Section->GetChannelProxy().GetChannels<ChannelType>();
	if (!Channels.IsValidIndex(ChannelIndex) || !Channels.IsValidIndex(ChannelIndex+NumChannels-1))
	{
		return;
	}
		
	for (int32 Index = 0; Index < NumChannels; ++Index, ++ChannelIndex)
	{
		TMovieSceneChannelData<ChannelValueType> ChannelInterface = Channels[ChannelIndex]->GetData();
		TArrayView<ChannelValueType> Values = ChannelInterface.GetValues();
		const int32 KeyIndex = ChannelInterface.FindKey(Time);
		if (KeyIndex != INDEX_NONE)
		{
			ChannelValueType& Value = Values[KeyIndex];
			Value.Tangent.ArriveTangent = InTangents[Index].ArriveTangent;
			Value.Tangent.LeaveTangent = InTangents[Index].LeaveTangent;
			Value.TangentMode = RCTM_Break;
		}
	}
}

template<typename ChannelType>
TPair<TOptional<FFrameNumber>, bool> IsActiveKeyAddedBeforeCurrentTime(
	const FFrameNumber& InCurrentTime,
	const FFrameNumber& InTime,
	const TArrayView<ChannelType*>& InChannels,
	TArray<FFrameNumber>& InOutFramesToCompensate)
{
	TPair< TOptional<FFrameNumber>, bool > Result;

	// this means that we're adding the constraint before the current frame
	if (InTime < InCurrentTime)
	{
		Result.Key = InCurrentTime;
		
		// add the current frame if missing
		const int32 Index = Algo::LowerBound(InOutFramesToCompensate, InCurrentTime);	
		if (Index >= InOutFramesToCompensate.Num() || InOutFramesToCompensate[Index] != InCurrentTime)
		{
			InOutFramesToCompensate.Insert(InCurrentTime, Index);
		}

		// check if there's a transform key before the current time
		// if that's not the case then we need to propagate the current offset
		const bool bPropagateRelative = [InChannels, InCurrentTime]()
		{
			for (const ChannelType* TransformChannel : InChannels)
			{
				const TArrayView<const FFrameNumber> FrameNumbers = TransformChannel->GetTimes();
				if (!FrameNumbers.IsEmpty())
				{
					if (FrameNumbers[0] < InCurrentTime)
					{
						return false;
					}
				}
			}
			return true;
		}();
		
		Result.Value = bPropagateRelative;
	}

	return Result;
}