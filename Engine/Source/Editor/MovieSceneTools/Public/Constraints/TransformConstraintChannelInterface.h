// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneSubTrack;
class UMovieSceneTrack;
struct FMovieSceneConstraintChannel;
class IMovieSceneConstrainedSection;
class UWorld;
class ISequencer;
class UMovieSceneSection;
class UTickableTransformConstraint;
class UTransformableHandle;
struct FFrameNumber;
enum class EMovieSceneTransformChannel : uint32;

/**
* Abstract interface that defines animatable capabilities for transformable handles
*/

struct ITransformConstraintChannelInterface
{
	virtual MOVIESCENETOOLS_API ~ITransformConstraintChannelInterface();

	/** Get the section where the channels live from the object that the handle wraps. */
	virtual UMovieSceneSection* GetHandleSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) = 0;
	virtual UMovieSceneSection* GetHandleConstraintSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) = 0;

	/** Get the world from the object that the handle wraps. */
	virtual UWorld* GetHandleWorld(UTransformableHandle* InHandle) = 0;

	/** Add an active/inactive key to the constraint channel if needed and does the transform compensation on the transform channels. */
	virtual bool SmartConstraintKey(
		UTickableTransformConstraint* InConstraint, const TOptional<bool>& InOptActive,
		const TRange<FFrameNumber>& InTimeRange, const TSharedPtr<ISequencer>& InSequencer) = 0;

	/** Add keys on the transform channels of the object that the handle wraps. */
	virtual void AddHandleTransformKeys(
		const TSharedPtr<ISequencer>& InSequencer,
		const UTransformableHandle* InHandle,
		const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InLocalTransforms,
		const EMovieSceneTransformChannel& InChannels) = 0;

protected:

	MOVIESCENETOOLS_API ITransformConstraintChannelInterface();

	static MOVIESCENETOOLS_API bool CanAddKey(const FMovieSceneConstraintChannel& InActiveChannel, const FFrameNumber& InTime, bool& ActiveValue);
	static MOVIESCENETOOLS_API void CleanDuplicates(FMovieSceneConstraintChannel& InOutActiveChannel, TArray<FFrameNumber>& OutTimesRemoved);

	void Initialize();
	void Shutdown();
	
	void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);
	void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	virtual void MOVIESCENETOOLS_API UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld);
	static MOVIESCENETOOLS_API void UnregisterConstraints(IMovieSceneConstrainedSection* InSection, UWorld* InWorld);
	void UnregisterMovieScene(const UMovieScene* InMovieScene, UWorld* InWorld);
	
	FDelegateHandle SequencerCreatedHandle;
	TArray<TWeakPtr<ISequencer>> Sequencers;
};

/**
* Handle animatable interface registry
*/

class FConstraintChannelInterfaceRegistry
{
public:
	~FConstraintChannelInterfaceRegistry() = default;

	/** Get the singleton registry object */
	static MOVIESCENETOOLS_API FConstraintChannelInterfaceRegistry& Get();

	/** Register an interface for the HandleType static class */
	template<typename HandleType>
	void RegisterConstraintChannelInterface(TUniquePtr<ITransformConstraintChannelInterface>&& InInterface)
	{
		UClass* HandleClass = HandleType::StaticClass();
		check(!HandleToInterfaceMap.Contains(HandleClass));
		HandleToInterfaceMap.Add(HandleClass, MoveTemp(InInterface));
	}

	/** Find the registered interface from the given class. Returns a nullptr if noting registered for that class. */
	MOVIESCENETOOLS_API ITransformConstraintChannelInterface* FindConstraintChannelInterface(const UClass* InClass) const;

private:
	FConstraintChannelInterfaceRegistry() = default;

	TMap<UClass*, TUniquePtr<ITransformConstraintChannelInterface>> HandleToInterfaceMap;
};