// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

#define UE_API MOVIESCENETOOLS_API

class ISequencer;
class UMovieSceneDecorationContainerObject;
class FMenuBuilder;

class IMovieSceneDecorationMenuProvider : public IModularFeature
{
public:

	static UE_API FName GetModularFeatureName();

	UE_API virtual ~IMovieSceneDecorationMenuProvider();

	/**
	 * Allows providers to specify the decoration classes they handle.
	 * @return The decoration UClass this menu provider handles.
	 */
	virtual const UClass* GetHandledDecorationClass() const = 0;


	/**
	 * Populate the custom menu entry for the decoration class handled by this provider.
	 * @param MenuBuilder The menu builder to extend, and add a custom entry to.
	 * @param BoundObject The object associated with the decoration container, if it exists.
	 * @param DecorationContainer The decoration container owning the decoration this provider is associated with.
	 * @param InSequencer The sequence editor.
	 */
	UE_API virtual void PopulateAddDecorationMenu(FMenuBuilder& MenuBuilder, TObjectPtr<UObject> BoundObject, UMovieSceneDecorationContainerObject* DecorationContainer, TWeakPtr<ISequencer> InSequencer) = 0;

	/**
	 * Allows providers to define their priority in their menu entries. 
	 * @return Menu priority for this decoration provider. Lower values appear first in the menu.
	 * Default is 100 (middle priority). Use 0-50 for high priority items, 100+ for normal items.
	 */
	virtual int32 GetDecorationMenuPriority() const { return 100; };
};

#undef UE_API