// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyTypeCustomizationUtils;
class UMovieSceneDecorationContainerObject;
class IDetailLayoutBuilder;
class SWidget;

/**
 * Detail customization for UMovieSceneDecorationContainerObject
 */
class FMovieSceneDecorationContainerCustomization : public IPropertyTypeCustomization
{
public:
	static MOVIESCENETOOLS_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
private:

	void AddCompatibleDecorationsUI(UMovieSceneDecorationContainerObject* Container, IDetailLayoutBuilder& LayoutBuilder);

	TSharedRef<SWidget> OnGetAddDecorationMenuContent(UMovieSceneDecorationContainerObject* Container, TSet<UClass*> CompatibleClasses);

	void OnAddDecoration(UMovieSceneDecorationContainerObject* Container, UClass* DecorationClass);

	void OnRemoveDecoration(UMovieSceneDecorationContainerObject* Container, UClass* DecorationClass);

	TWeakPtr<IPropertyUtilities> PropertyUtils;
};
