// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "MovieScene.h"
#include "MovieSceneTranslator.h"

#define UE_API MOVIESCENETOOLS_API

class FMovieSceneTranslatorContext;

/** 
 * The FFCPXMLImporter class is the entry point for launching an import of data from an XML file into Sequencer. 
 */
class FFCPXMLImporter : public FMovieSceneImporter
{
public:
	UE_API FFCPXMLImporter();

	UE_API virtual ~FFCPXMLImporter();

	/** Format description. */
	UE_API virtual FText GetFileTypeDescription() const;
	/** Import window title. */
	UE_API virtual FText GetDialogTitle() const;
	/** Scoped transaction description. */
	UE_API virtual FText GetTransactionDescription() const;
	/** Message log window title. */
	UE_API virtual FName GetMessageLogWindowTitle() const;
	/** Message log list label. */
	UE_API virtual FText GetMessageLogLabel() const;

public:
	/*
	 * Import FCP 7 XML
	 *
	 * @param InMovieScene The movie scene to import the XML file into
	 * @param InFrameRate The frame rate to import the XML at
	 * @param InFilename The filename to import
	 * @param OutError The return error message
	 * @return Whether the import was successful
	 */
	UE_API bool Import(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InFilename, TSharedRef<FMovieSceneTranslatorContext> InContext);
};

/** 
 * The FFCPXMLExporter class is the entry point for launching an export of data from Sequencer into an XML file. 
 */
class FFCPXMLExporter : public FMovieSceneExporter
{
public:
	/** Constructor */
	UE_API FFCPXMLExporter();
	/** Destructor */
	UE_API virtual ~FFCPXMLExporter();

	/** Format description. */
	UE_API virtual FText GetFileTypeDescription() const;
	/** Export dialog window title. */
	UE_API virtual FText GetDialogTitle() const;
	/** Default format file extension. */
	UE_API virtual FText GetDefaultFileExtension() const;
	/** Notification when export completes. */
	UE_API virtual FText GetNotificationExportFinished() const;
	/** Notification hyperlink to exported file path. */
	UE_API virtual FText GetNotificationHyperlinkText() const;
	/** Message log window title. */
	UE_API virtual FName GetMessageLogWindowTitle() const;
	/** Message log list label. */
	UE_API virtual FText GetMessageLogLabel() const;

public:
	/*
	 * Export FCP 7 XML
	 *
	 * @param InMovieScene The movie scene with the cinematic shot track and audio tracks to export
	 * @param InFilenameFormat The last filename format used to render shots.
	 * @param InFrameRate The frame rate for export.
	 * @param InResX Sequence resolution x.
	 * @param InResY Sequence resolution y.
	 * @param InHandleFrames The number of handle frames to include for each shot.
	 * @param InSaveFilename The file path to save to.
	 * @param OutError The return error message
	 * @param MovieExtension The movie extension for the shot filenames (ie. .avi, .mov, .mp4)
	 * @param InMetadata (optional) Metadata from export to override movie output file list
	 * @return Whether the export was successful
	 */
	
	UE_API virtual bool Export(const UMovieScene* InMovieScene, FString InFilenameFormat, FFrameRate InFrameRate, uint32 InResX, uint32 InResY, int32 InHandleFrames, FString InSaveFilename, TSharedRef<FMovieSceneTranslatorContext> InContext, FString InMovieExtension, const FMovieSceneExportMetadata* InMetadata=nullptr);
};

#undef UE_API
