// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "ISequencerNumericTypeInterface.h"
#include "Widgets/Input/NumericTypeInterface.h"

#define UE_API MOVIESCENETOOLS_API

class IDetailLayoutBuilder;

/**
 *  Customize the FFrameNumber to support conversion from seconds/frames/timecode formats.
 */
class FFrameNumberDetailsCustomization : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface)
	{
		return MakeShared<FFrameNumberDetailsCustomization>(MoveTemp(InNumericTypeInterface));
	}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<ISequencer> Sequencer)
	{
		return MakeShared<FFrameNumberDetailsCustomization>(Sequencer);
	}

	FFrameNumberDetailsCustomization(TWeakPtr<ISequencer> Sequencer)
	{
		WeakSequencer = Sequencer;
	}
	FFrameNumberDetailsCustomization(TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface)
	{
		NumericTypeInterface = InNumericTypeInterface;
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	UE_API FText OnGetTimeText() const;
	UE_API FText OnGetTimeToolTipText() const;
	UE_API void OnTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	TWeakPtr<ISequencer> WeakSequencer;

	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	/** Store the property handle to the FrameNumber field so we can get/set the value on the object via text box callbacks. */
	TSharedPtr<IPropertyHandle> FrameNumberProperty;

	/** If they've used the UIMin metadata on the FFrameNumber property, we store that for use via text box callbacks. */
	int32 UIClampMin;
	/** If they've used the UIMax metadata on the FFrameNumber property, we store that for use via text box callbacks. */
	int32 UIClampMax;
};

#undef UE_API
