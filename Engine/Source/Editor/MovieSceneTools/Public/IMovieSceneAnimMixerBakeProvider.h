// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "Templates/SubclassOf.h"

class FMenuBuilder;
class ISequencer;
class UAnimSequence;
class UAnimSeqExportOption;
class UMovieScene;
class UMovieSceneTrack;
class USkeletalMeshComponent;
class USkeleton;

/**
 * Parameters for building a "Bake To Control Rig" menu section.
 * All fields are Control Rig-agnostic so that the AnimMixer editor has no CR dependency.
 */
struct FAnimMixerBakeMenuParams
{
	/** Sequencer instance */
	TSharedPtr<ISequencer> Sequencer;

	/** Skeletal mesh component for the bound object */
	USkeletalMeshComponent* SkelMeshComp = nullptr;

	/** Skeleton for filtering control rig assets */
	USkeleton* Skeleton = nullptr;

	/** Object binding GUID */
	FGuid ObjectBinding;

	/** The bound actor/object */
	UObject* BoundObject = nullptr;

	/** Mutable filter-by-skeleton toggle state (caller-owned, persists across menu opens) */
	bool* bFilterAssetBySkeleton = nullptr;

	/**
	 * Verb used in the "{Verb} to Control Rig" submenu label.
	 * Examples: "Bake", "Bake Layer", "Bake Selected Layers".
	 * If empty, the provider defaults to "Bake".
	 */
	FText EntryLabelVerb;

	/**
	 * Delegate the provider calls to populate a temporary UAnimSequence with animation data.
	 * The caller captures mixer-specific context (track, filter, etc.) in this delegate.
	 * Return true on success, false to abort the bake.
	 */
	TDelegate<bool(UAnimSequence*, UAnimSeqExportOption*, USkeletalMeshComponent*)> PopulateAnimSequence;

	/**
	 * Creates a child track inside the mixer at the desired location.
	 * The provider calls this with the UClass of the track to create (e.g. UMovieSceneControlRigParameterTrack).
	 * Returns the created track, or nullptr on failure.
	 */
	TFunction<UMovieSceneTrack*(UMovieScene*, TSubclassOf<UMovieSceneTrack>)> ChildTrackFactory;

	/**
	 * Called by the provider after a successful bake.
	 * @param BakeTrack      The created track (same pointer returned by ChildTrackFactory)
	 * @param ControlRigObj  The control rig object (as UObject* to avoid CR type dependency)
	 * @param bSuccess       Whether the bake succeeded
	 */
	TFunction<void(UMovieSceneTrack* /*BakeTrack*/, UObject* /*ControlRigObj*/, bool /*bSuccess*/)> OnComplete;
};

/**
 * Modular feature interface for providing "Bake To Control Rig" menu entries
 * inside Animation Mixer track and layer context menus.
 *
 * When no provider is registered, the bake submenu is simply omitted.
 * The MovieSceneMixedControlRigEditor module registers a concrete provider
 * that delegates to the ControlRigEditor bake helpers.
 */
class IMovieSceneAnimMixerBakeProvider : public IModularFeature
{
public:

	MOVIESCENETOOLS_API static FName GetModularFeatureName();

	virtual ~IMovieSceneAnimMixerBakeProvider() {}

	/**
	 * Build the "Bake To Control Rig" submenu section into the provided MenuBuilder.
	 * This includes the "Filter Asset By Skeleton" toggle and the rig picker submenu.
	 *
	 * The provider is responsible for the entire bake flow:
	 *   1. Building the menu UI (rig picker, filter toggle)
	 *   2. On user selection: calling PopulateAnimSequence to fill a temp anim sequence
	 *   3. Calling ChildTrackFactory to create the destination track
	 *   4. Running the bake (e.g. InitControlRigFromAnimSequence)
	 *   5. Activating any required editor modes
	 *   6. Calling OnComplete with the results
	 */
	virtual void BuildBakeToControlRigMenuSection(
		FMenuBuilder& MenuBuilder,
		const FAnimMixerBakeMenuParams& Params) = 0;
};
