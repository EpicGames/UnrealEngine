// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class FMenuBuilder;
class UMovieSceneTrack;
class ISequencer;
struct FGuid;

/**
 * Modular feature interface for providing custom UI when adding items to Animation Mixer tracks.
 * Implement this interface to provide custom menu entries with specialized UI (e.g., asset pickers, class selectors).
 *
 * If no provider is registered for a class implementing IMovieSceneAnimationSectionInterface or
 * IMovieSceneAnimationTrackInterface, the AnimMixer editor will auto-generate a basic menu entry.
 */
class MOVIESCENETOOLS_API IMovieSceneAnimMixerItemMenuProvider : public IModularFeature
{
public:

	static FName GetModularFeatureName();

	IMovieSceneAnimMixerItemMenuProvider();

	virtual ~IMovieSceneAnimMixerItemMenuProvider();

	/**
	 * Returns the UClass this provider handles.
	 */
	virtual const UClass* GetHandledMixerItemClass() const = 0;

	/**
	 * Populate the menu with custom UI for adding this item type.
	 * Called when building the "Add Section" menu for an Animation Mixer track.
	 * @param MenuBuilder The menu builder to populate
	 * @param ObjectBindings The object bindings for the track
	 * @param Track The track to add items to
	 * @param Sequencer The sequencer instance
	 * @param RowIndex The target row index to add items to (INDEX_NONE for auto-row assignment)
	 */
	virtual void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) = 0;

	/**
	 * Returns the menu priority for this provider. Lower values appear first in the menu.
	 * Default is 100 (middle priority). Use 0-50 for high priority items, 100+ for normal items.
	 */
	virtual int32 GetMixerItemMenuPriority() const { return 100; }

	/**
	 * Populate the Object Binding Animation menu with entries for adding standalone tracks.
	 * Called when building the "Animation" menu/submenu in the Object Binding +Track menu.
	 * Default implementation does nothing - override to add custom menu entries.
	 * @param MenuBuilder The menu builder to populate
	 * @param ObjectBindings The object bindings for the track
	 * @param ObjectClass The class of the bound object
	 * @param bIsInsideSubmenu True if called inside an Animation submenu (can use BeginSection), false if at top-level menu (should use AddSubMenu)
	 */
	virtual void PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu) {}

	/**
	 * Returns the menu priority for the Object Binding Animation menu.
	 * Lower values appear first. Default delegates to GetMixerItemMenuPriority().
	 * Animation Mixer should use 0 to appear first.
	 */
	virtual int32 GetObjectBindingAnimationMenuPriority() const { return GetMixerItemMenuPriority(); }
};
