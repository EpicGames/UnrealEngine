// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "MovieSceneMixedAnimationTarget.h"
#include "StructUtils/InstancedStruct.h"

class FMenuBuilder;
class ISequencer;
class UMovieSceneAnimationMixerTrack;

/**
 * Modular feature interface for providing custom UI when selecting animation mixer targets.
 * Implement this interface to provide custom menu entries for target selection when creating
 * Animation Mixer tracks or changing the target on an existing track.
 *
 * Each target type (e.g. Automatic, Custom Anim Instance, Anim Blueprint Target, UAF Module Injection)
 * should have a corresponding provider that handles its menu population.
 */
class IMovieSceneAnimMixerTargetMenuProvider : public IModularFeature
{
public:
	MOVIESCENETOOLS_API static FName GetModularFeatureName();

	virtual ~IMovieSceneAnimMixerTargetMenuProvider() {}

	/** Returns the target struct type this provider handles (e.g., FMovieSceneMixedAnimationTarget::StaticStruct()) */
	virtual UScriptStruct* GetHandledTargetStructType() const = 0;

	/**
	 * Populate the menu with entries for this target type.
	 * Called when building the target selection menu (both for new track creation and target change).
	 *
	 * For simple targets (like Automatic or Custom Anim Instance), this may add a single menu entry.
	 * For complex targets (like Anim Blueprint Target), this may add a submenu with dynamically
	 * discovered options based on the bound object.
	 *
	 * OnTargetSelected should be invoked by the menu provider with a constructed target once a target is selected.
	 */
	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected,
		TSharedPtr<ISequencer> Sequencer,
		UMovieSceneAnimationMixerTrack* Track)
	{
		PopulateTargetMenu(MenuBuilder, BoundObject, OnTargetSelected);
	}

	virtual void PopulateTargetMenu(
		FMenuBuilder& MenuBuilder,
		TObjectPtr<UObject> BoundObject,
		TFunction<void(TInstancedStruct<FMovieSceneMixedAnimationTarget>)> OnTargetSelected) = 0;

	/**
	 * Returns the menu priority for this provider. Lower values appear first in the menu.
	 *   0 - Automatic (default target)
	 *   10 - Custom Anim Instance
	 *   20 - Anim Blueprint Target
	 *   30 - UAF Module Injection
	 *   100+ - Custom/plugin targets
	 */
	virtual int32 GetTargetMenuPriority() const { return 100; }
};
