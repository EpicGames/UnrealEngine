// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

/**
 * Modular feature interface for plugins that need to wrap the binding-level
 * "Bake Animation Sequence" / "Bake to Control Rig" recording window with
 * custom scope state. Implementations receive a Begin/End pair bracketing the
 * sequencer playback that the recorder runs internally.
 *
 * Current consumer: MovieSceneAnimMixerEditor registers an implementation that
 * pushes UMovieSceneAnimMixerSystem's force-root-bone scope, so mixer root motion
 * lands on the root bone rather than the actor during bake recording.
 */
class IMovieSceneAnimSequenceBakeScope : public IModularFeature
{
public:

	MOVIESCENETOOLS_API static FName GetModularFeatureName();

	virtual ~IMovieSceneAnimSequenceBakeScope() {}

	/** Called once before the recorder begins playback. */
	virtual void BeginBakeScope() = 0;

	/** Called once after the recorder finishes playback (always paired with BeginBakeScope). */
	virtual void EndBakeScope() = 0;
};
