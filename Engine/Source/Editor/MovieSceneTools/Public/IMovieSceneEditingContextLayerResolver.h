// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"

class UMovieSceneEntitySystemLinker;

// Modular feature for resolving an editing context (e.g. a UControlRig) to the
// animation mixer layer it belongs to. This allows systems like the root motion
// offset provider to compute per-layer offsets without hard dependencies on mixer
// or control rig types.
//
// The resolved layer is returned as UObject* to avoid a dependency from
// MovieSceneTools on MovieSceneAnimMixer. Consumers cast to the concrete type.
class IMovieSceneEditingContextLayerResolver : public IModularFeature
{
public:
	virtual ~IMovieSceneEditingContextLayerResolver() = default;

	// Resolve an editing context to the mixer layer it belongs to.
	// Returns nullptr if this resolver doesn't handle the context.
	virtual UObject* ResolveEditingContextToMixerLayer(
		const UMovieSceneEntitySystemLinker* Linker,
		const UObject* EditingContext) const = 0;

	static MOVIESCENETOOLS_API FName GetModularFeatureName();
};
