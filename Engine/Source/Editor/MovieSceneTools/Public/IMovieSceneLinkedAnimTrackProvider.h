// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class UAnimSequence;
class UMovieSceneTrack;
class ISequencer;
struct FGuid;
struct FLevelSequenceAnimSequenceLinkItem;

namespace UE::Sequencer { class FCommonAnimationTrackEditor; }

#define UE_API MOVIESCENETOOLS_API
/**
 * Modular feature interface for providing linked animation track or section operations.
 *
 * Implementations handle finding, creating, isolating, and updating linked animation tracks
 * that contain auto-baked animation sequences. Different providers handle different track
 * configurations (e.g., standalone skeletal animation tracks vs. animation mixer sections/tracks).
 *
 * Providers are queried by priority (lower = higher priority). The first provider whose
 * CanHandleBinding() returns true is used for that binding.
 */
class  IMovieSceneLinkedAnimTrackProvider : public IModularFeature
{
public:

	UE_API static FName GetModularFeatureName();

	virtual ~IMovieSceneLinkedAnimTrackProvider() {}

	/** Priority for provider selection. Lower value = higher priority (checked first). Default provider uses 100. */
	virtual int32 GetLinkedAnimTrackPriority() const = 0;

	/** Returns true if this provider should handle linked anim track operations for this binding. */
	virtual bool CanHandleBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const = 0;

	/**
	 * Find the skeletal animation track(section) containing the linked animation sequence.
	 * Read-only lookup, does not create anything.
	 * @return The UMovieSceneTrack containing the linked anim, or nullptr if not found.
	 */
	virtual UMovieSceneTrack* GetLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) const = 0;

	/**
	 * Find or create the linked animation track setup.
	 * If no linked track exists and creation is needed, creates the necessary track structure,
	 * linked animation sequence, section, and disables evaluation on the containing track.
	 * @param TrackEditor The track editor instance, needed for track creation operations.
	 * @return The UMovieSceneTrack containing the linked anim, or nullptr on failure.
	 */
	virtual UMovieSceneTrack* GetOrCreateLinkedAnimTrack(
		const TWeakPtr<ISequencer>& InSequencer,
		FGuid BindingID,
		UE::Sequencer::FCommonAnimationTrackEditor& TrackEditor) = 0;

	/**
	 * Isolate or un-isolate the linked animation track.
	 * When isolating: enables the linked anim track and disables other animation/control rig tracks.
	 * When un-isolating: restores the previous evaluation state.
	 * @return true if any track evaluation state was changed.
	 */
	virtual bool IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bIsolate) = 0;

	/** Update linked anim section ranges after bake, in case the playback range changed. */
	virtual void UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) = 0;

	/**
	 * Perform the auto-bake for the linked animation on behalf of the recorder.
	 *
	 * Providers that own a richer evaluation model than per-frame skeletal mesh component
	 * sampling can implement this to write directly into the AnimSequence. The mixer
	 * provider, for example, calls its dedicated bake helper which composites the mixer's
	 * RootMotionTransform onto the root bone and sets bEnableRootMotion on the asset.
	 *
	 * Return true if this provider performed the bake. The auto-bake recorder will skip
	 * its frame-by-frame fallback when true is returned. The default implementation
	 * returns false, leaving the recorder to use UAnimSequenceRecorder.
	 */
	virtual bool TryBakeLinkedAnimSequence(
		const TWeakPtr<ISequencer>& InSequencer,
		FGuid BindingID,
		UAnimSequence* InAnimSequence,
		const FLevelSequenceAnimSequenceLinkItem& InLinkItem) { return false; }

	/**
	 * Delete the linked animation track setup for this binding.
	 * @return true if a track was found and removed.
	 */
	virtual bool DeleteLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID) = 0;
};

#undef UE_API

