// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"

class UMovieSceneEntitySystemLinker;

// Modular feature for systems that apply root motion to scene components.
// Allows the transform track editor to subtract root motion when auto-keying,
// preventing double-application of root motion offsets.
class IMovieSceneRootMotionOffsetProvider : public IModularFeature
{
public:
	virtual ~IMovieSceneRootMotionOffsetProvider() = default;

	// Returns the root motion offset currently applied to the given object,
	// or Identity if this provider doesn't handle the object.
	virtual FTransform GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) const = 0;

	// Returns the root motion offset for a specific editing context (e.g. a UControlRig
	// on a particular mixer layer). When EditingContext is non-null, the provider may
	// return a per-layer offset rather than the total. Default delegates to the 2-arg version.
	virtual FTransform GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject, const UObject* EditingContext) const
	{
		return GetRootMotionOffset(Linker, AnimatedObject);
	}

	// Recalculates the initial actor transform baseline after the user manually
	// changes the transform.
	virtual void UpdateRootMotionOffset(UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) {}

	static MOVIESCENETOOLS_API FName GetModularFeatureName();
};
