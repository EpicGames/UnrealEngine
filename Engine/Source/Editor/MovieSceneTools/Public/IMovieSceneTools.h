// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * The public interface of the MovieSceneTools module
 */
class IMovieSceneTools : public IModuleInterface
{

public:

};

