// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Curves/RichCurve.h"
#include "Containers/UnrealString.h"

class UMovieScene;

class IMovieSceneToolsTrackImporter
{
public:

	/*
	 * ImportAnimatedProperty
	 * 
	 * @param InPropertyName The name of the property to import
	 * @param InCurve The curve with the keys to import
	 * @param InBinding The binding to import the property onto
	 * @param InMovieScene The movie scene that contains the binding to import onto
	 * @return Whether the property was imported successfully
	 */
	virtual bool ImportAnimatedProperty(const FString& InPropertyName, const FRichCurve& InCurve, FGuid InBinding, UMovieScene* InMovieScene) = 0;

	/*
	 * ImportStringProperty
	 * 
	 * @param InPropertyName The name of the property to import
	 * @param InStringValue The value of the string to import
	 * @param InBinding The binding to import the property onto
	 * @param InMovieScene The movie scene that contains the binding to import onto
	 * @return Whether the property was imported successfully
	 */
	virtual bool ImportStringProperty(const FString& InPropertyName, const FString& InStringValue, FGuid InBinding, UMovieScene* InMovieScene) = 0;
};