// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"

#include "K2Node_GetSequenceBinding.generated.h"


struct FAssetData;
class UMovieSceneSequence;
struct FKismetFunctionContext;
struct FMovieSceneSequenceID;

UCLASS(MinimalAPI)
class UK2Node_GetSequenceBinding
	: public UK2Node
{
public:
	GENERATED_BODY()

	UPROPERTY()
	FSoftObjectPath SourceSequence_DEPRECATED;

	/** The user-selected literal binding identifier from the sequence to use */
	UPROPERTY()
	FMovieSceneObjectBindingID Binding;

	MOVIESCENETOOLS_API void SetSequence(UMovieSceneSequence* InSequence);

public:
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UK2Node interface
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void PreloadRequiredAssets() override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void AllocateDefaultPins() override;
	virtual void PostPlacedNewNode() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	// End of UK2Node interface

	void SetSequence(const FAssetData& InAssetData);
	FText GetSequenceName() const;
	FText GetBindingName() const;

private:
	UMovieScene* GetObjectMovieScene() const;

	/** The sequence from which to choose a binding identifier */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(DisplayName="Source Sequence"))
	TObjectPtr<UMovieSceneSequence> SourceMovieSequence;

	mutable FMovieSceneSequenceHierarchy SequenceHierarchyCache;
	mutable TMap<FMovieSceneSequenceID, FGuid> SequenceSignatureCache;
};
