// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerKeyStructGenerator.h"
#include "MovieSceneTextKeyStruct.generated.h"

/**
 * Derived only to explicitly specify that it's Editor Only
 * to be ignored in Cooked Builds as these KeyStructs are instanced under a non-editor package for text localization
 * @see InstanceGeneratedStruct overload for FMovieSceneTextChannel
 */
UCLASS(MinimalAPI)
class UMovieSceneTextKeyStruct : public UMovieSceneKeyStructType
{
	GENERATED_BODY()

public:
	//~ Begin UObject
	virtual bool IsEditorOnly() const override { return true; }
	//~ End UObject
};
