// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "KeyParams.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "ScopedTransaction.h"
#include "MovieSceneTrackEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieSceneCommonHelpers.h"
#include "Channels/MovieSceneUnpackedChannelValues.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneBoolChannel.h"


enum class EKeyFrameTrackEditorSetDefault
{
	/** Do not set default */
	DoNotSetDefault,
	/** Set default just on keys that have bAddKey set */
	SetDefaultOnAddKeys,
	/** Set default */
	SetDefault,
};

struct IImpl
{
	virtual ~IImpl() = default;

	/* Returns whether a key was created */
	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const { return false; }

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const { }

	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) { return false; }

};

//If this implementation changes need to change the specializations below.
template<typename ChannelType, typename ValueType>
struct TAddKeyImpl : IImpl
{
	int32 ChannelIndex;
	bool bAddKey;
	ValueType ValueToSet;

	TAddKeyImpl(int32 InChannelIndex, bool bInAddKey, const ValueType& InValue)
		: ChannelIndex(InChannelIndex), bAddKey(bInAddKey), ValueToSet(InValue)
	{}

	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
	{
		bool bKeyCreated = false;
		using namespace UE::MovieScene;

		ChannelType* Channel = Proxy.GetChannel<ChannelType>(ChannelIndex);
		if (bAddKey && Channel)
		{
			bool bShouldKeyChannel = bKeyEvenIfUnchanged;
			if (!bShouldKeyChannel)
			{
				bShouldKeyChannel = !ValueExistsAtTime(Channel, InTime, ValueToSet);
			}

			if (bShouldKeyChannel)
			{
				if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
				{
					if (Section->TryModify())
					{
						InterpolationMode = GetInterpolationMode(Channel, InTime, InterpolationMode);
						AddKeyToChannel(Channel, InTime, ValueToSet, InterpolationMode);
						bKeyCreated = true;
					}
				}
			}
		}

		return bKeyCreated;
	}

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
	{
		using namespace UE::MovieScene;

		ChannelType* Channel = Proxy.GetChannel<ChannelType>(ChannelIndex);
		if (Channel && Channel->GetData().GetTimes().Num() == 0)
		{
			if (Section->TryModify())
			{
				using namespace UE::MovieScene;
				SetChannelDefault(Channel, ValueToSet);
			}
		}
	}
	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) { return false; }

};

//bool specialization
template<>
struct TAddKeyImpl<FMovieSceneBoolChannel, bool> : IImpl
{
	int32 ChannelIndex;
	bool bAddKey;
	bool ValueToSet;

	TAddKeyImpl(int32 InChannelIndex, bool bInAddKey, const bool& InValue)
		: ChannelIndex(InChannelIndex), bAddKey(bInAddKey), ValueToSet(InValue)
	{}

	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
	{
		bool bKeyCreated = false;
		using namespace UE::MovieScene;

		FMovieSceneBoolChannel* Channel = Proxy.GetChannel<FMovieSceneBoolChannel>(ChannelIndex);
		if (bAddKey && Channel)
		{
			bool bShouldKeyChannel = bKeyEvenIfUnchanged;
			if (!bShouldKeyChannel)
			{
				bShouldKeyChannel = !ValueExistsAtTime(Channel, InTime, ValueToSet);
			}

			if (bShouldKeyChannel)
			{
				if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
				{
					if (Section->TryModify())
					{
						InterpolationMode = GetInterpolationMode(Channel, InTime, InterpolationMode);
						AddKeyToChannel(Channel, InTime, ValueToSet, InterpolationMode);
						bKeyCreated = true;
					}
				}
			}
		}

		return bKeyCreated;
	}

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
	{
		using namespace UE::MovieScene;

		FMovieSceneBoolChannel* Channel = Proxy.GetChannel<FMovieSceneBoolChannel>(ChannelIndex);
		if (Channel && Channel->GetData().GetTimes().Num() == 0 && Channel->GetDefault() != ValueToSet
			&& (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefault || (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys && bAddKey)))
		{
			if (Section->TryModify())
			{
				using namespace UE::MovieScene;
				SetChannelDefault(Channel, ValueToSet);
			}
		}
	}
	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) { return false; }

};
//Specializations for channels that SUPPORT Blending
template<>
struct TAddKeyImpl<FMovieSceneDoubleChannel, double> : IImpl
{
	int32 ChannelIndex;
	bool bAddKey;
	double ValueToSet;

	TAddKeyImpl(int32 InChannelIndex, bool bInAddKey,  const double& InValue)
		: ChannelIndex(InChannelIndex), bAddKey(bInAddKey), ValueToSet(InValue)
	{}

	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
	{
		bool bKeyCreated = false;
		using namespace UE::MovieScene;

		FMovieSceneDoubleChannel* Channel = Proxy.GetChannel<FMovieSceneDoubleChannel>(ChannelIndex);
		if (bAddKey && Channel)
		{
			bool bShouldKeyChannel = bKeyEvenIfUnchanged;
			if (!bShouldKeyChannel)
			{
				bShouldKeyChannel = !ValueExistsAtTime(Channel, InTime, ValueToSet);
			}

			if (bShouldKeyChannel)
			{
				if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
				{
					if (Section->TryModify())
					{
						InterpolationMode = GetInterpolationMode(Channel, InTime, InterpolationMode);
						AddKeyToChannel(Channel, InTime, ValueToSet, InterpolationMode);
						bKeyCreated = true;
					}
				}
			}
		}

		return bKeyCreated;
	}

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
	{
		using namespace UE::MovieScene;

		FMovieSceneDoubleChannel* Channel = Proxy.GetChannel<FMovieSceneDoubleChannel>(ChannelIndex);
		if (Channel && Channel->GetData().GetTimes().Num() == 0  && Channel->GetDefault() != ValueToSet
			&& (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefault || (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys && bAddKey)))
		{
			if (Section->TryModify())
			{
				using namespace UE::MovieScene;
				SetChannelDefault(Channel, ValueToSet);
			}
		}
	}

	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) override
	{
		using namespace UE::MovieScene;
		double CurrentValue = *(double*)(VCurrentValue);
		FMovieSceneDoubleChannel* Channel = Proxy.GetChannel<FMovieSceneDoubleChannel>(ChannelIndex);
		if (Channel)
		{
			double LocalValue;
			using namespace UE::MovieScene;
			if (!EvaluateChannel(Channel, InTime, LocalValue))
			{
				TOptional<double> OptDouble = Channel->GetDefault();
				LocalValue = OptDouble.IsSet() ? OptDouble.GetValue() : 0.0f;
			}
			ValueToSet = (ValueToSet - CurrentValue) * Weight + LocalValue;
			return true;
		}

		return false;
	}
};
template<>
struct TAddKeyImpl<FMovieSceneFloatChannel, float> : IImpl
{
	int32 ChannelIndex;
	bool bAddKey;
	float ValueToSet;

	TAddKeyImpl(int32 InChannelIndex, bool bInAddKey,  const float& InValue)
		: ChannelIndex(InChannelIndex), bAddKey(bInAddKey), ValueToSet(InValue)
	{}

	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
	{
		bool bKeyCreated = false;
		using namespace UE::MovieScene;

		FMovieSceneFloatChannel* Channel = Proxy.GetChannel<FMovieSceneFloatChannel>(ChannelIndex);
		if (bAddKey && Channel)
		{
			bool bShouldKeyChannel = bKeyEvenIfUnchanged;
			if (!bShouldKeyChannel)
			{
				bShouldKeyChannel = !ValueExistsAtTime(Channel, InTime, ValueToSet);
			}

			if (bShouldKeyChannel)
			{
				if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
				{
					if (Section->TryModify())
					{
						InterpolationMode = GetInterpolationMode(Channel, InTime, InterpolationMode);
						AddKeyToChannel(Channel, InTime, ValueToSet, InterpolationMode);
						bKeyCreated = true;
					}
				}
			}
		}

		return bKeyCreated;
	}

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
	{
		using namespace UE::MovieScene;

		FMovieSceneFloatChannel* Channel = Proxy.GetChannel<FMovieSceneFloatChannel>(ChannelIndex);
		if (Channel && Channel->GetData().GetTimes().Num() == 0  && Channel->GetDefault() != ValueToSet
			&& (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefault || (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys && bAddKey)))
		{
			if (Section->TryModify())
			{
				using namespace UE::MovieScene;
				SetChannelDefault(Channel, ValueToSet);
			}
		}
	}

	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) override
	{
		using namespace UE::MovieScene;
		float CurrentValue = *(float*)(VCurrentValue);
		FMovieSceneFloatChannel* Channel = Proxy.GetChannel<FMovieSceneFloatChannel>(ChannelIndex);
		if (Channel)
		{
			float LocalValue;
			using namespace UE::MovieScene;
			if (!EvaluateChannel(Channel, InTime, LocalValue))
			{
				TOptional<float> OptFloat = Channel->GetDefault();
				LocalValue = OptFloat.IsSet() ? OptFloat.GetValue() : 0.0f;
			}
			ValueToSet = (ValueToSet - CurrentValue) * Weight + LocalValue;
			return true;
		}

		return false;
	}
};
template<>
struct TAddKeyImpl<FMovieSceneIntegerChannel, int32> : IImpl
{
	int32 ChannelIndex;
	bool bAddKey;
	int32 ValueToSet;

	TAddKeyImpl(int32 InChannelIndex, bool bInAddKey, const int32& InValue)
		: ChannelIndex(InChannelIndex),bAddKey(bInAddKey), ValueToSet(InValue)
	{}

	virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
	{
		bool bKeyCreated = false;
		using namespace UE::MovieScene;

		FMovieSceneIntegerChannel* Channel = Proxy.GetChannel<FMovieSceneIntegerChannel>(ChannelIndex);
		if (bAddKey && Channel)
		{
			bool bShouldKeyChannel = bKeyEvenIfUnchanged;
			if (!bShouldKeyChannel)
			{
				bShouldKeyChannel = !ValueExistsAtTime(Channel, InTime, ValueToSet);
			}

			if (bShouldKeyChannel)
			{
				if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
				{
					if (Section->TryModify())
					{
						InterpolationMode = GetInterpolationMode(Channel, InTime, InterpolationMode);
						AddKeyToChannel(Channel, InTime, ValueToSet, InterpolationMode);
						bKeyCreated = true;
					}
				}
			}
		}

		return bKeyCreated;
	}

	virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
	{
		using namespace UE::MovieScene;

		FMovieSceneIntegerChannel* Channel = Proxy.GetChannel<FMovieSceneIntegerChannel>(ChannelIndex);
		if (Channel && Channel->GetData().GetTimes().Num() == 0 && Channel->GetDefault() != ValueToSet
			&& (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefault || (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys && bAddKey)))
		{
			if (Section->TryModify())
			{
				using namespace UE::MovieScene;
				SetChannelDefault(Channel, ValueToSet);
			}
		}
	}

	virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) override
	{
		using namespace UE::MovieScene;
		int32 CurrentValue = *(int32*)(VCurrentValue);
		FMovieSceneIntegerChannel* Channel = Proxy.GetChannel<FMovieSceneIntegerChannel>(ChannelIndex);
		if (Channel)
		{
			int32 LocalValue;
			using namespace UE::MovieScene;
			if (!EvaluateChannel(Channel, InTime, LocalValue))
			{
				TOptional<int32> OptInt = Channel->GetDefault();
				LocalValue = OptInt.IsSet() ? OptInt.GetValue() : 0;
			}
			ValueToSet = FMath::TruncToInt32((float)((ValueToSet - CurrentValue)) * Weight + LocalValue);
			return true;
		}

		return false;
	}
};
struct FMovieSceneChannelValueSetter
{
	FMovieSceneChannelValueSetter(const FMovieSceneChannelValueSetter&) = delete;
	FMovieSceneChannelValueSetter& operator=(const FMovieSceneChannelValueSetter&) = delete;

	FMovieSceneChannelValueSetter(FMovieSceneChannelValueSetter&&) = default;
	FMovieSceneChannelValueSetter& operator=(FMovieSceneChannelValueSetter&&) = default;

	/** Templated construction function that can add a key (and potentially also set a default) for the specified channel and value */
	template<typename ChannelType, typename ValueType>
	static FMovieSceneChannelValueSetter Create(int32 ChannelIndex, ValueType&& InNewValue, bool bAddKey)
	{
		FMovieSceneChannelValueSetter NewValue;
		NewValue.Impl = TAddKeyImpl<ChannelType, typename TDecay<ValueType>::Type>(ChannelIndex, bAddKey, Forward<ValueType>(InNewValue));
		
		return MoveTemp(NewValue);
	}

	/** Templated construction function that can add a key (and potentially also set a default) for the specified channel and value */
	static FMovieSceneChannelValueSetter Create(UE::MovieScene::FUnpackedChannelValue&& InValue, bool bAddKey)
	{
		FMovieSceneChannelValueSetter NewValue;
		NewValue.Impl = FUnpackedChannelValueSetter(MoveTemp(InValue), bAddKey);
		return MoveTemp(NewValue);
	}

	IImpl* operator->()
	{
		return &Impl.GetValue();
	}

	const IImpl* operator->() const
	{
		return &Impl.GetValue();
	}

private:

	FMovieSceneChannelValueSetter()
	{}

	struct FUnpackedChannelValueSetter : IImpl
	{
		UE::MovieScene::FUnpackedChannelValue ChannelValue;
		bool bAddKey;

		FUnpackedChannelValueSetter(UE::MovieScene::FUnpackedChannelValue&& InValue, bool bInAddKey)
			: ChannelValue(MoveTemp(InValue)), bAddKey(bInAddKey)
		{}

		virtual bool Apply(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode, bool bKeyEvenIfUnchanged, bool bKeyEvenIfEmpty) const override
		{
			bool bKeyCreated = false;
			using namespace UE::MovieScene;

			FMovieSceneChannel* Channel = ChannelValue->RetrieveChannel(Proxy);
			if (bAddKey && Channel)
			{
				bool bShouldKeyChannel = bKeyEvenIfUnchanged;
				if (!bShouldKeyChannel)
				{
					bShouldKeyChannel = !ChannelValue->AlreadyExistsAtTime(Channel, InTime);
				}

				if (bShouldKeyChannel)
				{
					if (Channel->GetNumKeys() != 0 || bKeyEvenIfEmpty)
					{
						if (Section->TryModify())
						{
							ChannelValue->AddKey(Channel, InTime, InterpolationMode);
							bKeyCreated = true;
						}
					}
				}
			}

			return bKeyCreated;
		}

		virtual void ApplyDefault(UMovieSceneSection* Section, FMovieSceneChannelProxy& Proxy, EKeyFrameTrackEditorSetDefault SetDefault) const override
		{
			using namespace UE::MovieScene;

			const bool bShouldSetDefault = SetDefault == EKeyFrameTrackEditorSetDefault::SetDefault || (SetDefault == EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys && bAddKey);
			FMovieSceneChannel* Channel = ChannelValue->RetrieveChannel(Proxy);
			if (bShouldSetDefault && Channel && Channel->GetNumKeys() == 0 && !ChannelValue->IsAlreadyDefault(Channel))
			{
				if (Section->TryModify())
				{
					ChannelValue->SetDefault(Channel);
				}
			}
		}

		virtual bool ModifyByCurrentAndWeight(FMovieSceneChannelProxy& Proxy, FFrameNumber InTime, void* VCurrentValue, float Weight) override
		{
			ensure(!"This should never be called any more");
			return false;
		}
	};

	TInlineValue<IImpl> Impl;
};

typedef TArray<FMovieSceneChannelValueSetter, TInlineAllocator<1>> FGeneratedTrackKeys;

/**
 * A base class for track editors that edit tracks which contain sections implementing GetKeyDataInterface.
  */
template<typename TrackType>
class FKeyframeTrackEditor : public FMovieSceneTrackEditor
{
public:

	using GenerateKeysCallback = TFunction<void(UMovieSceneSection*, FGeneratedTrackKeys&)>;

	/**
	* Constructor
	*
	* @param InSequencer The sequencer instance to be used by this tool
	*/
	FKeyframeTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FMovieSceneTrackEditor( InSequencer )
	{ }

	/** Virtual destructor. */
	~FKeyframeTrackEditor() = default;

public:

	// ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override
	{
		return NSLOCTEXT("Sequencer", "KeyframeTrackEditor_DisplayName", "Keyframe");
	}

	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override
	{
		return Type == TrackType::StaticClass();
	}


	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override
	{
		MenuBuilder.AddSubMenu(
			NSLOCTEXT("KeyframeTrackEditor", "TrackDefaults", "Track Defaults"),
			NSLOCTEXT("KeyframeTrackEditor", "TrackDefaultsTooltip", "Track default value operations."),
			FNewMenuDelegate::CreateSP(this, &FKeyframeTrackEditor::AddTrackDefaultsItems, Track));
	}

protected:

	/*
	 * Adds keys to the specified object.  This may also add tracks and sections depending on the options specified. 
	 *
	 * @param ObjectsToKey An array of objects to add keyframes to.
	 * @param KeyTime The time to add keys.
	 * @param KeyedChannels Aggregate of optionally enabled channel values for which to create new keys
	 * @param ChannelDefaults Aggregate of optionally enabled channel default values that should be applied to the section
	 * @param KeyParams The parameters to control keyframing behavior.
	 * @param TrackClass The class of track which should be created if specified in the parameters.
	 * @param PropertyName The name of the property to add keys for.
	 * @param OnInitializeNewTrack A delegate which allows for custom initialization for new tracks.  This is called after the 
	 *        track is created, but before any sections or keys have been added.
	 * @return Whether or not a handle guid or track was created. Note this does not return true if keys were added or modified.
	 */
	FKeyPropertyResult AddKeysToObjects(
		TArrayView<UObject* const> ObjectsToKey, FFrameNumber KeyTime,
		ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName PropertyName,
		TFunction<void(TrackType*)> OnInitializeNewTrack,
		const GenerateKeysCallback& OnGenerateKeys)
	{
		FKeyPropertyResult KeyPropertyResult;

		EAutoChangeMode AutoChangeMode = GetSequencer()->GetAutoChangeMode();
		EAllowEditsMode AllowEditsMode = GetSequencer()->GetAllowEditsMode();

		bool bCreateHandle =
			(KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode == EAutoChangeMode::AutoTrack || AutoChangeMode == EAutoChangeMode::All)) ||
			KeyMode == ESequencerKeyMode::ManualKey ||
			KeyMode == ESequencerKeyMode::ManualKeyForced ||
			AllowEditsMode == EAllowEditsMode::AllowSequencerEditsOnly;

		bool bShouldPlaceInChildFolder = KeyMode == ESequencerKeyMode::AutoKey;
		const FName CreatedFolderName = bShouldPlaceInChildFolder ? FName(TEXT("Autotracked Changes")) : NAME_None;

		for ( UObject* Object : ObjectsToKey )
		{
			FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object, bCreateHandle, CreatedFolderName);
			FGuid ObjectHandle = HandleResult.Handle;
			KeyPropertyResult.bHandleCreated = HandleResult.bWasCreated;

			if ( ObjectHandle.IsValid() )
			{
				KeyPropertyResult |= AddKeysToHandle( Object, ObjectHandle, KeyTime, KeyMode, TrackClass, PropertyName, OnInitializeNewTrack, OnGenerateKeys );
			}
		}
		return KeyPropertyResult;
	}

private:

	void ClearDefaults( UMovieSceneTrack* Track )
	{
		const FScopedTransaction Transaction(NSLOCTEXT("KeyframeTrackEditor", "ClearTrackDefaultsTransaction", "Clear track defaults"));
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			Section->Modify();

			// Clear all defaults on the section
			for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
			{
				for (FMovieSceneChannel* Channel : Entry.GetChannels())
				{
					Channel->ClearDefault();
				}
			}
		}
		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	void AddTrackDefaultsItems( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track )
	{
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT( "KeyframeTrackEditor", "ClearDefaults", "Clear Defaults" ),
			NSLOCTEXT( "KeyframeTrackEditor", "ClearDefaultsToolTip", "Clear the current default values for this track." ),
			FSlateIcon(),
			FUIAction( FExecuteAction::CreateSP( this, &FKeyframeTrackEditor::ClearDefaults, Track ) ),
			NAME_None,
			EUserInterfaceActionType::Button );
	}

	/*
	 * Adds keys to the specified guid.  This may also add tracks and sections depending on the options specified. 
	 *
	 * @param Object Object to add keyframes to.
	 * @param ObjectHandle Object Handle to add keyframes to.
	 * @param KeyTime The time to add keys.
	 * @param GeneratedKeys Array of keys to set
	 * @param KeyParams The parameters to control keyframing behavior.
	 * @param TrackClass The class of track which should be created if specified in the parameters.
	 * @param PropertyName The name of the property to add keys for.
	 * @param OnInitializeNewTrack A delegate which allows for custom initialization for new tracks.  This is called after the 
	 *        track is created, but before any sections or keys have been added.
	 * @return Whether or not a track was created. Note this does not return true if keys were added or modified.
	*/
	FKeyPropertyResult AddKeysToHandle(UObject *Object,
		FGuid ObjectHandle, FFrameNumber KeyTime,
		ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName PropertyName,
		TFunction<void(TrackType*)> OnInitializeNewTrack,
		const GenerateKeysCallback& OnGenerateKeys)
	{
		bool bTrackCreated = false;

		EAutoChangeMode AutoChangeMode = GetSequencer()->GetAutoChangeMode();
		EAllowEditsMode AllowEditsMode = GetSequencer()->GetAllowEditsMode();

		bool bCreateTrack =
			(KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode == EAutoChangeMode::AutoTrack || AutoChangeMode == EAutoChangeMode::All)) ||
			KeyMode == ESequencerKeyMode::ManualKey ||
			KeyMode == ESequencerKeyMode::ManualKeyForced ||
			AllowEditsMode == EAllowEditsMode::AllowSequencerEditsOnly;

		bool bCreateSection = bCreateTrack || 
			(KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode != EAutoChangeMode::None));

		// Try to find an existing Track, and if one doesn't exist check the key params and create one if requested.
		FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject( ObjectHandle, TrackClass, PropertyName, bCreateTrack );
		TrackType* Track = CastChecked<TrackType>( TrackResult.Track, ECastCheckedType::NullAllowed );
		bTrackCreated = TrackResult.bWasCreated;

		if ( bTrackCreated )
		{
			if (OnInitializeNewTrack)
			{
				OnInitializeNewTrack(Track);
			}
		}

		bool bSectionCreated = false;

		FKeyPropertyResult KeyPropertyResult;

		if ( Track )
		{
			float Weight = 1.0f;
			UMovieSceneSection* SectionToKey = bCreateSection ? Track->FindOrExtendSection(KeyTime, Weight) : Track->FindSection(KeyTime);

			// If there's no overlapping section to key, create one only if a track was newly created. Otherwise, skip keying altogether
			// so that the user is forced to create a section to key on.
			if (bTrackCreated && !SectionToKey)
			{
				Track->Modify();
				SectionToKey = Track->FindOrAddSection(KeyTime, bSectionCreated);
				if (bSectionCreated && GetSequencer()->GetInfiniteKeyAreas())
				{
					SectionToKey->SetRange(TRange<FFrameNumber>::All());
					KeyPropertyResult.SectionsCreated.Add(SectionToKey);
				}
			}

			if (SectionToKey && CanAutoKeySection(SectionToKey, KeyTime) && OnGenerateKeys)
			{
				FGeneratedTrackKeys GeneratedKeys;
				OnGenerateKeys(SectionToKey, GeneratedKeys);
				KeyPropertyResult |= AddKeysToSection( SectionToKey, KeyTime, GeneratedKeys, KeyMode );
			}
		}

		KeyPropertyResult.bTrackCreated |= bTrackCreated || bSectionCreated;

		return KeyPropertyResult;
	}

protected:
	/* Returns whether a section was added */
	FKeyPropertyResult AddKeysToSection(UMovieSceneSection* Section, FFrameNumber KeyTime, const FGeneratedTrackKeys& Keys, ESequencerKeyMode KeyMode, EKeyFrameTrackEditorSetDefault SetDefault = EKeyFrameTrackEditorSetDefault::SetDefault)
	{
		FKeyPropertyResult KeyPropertyResult;

		EAutoChangeMode AutoChangeMode = GetSequencer()->GetAutoChangeMode();

		FMovieSceneChannelProxy& Proxy = Section->GetChannelProxy();
			
		const bool bSetDefaults = GetSequencer()->GetAutoSetTrackDefaults() && (SetDefault != EKeyFrameTrackEditorSetDefault::DoNotSetDefault);

		// The default value is a value for the channel when there are no keyframes. For example, if you add keys and 
		// then delete them all, the default value is the value of the channel. In the implementation of ApplyDefault, 
		// all the setters check that the default value is only set when there are NO keyframes. So, ApplyDefault needs 
		// to be called here in AddKeysToSection BEFORE any keys are added.
		if (bSetDefaults)
		{
			for (const FMovieSceneChannelValueSetter& GeneratedKey : Keys)
			{
				GeneratedKey->ApplyDefault(Section, Proxy, SetDefault);
			}
		}

		if ( KeyMode != ESequencerKeyMode::AutoKey || AutoChangeMode == EAutoChangeMode::AutoKey || AutoChangeMode == EAutoChangeMode::All)
		{
			EMovieSceneKeyInterpolation InterpolationMode = GetSequencer()->GetKeyInterpolation();

			const bool bKeyEvenIfUnchanged =
				KeyMode == ESequencerKeyMode::ManualKeyForced ||
				GetSequencer()->GetKeyGroupMode() == EKeyGroupMode::KeyAll ||
				GetSequencer()->GetKeyGroupMode() == EKeyGroupMode::KeyGroup;

			const bool bKeyEvenIfEmpty =
				(KeyMode == ESequencerKeyMode::AutoKey && AutoChangeMode == EAutoChangeMode::All) ||
				KeyMode == ESequencerKeyMode::ManualKeyForced;

			for (const FMovieSceneChannelValueSetter& GeneratedKey : Keys)
			{
				KeyPropertyResult.bKeyCreated |= GeneratedKey->Apply(Section, Proxy, KeyTime, InterpolationMode, bKeyEvenIfUnchanged, bKeyEvenIfEmpty);
				if (KeyPropertyResult.bKeyCreated)
				{
					KeyPropertyResult.SectionsKeyed.Add(Section);
				}
			}
		}
			
		return KeyPropertyResult;
	}

	UE_DEPRECATED(5.7, "This function is no longer called. Please ensure that generated keys populated through GenerateKeysCallback have already been compensated.")
	virtual bool ModifyGeneratedKeysByCurrentAndWeight(UObject *Object, UMovieSceneTrack *Track, UMovieSceneSection* SectionToKey, FFrameNumber Time, FGeneratedTrackKeys& GeneratedTotalKeys, float Weight) const
	{
		return false;
	}

	/** Check whether we can autokey the specified section at the specified time */
	static bool CanAutoKeySection(UMovieSceneSection* Section, FFrameNumber Time)
	{
		// Sections are only eligible for autokey if they overlap the current time
		return Section->GetRange().Contains(Time);
	}
};

