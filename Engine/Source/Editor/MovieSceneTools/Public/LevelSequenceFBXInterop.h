// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"

#define UE_API MOVIESCENETOOLS_API

class FLevelSequenceFBXInterop
{
public:

	UE_API FLevelSequenceFBXInterop(TSharedRef<ISequencer> InSequencer);

	/** Imports the animation from an fbx file. */
	UE_API void ImportFBX();
	UE_API void ImportFBXOntoSelectedNodes();

	/** Exports the animation to an fbx file. */
	UE_API void ExportFBX();

private:

	/** Exports sequence to a FBX file */
	UE_API void ExportFBXInternal(const FString& ExportFilename, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks);

private:

	TWeakPtr<ISequencer> WeakSequencer;
};

#undef UE_API
