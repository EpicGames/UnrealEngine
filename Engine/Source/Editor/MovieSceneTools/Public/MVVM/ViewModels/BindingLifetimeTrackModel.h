// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "Templates/SharedPointer.h"
#include "MVVM/Extensions/IBindingLifetimeExtension.h"

#define UE_API MOVIESCENETOOLS_API

class UMovieSceneBindingLifetimeTrack;
class UMovieSceneTrack;
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }

namespace UE
{
	namespace Sequencer
	{

		class FBindingLifetimeTrackModel
			: public FTrackModel
			, public IBindingLifetimeExtension
		{
		public:

			UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FBindingLifetimeTrackModel, FTrackModel, IBindingLifetimeExtension);

			static UE_API TSharedPtr<FTrackModel> CreateTrackModel(UMovieSceneTrack* Track);

			UE_API explicit FBindingLifetimeTrackModel(UMovieSceneBindingLifetimeTrack* Track);

			UE_API FSortingKey GetSortingKey() const override;

			UE_API void OnDeferredModifyFlush() override;

			// IBindingLifetimeExtension
			const TArray<FFrameNumberRange>& GetInverseLifetimeRange() const override { return InverseLifetimeRange; }
		
		protected:
			UE_API virtual void OnConstruct() override;

		private:
			UE_API void RecalculateInverseLifetimeRange();

			// The inverse of the range created by all of our binding lifetime sections
			// In other words, the ranges where the object binding should be deactivated.
			TArray<FFrameNumberRange> InverseLifetimeRange;

		};

	} // namespace Sequencer
} // namespace UE

#undef UE_API
