// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "IPropertyTypeCustomization.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "Types/SlateEnums.h"

#define UE_API MOVIESCENETOOLS_API

class UK2Node;
class UBlueprint;
class UEdGraphPin;
class UEdGraphNode;
class FMenuBuilder;
class FStructOnScope;
class IPropertyHandle;
class FDetailWidgetRow;
class SWidget;
class UMovieSceneSection;
class UMovieSceneSequence;
class UK2Node_CustomEvent;
class UK2Node_FunctionEntry;
class IDetailChildrenBuilder;
struct FAssetData;
struct FBlueprintActionMenuBuilder;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

enum class ECheckBoxState : uint8;

enum class EAutoCreatePayload : uint8
{
	None      = 0,
	Pins      = 1 << 0,
	Variables = 1 << 1,
};
ENUM_CLASS_FLAGS(EAutoCreatePayload)

DECLARE_DELEGATE_TwoParams(FOnActionSelected, const TArray< TSharedPtr<FEdGraphSchemaAction> >&, ESelectInfo::Type);
DECLARE_DELEGATE_FourParams(FOnQuickBindActionSelected, const TArray<TSharedPtr<FEdGraphSchemaAction>>&, ESelectInfo::Type, UBlueprint*, FMovieSceneDirectorBlueprintEndpointDefinition);

/**
 * Base class for details view customizations that operate on sequence director blueprint endpoints.
 */
class FMovieSceneDirectorBlueprintEndpointCustomization : public IPropertyTypeCustomization
{
public:

	// IPropertyTypeCustomization interface
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;


	// These functions exposed below so this customization can act as a helper to another

	/**
	 * Creates a single new endpoint represented by this property handle.
	 */
	UE_API void CreateEndpoint();

	/**
	 * Generate the content of the quick bind sub-menu dropdown (shown if the endpoint is not already bound)
	 */
	UE_API void PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSequence* Sequence, FOnQuickBindActionSelected InOnQuickBindActionSelected);

	/*
	* Called when a quick bind action has been selected
	*/
	UE_API void HandleQuickBindActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition);

	/* Set the property handle to use for customization. Used when using the customization class as a helper. */
	UE_API void SetPropertyHandle(TSharedPtr<IPropertyHandle> InPropertyHandle);

	/* Set the raw data to use for customization. Used when the customization class is used as a helper without a details view. */
	void SetRawData(const TArray<void*>& InRawData) { PropertyRawData = InRawData; }

protected:

	using FPayloadVariableMap = TMap<FName, FMovieSceneDirectorBlueprintVariableValue, TInlineSetAllocator<8>>;

	/**
	 * User-interface information for "well-known parameters", i.e. parameters that can be bound
	 * to values provided by the underlying system, as opposed to values entered by the user.
	 */
	struct FWellKnownParameterMetadata
	{
		FText PickerLabel;
		FText PickerTooltip;
		FText CandidateTooltip;
	};

	/**
	 * Structure specifying candidate pins for a given "well-known parameter".
	 */
	struct FWellKnownParameterCandidates
	{
		/** Names of the pins that can be bound to the "well-known parameter" described in the metadata */
		TArray<FName> CandidatePinNames;
		/** User-interface information for the "well-known parameter" */
		FWellKnownParameterMetadata Metadata;
		/** Whether to show this "well-known" parameter when no candidate pins have been found */
		bool bShowUnmatchedParameters = false;
	};

	/**
	 * Gets the payload values to pass to the endpoint. 
	 *
	 * @param EditObject		The edited object for which to get the endpoint call payload variables
	 * @param RawData			The raw data for which to get the endpoint call payload variables
	 * @param OutPayloadVariables	The string values of the payload variables to pass to the endpoint
	 */
	virtual void GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const = 0;

	/**
	 * Sets a payload value to pass to the endpoint.
	 *
	 * @param EditObject		The edited object for which to set the endpoint call payload variable
	 * @param RawData			The raw data for which to set the endpoint call payload variable
	 * @param FieldName			The name of the payload variable to set
	 * @param NewVariableValue	The string value of the payload variable to set
	 * @param NewVariableObject	In the case the payload is a UObject, a reference to the UObject
	 * @return					Whether the payload variable was set
	 */
	virtual bool SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewVariableValue) = 0;

	/**
	 * Get the pin names setup with "well-known parameters" for the currently edited objects.
	 *
	 * The output array of pin names should be of the same size as GetWellKnownParameterCandidates, with NAME_None
	 * for those parameters that aren't hooked up to any pin.
	 *
	 * @param EditObject		The edited object on which to get the connected pin names
	 * @param RawData			The raw data on which to get the connected pin names
	 * @param OutParameters		The list of pin names connected to "well-known parameters"
	 */
	virtual void GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutParameters) const {}

	/**
	 * Get the pins that are candidates for any "well-known parameter".
	 *
	 * The output array of pin candidates should be of the same size as GetWellKnownParameterPinNames, with an empty
	 * array of candidates for those parameters that currently have no compatible pins.
	 *
	 * @param EditObject		The edited object on which to set the connected pin name
	 * @param RawData			The raw data on which to set the connected pin name
	 */
	virtual void GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const {}

	/**
	 * Set the pin name for the given "well-known parameter".
	 *
	 * @param EditObject		The edited object on which to set the connected pin name
	 * @param RawData			The raw data on which to set the connected pin name
	 * @param ParameterIndex	The index of the "well-known parameter", as reported by GetWellKnownParameterCandidates
	 * @return					Whether the pin name was successfully set
	 */
	virtual bool SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName) { return false; }

	/**
	 * Generate endpoint definition for the sequence.
	 */
	virtual FMovieSceneDirectorBlueprintEndpointDefinition GenerateEndpointDefinition(UMovieSceneSequence* Sequence) = 0;

	/**
	 * Called to find the endpoint node in the director blueprint.
	 */
	virtual UK2Node* FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const = 0;

	/**
	 * Called when an endpoint has been created.
	 */
	virtual void OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) = 0;

	/**
	 * Called when an endpoint has been set.
	 */
	virtual void OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) = 0;

	/**
	 * Called when an endpoint was set and has been changed to another endpoint.
	 */
	virtual bool OnRebindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, TSharedPtr<FEdGraphSchemaAction> Action) { return false; }

	/**
	 * Get all the objects that the endpoints reside within.
	 * By default, this gets the outer objects of the property handle.
     */
	UE_API virtual void GetEditObjects(TArray<UObject*>& OutObjects) const;

	/**
	 * Collect blueprint actions for quick-binding a non-connected endpoint.
	 */
	virtual void OnCollectQuickBindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder) {}

	/**
	 * Collect blueprint actions for rebinding an already connected endpoint.
	 */
	virtual void OnCollectAllRebindActions(UBlueprint* Blueprint, FBlueprintActionMenuBuilder& MenuBuilder) {}

	virtual bool CreateNewCategoryForPayloadVariables() const { return true; }

protected:

	/**
	 * Clear the endpoint represented by this property handle.
	 * @note: Does not delete the endpoint in the blueprint itself.
	 */
	UE_API void ClearEndpoint();


	/**
	 * Find the endpoint node in the director blueprint.
	 */
	UE_API UK2Node* FindEndpoint(UObject* EditObject, void* RawData) const;

	/**
	 * Assigns the specified function entry to this property handle.
	 */
	UE_API void SetEndpoint(const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint, UK2Node* PayloadTemplate, EAutoCreatePayload AutoCreatePayload);

	/**
	 * Navigate to the definition of the endpoint represented by this property handle.
	 */
	UE_API void NavigateToDefinition();

	/**
	 * Generate the content of the main combo button menu dropdown
	 */
	UE_API TSharedRef<SWidget> GetMenuContent();

	/**
	 * Generate the content of the rebind sub-menu dropdown (shown if the endpoint is already bound)
	 */
	UE_API void PopulateRebindSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneSequence* Sequence);

	/**
	 * Called when a payload variable property value is changed on the details panel
	 */
	UE_API void OnPayloadVariableChanged(TSharedRef<FStructOnScope> InStructData, TSharedPtr<IPropertyHandle> LocalVariableProperty);

protected:

	struct FSequenceData
	{
		UBlueprint* Blueprint;
		TArray<UObject*> EditObjects;
		TArray<void*> RawData;
	};
	using FSequenceDataMap = TMap<UMovieSceneSequence*, FSequenceData>;

	/**
	 * Gather data for the sequences being edited.
	 */
	UE_API void GatherSequenceData(FSequenceDataMap& AllSequenceData);

	/**
	 * Get the property handle for this customization.
	 */
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const { return PropertyHandle; }

	/**
	 * Get the property utilities for this customization.
	 */
	TSharedPtr<IPropertyUtilities> GetPropertyUtilities() const { return PropertyUtilities; }

	/**
	 * Get the sequence that is common to all the values in our property handle, or nullptr if they are not all the same.
	 */
	UE_API UMovieSceneSequence* GetCommonSequence() const;

	/**
	 * Get the endpoint that is common to all the values in our property handle, or nullptr if they are not all the same.
	 */
	UE_API UK2Node* GetCommonEndpoint() const;

	/**
	 * Get all the endpoints of all the values in our property handle.
	 */
	UE_API TArray<UK2Node*> GetAllValidEndpoints() const;

	/**
	 * Invoke a callback on all the endpoints.
	 */
	UE_API void IterateEndpoints(TFunctionRef<bool(UK2Node*)> Callback) const;


	/** The property handle we're reflecting */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Property utilities for the property we're editing */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	// RawData taken either from the PropertyHandle, or manually passed into the customization for cases this is used as a helper
	TArray<void*> PropertyRawData;

private:

	UE_API void CollectQuickBindActions(FGraphActionListBuilderBase& OutAllActions, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition);
	UE_API void CollectAllRebindActions(FGraphActionListBuilderBase& OutAllActions, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition);

private:

	/** Get the name of the endpoint to display on the main combo button */
	UE_API FText GetEndpointName() const;
	/** Get the icon of the endpoint to display on the main combo button */
	UE_API const FSlateBrush* GetEndpointIcon() const;

	UE_API void HandleRebindActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType, UBlueprint* Blueprint, FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition);
	
	UE_API void OnBlueprintCompiled(UBlueprint*);

	UE_API ECheckBoxState GetCallInEditorCheckState() const;
	UE_API void OnSetCallInEditorCheckState(ECheckBoxState NewCheckBoxState);

	UE_API TSharedRef<SWidget> GetWellKnownParameterPinMenuContent(int32 ParameterIndex);
	UE_API const FSlateBrush* GetWellKnownParameterPinIcon(int32 ParameterIndex) const;
	UE_API FText GetWellKnownParameterPinText(int32 ParameterIndex) const;
	UE_API bool CompareWellKnownParameterPinName(int32 ParameterIndex, FName InPinName) const;
	UE_API void SetWellKnownParameterPinName(int32 ParameterIndex, FName InNewWellKnownParameterPinName);

private:

	/** A cache of the common endpoint that is only used when the menu is open to avoid re-computing it every frame. */
	TWeakObjectPtr<UK2Node_FunctionEntry> CachedCommonEndpoint;
};

#undef UE_API
