// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "EdGraph/EdGraphNode.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MOVIESCENETOOLS_API

class FKismetCompilerContext;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UK2Node;
class UK2Node_CallFunction;
class UK2Node_CustomEvent;
class UK2Node_FunctionEntry;
class UK2Node_FunctionResult;
class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneTrack;
struct FGuid;

/**
 * Sequence Director Blueprint endpoint type
 */
enum class EMovieSceneDirectorBlueprintEndpointType
{
	None = 0,
	Event,
	Function
};

/**
 * Structure for creating extra parameter pins on an enpdoint
 */
struct FMovieSceneDirectorBlueprintEndpointParameter
{
	FString PinName;
	EEdGraphPinDirection PinDirection;
	FName PinTypeCategory;
	TWeakObjectPtr<UObject> PinTypeClass;

	static UE_API FString SanitizePinName(const FString& InPinName);
};

/*
* Structure for holding value of a blueprint variable 
*/
struct FMovieSceneDirectorBlueprintVariableValue
{
	/* In the case the variable is a UObject, reference to the UObject */
	FSoftObjectPath ObjectValue;

	/** String value of the variable */
	FString Value;
};

/**
 * Parameters for creating a blueprint endpoint
 */
struct FMovieSceneDirectorBlueprintEndpointDefinition
{
	/** Type of endpoint */
	EMovieSceneDirectorBlueprintEndpointType EndpointType;

	/** Name of the event or function graph */
	FString GraphName;

	/** Name of the event or function */
	FString EndpointName;

	/** Function signature, for function endpoint types */
	UFunction* EndpointSignature = nullptr;

	/** Optional class on which the endpoint can call functions */
	UClass* PossibleCallTargetClass = nullptr;

	/** Extra pins to create on the endpoint */
	TArray<FMovieSceneDirectorBlueprintEndpointParameter> ExtraPins;

	FMovieSceneDirectorBlueprintEndpointDefinition()
		: EndpointType(EMovieSceneDirectorBlueprintEndpointType::None)
	{}

	void AddExtraInputPin(const FString& InName, FName InPinCategory, TWeakObjectPtr<UObject> InPinTypeClass = nullptr)
	{
		FMovieSceneDirectorBlueprintEndpointParameter Parameter;
		Parameter.PinName = InName;
		Parameter.PinDirection = EGPD_Input;
		Parameter.PinTypeCategory = InPinCategory;
		Parameter.PinTypeClass = InPinTypeClass;
		ExtraPins.Add(Parameter);
	}

	void AddExtraOutputPin(const FString& InName, FName InPinCategory, TWeakObjectPtr<UObject> InPinTypeClass = nullptr)
	{
		FMovieSceneDirectorBlueprintEndpointParameter Parameter;
		Parameter.PinName = InName;
		Parameter.PinDirection = EGPD_Output;
		Parameter.PinTypeCategory = InPinCategory;
		Parameter.PinTypeClass = InPinTypeClass;
		ExtraPins.Add(Parameter);
	}
};

/** Parameters for endpoint call customization callback */
struct FMovieSceneCustomizeDirectorBlueprintEndpointCallParams
{
	UEdGraph* FunctionGraph = nullptr;
	UK2Node_FunctionEntry* FunctionEntryNode = nullptr;
	UK2Node_CallFunction* CallFunctionNode = nullptr;
	UK2Node_FunctionResult* FunctionResultNode = nullptr;
};

DECLARE_DELEGATE_OneParam(FMovieSceneCustomizeDirectorBlueprintEndpointCallDelegate, const FMovieSceneCustomizeDirectorBlueprintEndpointCallParams&);

/**
 * Parameters for creating an entry point, i.e. a call to an endpoint
 */
struct FMovieSceneDirectorBlueprintEndpointCall
{
	/** The endpoint node to call */
	UEdGraphNode* Endpoint = nullptr;

	/** List of endpoint node pins that should be exposed to the call, for receiving special values at runtime */
	TArray<FName> ExposedPinNames;

	/** The values to pass to the endpoint node's parameters */
	TMap<FName, FMovieSceneDirectorBlueprintVariableValue> PayloadVariables;

	/** Callback for customizing the endpoint call */
	FMovieSceneCustomizeDirectorBlueprintEndpointCallDelegate CustomizeEndpointCall;
};

/**
 * Result structure for a created blueprint entry point, i.e. a call to an endpoint
 */
struct FMovieSceneDirectorBlueprintEntrypointResult
{
	/** The name of the function that calls the endpoint */
	FName CompiledFunctionName;
	/** The entry point node */
	UK2Node_FunctionEntry* Entrypoint = nullptr;
	/** A list of payload variables that aren't valid anymore */
	TArray<FName> StalePayloadVariables;

	/** A utility method for cleaning up stale payload variables */
	template<typename Container>
	void CleanUpStalePayloadVariables(Container& InPayloadVariables) const
	{
		for (FName StalePayloadVariable : StalePayloadVariables)
		{
			InPayloadVariables.Remove(StalePayloadVariable);
		}
	}
};

/**
 * Static utility library for dealing with movie-scene blueprint endpoints at edit/cook-time
 */
struct FMovieSceneDirectorBlueprintUtils
{
	/**
	 * Create a new endpoint for the specified sequence.
	 *
	 * @param Blueprint       (Required, non-null) The blueprint within which to create the new custom event
	 * @param EndpointDefinition  The end point definition
	 * @return A valid pointer to the blueprint node for the endpoint
	 */
	static UE_API UK2Node* CreateEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition);

	/**
	 * Create a new event endpoint for the specified sequence.
	 *
	 * By default, event endpoints are presented as a UK2Node_CustomEvent in an event graph called "Sequencer Events" with a pin for the bound object.
	 *
	 * @param Blueprint       (Required, non-null) The blueprint within which to create the new custom event
	 * @param EndpointDefinition  The end point definition
	 * @return A valid pointer to the blueprint node for the endpoint
	 */
	static UE_API UK2Node_CustomEvent* CreateEventEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition);
	
	/**
	 * Create a new function endpoint for the specified sequence.
	 *
	 * @param Blueprint       (Required, non-null) The blueprint within which to create the new custom event
	 * @param EndpointDefinition  The end point definition
	 * @return A valid pointer to the blueprint node for the endpoint
	 */
	static UE_API UK2Node_FunctionEntry* CreateFunctionEndpoint(UBlueprint* Blueprint, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition);

	/**
	 * Main generation function for creating the necessary function graph for calling an event endpoint with 
	 * the parameters specified in the entry point payload
	 *
	 * @param EntryPoint             (Required, non-null) The event entry point definition to generate a function graph for
	 * @param Compiler               (Required, non-null) The kismet compiler context for generation
	 * @param Endpoint               (Required, non-null) The endpoint node that the event is bound to
	 */
	static UE_API FMovieSceneDirectorBlueprintEntrypointResult GenerateEntryPoint(const FMovieSceneDirectorBlueprintEndpointCall& EndpointCall, FKismetCompilerContext* Compiler);

	/**
	 * Attempt to locate an output pin on the specified endpoint node that matches the specified pin class
	 * 
	 * @param InEndpoint             (Required, non-null) The endpoint node to search on
	 * @param BoundObjectPinClass    (Required, non-null) The class of the object pin to locate
	 * @return An output pin that is of type PC_Object with the specified class as its PinSubCategoryObject, or nullptr if one was not found
	 */
	static UE_API UEdGraphPin* FindCallTargetPin(UK2Node* InEndpoint, UClass* CallTargetClass);

private:

	static UE_API UEdGraph* FindOrCreateEventGraph(UBlueprint* Blueprint, const FString& GraphName);

	static UE_API bool GenerateEntryPointRawActorParameter(
			FKismetCompilerContext* Compiler, UEdGraph* Graph, UEdGraphNode* Endpoint, 
			UK2Node* OriginNode, UEdGraphPin* DestPin, const FMovieSceneDirectorBlueprintVariableValue& PayloadValue);
};

#undef UE_API
