// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieScene.h"
#include "MovieSceneDynamicBinding.h"
#include "Bindings/MovieSceneReplaceableDirectorBlueprintBinding.h"
#include "Bindings/MovieSceneSpawnableDirectorBlueprintBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "Editor.h"
#include "MovieSceneBindingReferences.h"
#include "MovieSceneCommonHelpers.h"

#include "MovieSceneDynamicBindingUtils.generated.h"

#define UE_API MOVIESCENETOOLS_API

/**
 * A utility class for managing dynamic binding endpoints.
 */
struct FMovieSceneDynamicBindingUtils
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	/**
	 * Set an endpoint on the given dynamic binding.
	 */
	static UE_API void SetEndpoint(UMovieScene* MovieScene, FMovieSceneDynamicBinding* DynamicBinding, UK2Node* NewEndpoint);

	/**
	 * Ensures that the dynamic binding blueprint extension has been added to the given sequence's director blueprint.
	 */
	static UE_API void EnsureBlueprintExtensionCreated(UMovieSceneSequence* MovieSceneSequence, UBlueprint* Blueprint);

	/**
	 * Utility function for iterating all dynamic bindings in a sequence.
	 */
	template<typename Callback>
	static void IterateDynamicBindings(UMovieScene* InMovieScene, Callback&& InCallback)
	{
		UMovieSceneSequence* ThisSequence = InMovieScene->GetTypedOuter<UMovieSceneSequence>();

		ThisSequence->IterateDynamicBindings(InCallback);
	}
	
	/**
	 * Utility function for gathering all dynamic bindings in a sequence into a container.
	 */
	static void GatherDynamicBindings(UMovieScene* InMovieScene, TArray<FMovieSceneDynamicBinding*>& OutDynamicBindings)
	{
		IterateDynamicBindings(InMovieScene, [&](const FGuid&, FMovieSceneDynamicBinding& Item)
			{
				OutDynamicBindings.Add(&Item);
			});
	}
};

/**
 * Dummy class, used for easily getting a valid UFunction that helps prepare blueprint function graphs.
 */
UCLASS(MinimalAPI)
class UMovieSceneDynamicBindingEndpointUtil : public UObject
{
	GENERATED_BODY()

	UFUNCTION()
	FMovieSceneDynamicBindingResolveResult SampleResolveBinding() { return FMovieSceneDynamicBindingResolveResult(); }
};

#undef UE_API
