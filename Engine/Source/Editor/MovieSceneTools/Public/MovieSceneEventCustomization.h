// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "MovieSceneDirectorBlueprintEndpointCustomization.h"
#include "Types/SlateEnums.h"

class UMovieSceneEventTrack;

/**
 * Customization for FMovieSceneEvent structs.
 * Will deduce the event's section either from the outer objects on the details customization, or use the one provided on construction (for instanced property type customizations)
 */
class FMovieSceneEventCustomization : public FMovieSceneDirectorBlueprintEndpointCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(UMovieSceneSection* InSection);

protected:

	virtual void GetPayloadVariables(UObject* EditObject, void* RawData, FPayloadVariableMap& OutPayloadVariables) const override;
	virtual bool SetPayloadVariable(UObject* EditObject, void* RawData, FName FieldName, const FMovieSceneDirectorBlueprintVariableValue& NewValue) override;
	virtual UK2Node* FindEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, UObject* EditObject, void* RawData) const override;
	virtual void GetWellKnownParameterPinNames(UObject* EditObject, void* RawData, TArray<FName>& OutWellKnownParameters) const override;
	virtual void GetWellKnownParameterCandidates(UK2Node* Endpoint, TArray<FWellKnownParameterCandidates>& OutCandidates) const override;
	virtual bool SetWellKnownParameterPinName(UObject* EditObject, void* RawData, int32 ParameterIndex, FName BoundPinName) override;
	virtual FMovieSceneDirectorBlueprintEndpointDefinition GenerateEndpointDefinition(UMovieSceneSequence* Sequence) override;
	virtual void OnCreateEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) override;
	virtual void OnSetEndpoint(UMovieSceneSequence* Sequence, UBlueprint* Blueprint, const TArray<UObject*> EditObjects, const TArray<void*> RawData, const FMovieSceneDirectorBlueprintEndpointDefinition& EndpointDefinition, UK2Node* NewEndpoint) override;
	virtual void GetEditObjects(TArray<UObject*>& OutObjects) const override;

private:

	/** Externally supplied section that the event(s) we're reflecting reside within */
	TWeakObjectPtr<UMovieSceneSection> WeakExternalSection;
};
