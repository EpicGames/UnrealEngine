// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MOVIESCENETOOLS_API

class FKismetCompilerContext;
class UBlueprint;
class UClass;
class UEdGraphNode;
class UEdGraphPin;
class UK2Node;
class UK2Node_CallFunction;
class UK2Node_CustomEvent;
class UK2Node_FunctionEntry;
class UMovieScene;
class UMovieSceneEventSectionBase;
class UMovieSceneEventTrack;
class UMovieSceneSequence;
class UMovieSceneTrack;
struct FGuid;
struct FMovieSceneEvent;

/**
 * Static utility library for dealing with movie-scene events at edit/cook-time
 */
struct FMovieSceneEventUtils
{
	/**
	 * Generate event endpoint definition for the given track
	 */
	static UE_API FMovieSceneDirectorBlueprintEndpointDefinition GenerateEventDefinition(UMovieSceneTrack* Track);

	/**
	 * Generate event endpoint definition for the given object binding
	 */
	static UE_API FMovieSceneDirectorBlueprintEndpointDefinition GenerateEventDefinition(UMovieScene* MovieScene, const FGuid& ObjectBindingID);


	/**
	 * As with CreateUserFacingEvent, but also binds the section to a UBlueprintExtension which ensures that it will hook into the compilation process to generate its entry point function graph
	 *
	 * @param EntryPoint      (Required, non-null) The event entry point definition
	 * @param EventSection    (Required, non-null) The event section that owns the specified entry point
	 * @param Blueprint       (Required, non-null) The blueprint within which to create the new custom event
	 * @return A valid pointer to the blueprint node for the event endpoint
	 */
	static UE_API UK2Node_CustomEvent* BindNewUserFacingEvent(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UBlueprint* Blueprint);

	/**
	 * Main generation function for creating the necessary function graph for calling an event endpoint with the parameters specified in the entry point payload
	 *
	 * @param EventSection           (Required, non-null) The event section that owns the event
	 * @param EntryPoint             (Required, non-null) The event entry point definition to generate a function graph for
	 * @param Compiler               (Required, non-null) The kismet compiler context for generation
	 * @param Endpoint               (Required, non-null) The endpoint node that the event is bound to
	 */
	static UE_API UK2Node_FunctionEntry* GenerateEntryPoint(FMovieSceneEvent* EntryPoint, FKismetCompilerContext* Compiler, UEdGraphNode* Endpoint);

	/**
	 * Create the necessary mapping between a director Blueprint and and event section in order for the event section to be compiled correctly.
	 *
	 * @param EventSection           (Required, non-null) The event section to bind to the director BP
	 * @param DirectorBP             (Required, non-null) The owning blueprint that will compile the event section's entry points
	 */
	static UE_API void BindEventSectionToBlueprint(UMovieSceneEventSectionBase* EventSection, UBlueprint* DirectorBP);

	/**
	 * Remove all event end points for the given event section
	 *
	 * @param EventSection           (Required, non-null) The event section to remove event endpoints from
	 * @param DirectorBP             (Required, non-null) The owning blueprint from which to remove the event section's entry points
	 */
	static UE_API void RemoveEndpointsForEventSection(UMovieSceneEventSectionBase* EventSection, UBlueprint* DirectorBP);

	/**
	 * Clear invalid endpoints (end points to that point to a blueprint in a different movie scene, possibly from copy/paste)
	 *
	 * @param EventSection           (Required, non-null) The event section to search for invalid event endpoints from
	 */
	static UE_API void ClearInvalidEndpointsForEventSection(UMovieSceneEventSectionBase* EventSection);

	/**
	 * Remove unused custom events (that don't have corresponding event end points in the given event sections)
	 *
	 * @param EventSections          (Required, non-null) The event sections to search for event endpoints from
	 * @param DirectorBP             (Required, non-null) The owning blueprint from which to remove the event section's entry points
	 */
	static UE_API void RemoveUnusedCustomEvents(const TArray<TWeakObjectPtr<UMovieSceneEventSectionBase>>& EventSections, UBlueprint* DirectorBP);

	/**
	 * Attempt to locate the blueprint node that relates to an event's end-point.
	 * 
	 * @param EntryPoint             (Required, non-null) The event entry point definition to look up
	 * @param EventSection           (Required, non-null) The event section that owns the event
	 * @param OwnerBlueprint         (Required, non-null) The blueprint to search within
	 * @return The K2Node that relates to the specified event's end point, or nullptr if one was not found
	 */
	static UE_API UK2Node* FindEndpoint(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UBlueprint* OwnerBlueprint);

	/**
	 * Set the specified EntryPoint to be bound to a new K2 node
	 * 
	 * @param EntryPoint             (Required, non-null) The event entry point definition to assign to
	 * @param EventSection           (Required, non-null) The event section that owns the event entry point definition
	 * @param InNewEndpoint          (Optional) The new endpoint node to call from the event
	 * @param BoundObjectPin         (Optional) The pin that should receive the bound object when invoking the event
	 */
	static UE_API void SetEndpoint(FMovieSceneEvent* EntryPoint, UMovieSceneEventSectionBase* EventSection, UK2Node* InNewEndpoint, UEdGraphPin* BoundObjectPin);
};

#undef UE_API
