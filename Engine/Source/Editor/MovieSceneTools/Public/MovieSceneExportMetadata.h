// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Sections/MovieSceneCinematicShotSection.h"

struct FMovieSceneExportMetadataClip
{
	FMovieSceneExportMetadataClip()
		: StartFrame(INT32_MAX)
		, EndFrame(INT32_MIN)
		, bHasAlpha(false)
	{}

	bool IsValid() const { return EndFrame >= StartFrame; }
	int32 GetDuration() const { return IsValid() ? EndFrame - StartFrame + 1 : 0; }

	int32 StartFrame;
	int32 EndFrame;
	bool bHasAlpha;

	FString FileName;
};

struct FMovieSceneExportMetadataShot
{
	TWeakObjectPtr<UMovieSceneCinematicShotSection> MovieSceneShotSection;
	int32 HandleFrames;

	// All of the clips for this shot, stored by ClipName
	// Multiple formats may be exported, so each ClipName has a list metadata stored by extension
	TMap< FString, TMap<FString, FMovieSceneExportMetadataClip> > Clips;
};

struct FMovieSceneExportMetadata
{
	TArray<FMovieSceneExportMetadataShot> Shots;
};