// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"

#define UE_API MOVIESCENETOOLS_API

class FDetailWidgetRow;
class FDragDropEvent;
class FDragDropOperation;
class FReply;
class IPropertyHandle;
class ISequencer;
class UMovieSceneSequence;
struct FGeometry;

namespace UE
{
namespace MovieScene
{
	struct FFixedObjectBindingID;
}
}



class FMovieSceneObjectBindingIDCustomization
	: public IPropertyTypeCustomization
	, FMovieSceneObjectBindingIDPicker
{
public:

	FMovieSceneObjectBindingIDCustomization()
	{}

	FMovieSceneObjectBindingIDCustomization(FMovieSceneSequenceID InLocalSequenceID, TWeakPtr<ISequencer> InSequencer)
		: FMovieSceneObjectBindingIDPicker(InLocalSequenceID, InSequencer)
	{}

	static UE_API void BindTo(TSharedRef<ISequencer> InSequencer);

	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	UE_API virtual UMovieSceneSequence* GetSequence() const override;

	UE_API virtual void SetCurrentValue(const FMovieSceneObjectBindingID& InBindingId) override;

	UE_API virtual FMovieSceneObjectBindingID GetCurrentValue() const override;

	UE_API virtual bool HasMultipleValues() const override;

	UE_API FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

	UE_API void OnResetToDefault();

	TSharedPtr<IPropertyHandle> StructProperty;
};

#undef UE_API
