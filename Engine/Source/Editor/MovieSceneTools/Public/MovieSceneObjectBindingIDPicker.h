// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

#define UE_API MOVIESCENETOOLS_API

class FMenuBuilder;
class FReply;
class ISequencer;
class STextBlock;
class SWidget;
class UMovieSceneSequence;
struct EVisibility;
struct FMovieSceneObjectBindingID;
struct FSequenceBindingNode;
struct FSequenceBindingTree;
struct FSlateBrush;

namespace UE
{
namespace MovieScene
{

struct FFixedObjectBindingID;
struct FRelativeObjectBindingID;

} // namespace MovieScene
} // namespace UE

/**
 * Helper class that is used to pick object bindings for movie scene data
 */
class FMovieSceneObjectBindingIDPicker
{
public:

	/** Default constructor used in contexts external to the sequencer interface. Always generates FMovieSceneObjectBindingIDs from the root of the sequence */
	FMovieSceneObjectBindingIDPicker()
		: bIsCurrentItemSpawnable(false)
	{}

	/**
	 * Constructor used from within the sequencer interface to generate IDs from the currently focused sequence if possible (else from the root sequence).
	 * This ensures that the bindings will resolve correctly in isolation only the the focused sequence is being used, or from the root sequence.
	 */
	FMovieSceneObjectBindingIDPicker(FMovieSceneSequenceID InLocalSequenceID, TWeakPtr<ISequencer> InSequencer)
		: WeakSequencer(InSequencer)
		, LocalSequenceID(InLocalSequenceID)
		, bIsCurrentItemSpawnable(false)
	{}


	/**
	 * Check whether this picker actually has anything to pick
	 */
	UE_API bool IsEmpty() const;

protected:
	virtual ~FMovieSceneObjectBindingIDPicker() = default;

	/** Get the sequence to look up object bindings within. Only used when no sequencer is available. */
	virtual UMovieSceneSequence* GetSequence() const = 0;

	/** Set the current binding ID */
	virtual void SetCurrentValue(const FMovieSceneObjectBindingID& InBindingId) = 0;

	/** Get the current binding ID */
	virtual FMovieSceneObjectBindingID GetCurrentValue() const = 0;

	/** Whether there are multiple values */
	virtual bool HasMultipleValues() const { return false; }

	/** Checks if node is allowed by this binding */
	UE_API virtual bool IsNodeAllowed(const TSharedPtr<FSequenceBindingNode>& Node) const;

protected:

	/** Initialize this class - rebuilds sequence hierarchy data and available IDs from the source sequence */
	UE_API void Initialize();

	/** Access the text that relates to the currently selected binding ID */
	UE_API FText GetCurrentText() const;

	/** Access the tooltip text that relates to the currently selected binding ID */
	UE_API FText GetToolTipText() const;

	/** Get the icon that represents the currently assigned binding */
	UE_API FSlateIcon GetCurrentIcon() const;
	UE_API const FSlateBrush* GetCurrentIconBrush() const;

	/** Get the visibility for the spawnable icon overlap */ 
	UE_API EVisibility GetSpawnableIconOverlayVisibility() const;

	/** Assign a new binding ID in response to user-input */
	UE_API void SetBindingId(UE::MovieScene::FFixedObjectBindingID InBindingId);

	/** Build menu content that allows the user to choose a binding from inside the source sequence */
	UE_API TSharedRef<SWidget> GetPickerMenu();

	/** Build menu content that allows the user to choose a binding from inside the source sequence */
	UE_API void GetPickerMenu(FMenuBuilder& MenuBuilder);

	/** Get a widget that represents the currently chosen item */
	UE_API TSharedRef<SWidget> GetCurrentItemWidget(TSharedRef<STextBlock> TextContent);

	/** Get a widget that represents a warning/fixup button for this binding */
	UE_API TSharedRef<SWidget> GetWarningWidget();

	/** Optional sequencer ptr */
	TWeakPtr<ISequencer> WeakSequencer;

	/** The ID of the sequence to generate IDs relative to */
	FMovieSceneSequenceID LocalSequenceID;

	/** Update the cached text, tooltip and icon */
	UE_API void UpdateCachedData();

private:

	/** Get the currently set binding ID as a fixed binding ID from the root sequence */
	UE_API UE::MovieScene::FFixedObjectBindingID GetCurrentValueAsFixed() const;

	/** Set the binding ID */
	UE_API void SetCurrentValueFromFixed(UE::MovieScene::FFixedObjectBindingID InValue);

	/** Called when the combo box has been clicked to populate its menu content */
	UE_API void OnGetMenuContent(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceBindingNode> Node);

	/** Get the visibility for the warning/fixup button */
	UE_API EVisibility GetFixedWarningVisibility() const;

	/** Get the visibility for the warning/fixup button */
	UE_API FReply AttemptBindingFixup();

	/** Checks if a class is allowed by this binding */
	UE_API bool IsClassAllowed(const TSharedPtr<FSequenceBindingNode>& Node) const;

	/** Cached current text and tooltips */
	FText CurrentText, ToolTipText;

	/** Cached current icon */
	FSlateIcon CurrentIcon;

	/** Cached value indicating whether the current item is a spawnable */
	bool bIsCurrentItemSpawnable;

	/** Data tree that stores all the available bindings for the current sequence, and their identifiers */
	TSharedPtr<FSequenceBindingTree> DataTree;

	/** Weak ptr to a widget used to dismiss menus to */
	TWeakPtr<SWidget> DismissWidget;

protected:
	/** The meta class for the property that classes must be a child-of. */
	const UClass* ClassPropertyMetaClass = UObject::StaticClass();

	/** The interface that must be implemented. */
	const UClass* InterfaceThatMustBeImplemented = nullptr;

	/** Classes that can be picked */
	TArray<const UClass*> AllowedClassFilters;

	/** Classes that can't be picked */
	TArray<const UClass*> DisallowedClassFilters;
};

#undef UE_API
