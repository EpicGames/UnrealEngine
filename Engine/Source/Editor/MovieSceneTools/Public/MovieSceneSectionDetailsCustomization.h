// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SCheckBox.h"
#include "IDetailCustomization.h"

#define UE_API MOVIESCENETOOLS_API

class IDetailLayoutBuilder;
class UMovieScene;

/**
 *  Customizes FMovieSceneSection to expose the section bounds to the UI and allow changing their bounded states.
 */
class FMovieSceneSectionDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface, TWeakObjectPtr<UMovieScene> InParentMovieScene)
	{
		return MakeShared<FMovieSceneSectionDetailsCustomization>(InNumericTypeInterface, InParentMovieScene);
	}

	FMovieSceneSectionDetailsCustomization(TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface, TWeakObjectPtr<UMovieScene> InParentMovieScene)
	{
		NumericTypeInterface = InNumericTypeInterface;
		ParentMovieScene = InParentMovieScene;
	}

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder);

private:
	enum class ERangeBoundValueType
	{
		Finite,
		Infinite,
		MultipleValues
	};

private:
	/** Get the range start value */
	UE_API ERangeBoundValueType GetRangeStartValue(FFrameNumber& OutValue) const;

	/** Convert the range start into an FText for display */
	UE_API FText OnGetRangeStartText() const;
	/** Convert the range start into an FText for tooltip display */
	UE_API FText OnGetRangeStartToolTipText() const;
	/** Convert the text into a new range start */
	UE_API void OnRangeStartTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	/** Should the textbox be editable? False if we have an infinite range.  */
	UE_API bool IsRangeStartTextboxEnabled() const;

	/** Determines if the range is Open, Closed, or Undetermined which can happen in the case of multi-select.  */
	UE_API ECheckBoxState GetRangeStartBoundedState() const;
	/** Sets the range to have a fixed bound or convert to an open bound. */
	UE_API void SetRangeStartBounded(bool InbIsBounded);


	/** Get the FText representing the appropriate Unicode icon for the toggle button. */
	UE_API FText GetRangeStartButtonIcon() const;
	/** Called by the UI when the button is pressed to toggle the current state. */
	UE_API FReply ToggleRangeStartBounded();

private:
	/** Get the range end value */
	UE_API ERangeBoundValueType GetRangeEndValue(FFrameNumber& OutValue) const;

	/** Convert the range end into an FText for display */
	UE_API FText OnGetRangeEndText() const;
	/** Convert the range end into an FText for tooltip display */
	UE_API FText OnGetRangeEndToolTipText() const;
	/** Convert the text into a new range start */
	UE_API void OnRangeEndTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	/** Should the textbox be editable? False if we have an infinite range.  */
	UE_API bool IsRangeEndTextboxEnabled() const;

	/** Determines if the range is Open, Closed, or Undetermined which can happen in the case of multi-select.  */
	UE_API ECheckBoxState GetRangeEndBoundedState() const;
	/** Sets the range to have a fixed bound or convert to an open bound. */
	UE_API void SetRangeEndBounded(bool InbIsBounded);


	/** Get the FText representing the appropriate Unicode icon for the toggle button. */
	UE_API FText GetRangeEndButtonIcon() const;
	/** Called by the UI when the button is pressed to toggle the current state. */
	UE_API FReply ToggleRangeEndBounded();
private:

	/** The Numeric Type interface used to convert between display formats and internal tick resolution. */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	/** Store the property handle to the FrameNumber field so we can get/set the value on the object via text box callbacks. */
	TSharedPtr<IPropertyHandle> MovieSceneSectionPropertyHandle;

	/** The movie scene that owns the section we're customizing. Used to find out the overall bounds for changing a section bounds from infinite -> closed. */
	TWeakObjectPtr<UMovieScene> ParentMovieScene;
};

#undef UE_API
