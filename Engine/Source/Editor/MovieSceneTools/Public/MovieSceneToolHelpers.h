// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Curves/RichCurve.h"
#include "Math/InterpCurvePoint.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "ISequencer.h"
#include "Logging/TokenizedMessage.h"
#include "MovieSceneTranslator.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneCaptureSettings.h"
#include "KeyParams.h"
#include "SEnumCombo.h"
#include "Animation/AnimSequence.h"
#include "INodeAndChannelMappings.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSection.h"

#define UE_API MOVIESCENETOOLS_API

class UMovieScene;
class UMovieSceneSequence;
struct FMovieSceneObjectBindingID;
class UMovieSceneTrack;
struct FMovieSceneEvaluationTrack;
class UMovieSceneUserImportFBXSettings;
class UMovieSceneUserImportFBXControlRigSettings;
class UMovieSceneUserExportFBXControlRigSettings;
struct FMovieSceneDoubleValue;
struct FMovieSceneFloatValue;
class INodeNameAdapter;
struct FMovieSceneSequenceTransform;
class UAnimSeqExportOption;
template<typename ChannelType> struct TMovieSceneChannelData;
enum class EVisibilityBasedAnimTickOption : uint8;
class ACameraActor;
struct FActorForWorldTransforms;
class UMovieScene3DTransformSection;
class UMovieSceneSubTrack;
struct FBakingAnimationKeySettings;
struct FKeyDataOptimizationParams;
class UMovieSceneSubSection;
enum class EMovieSceneTransformChannel : uint32;
class UMovieScene3DTransformTrack;

namespace fbxsdk
{
	class FbxCamera;
	class FbxNode;
}
namespace UnFbx
{
	class FFbxImporter;
	class FFbxCurvesAPI;
};

struct FFBXInOutParameters
{
	bool bConvertSceneBackup;
	bool bConvertSceneUnitBackup;
	bool bForceFrontXAxisBackup;
	float ImportUniformScaleBackup;
};

struct FAnimExportSequenceParameters
{
	FAnimExportSequenceParameters() = default;
	UMovieSceneSequence* MovieSceneSequence;
	UMovieSceneSequence* RootMovieSceneSequence;
	IMovieScenePlayer* Player;
	FMovieSceneSequenceTransform RootToLocalTransform;
	bool bForceUseOfMovieScenePlaybackRange = false;
};

//callback's used by skel mesh recorders

DECLARE_DELEGATE(FInitAnimationCB);
DECLARE_DELEGATE_OneParam(FStartAnimationCB, FFrameNumber);
DECLARE_DELEGATE_TwoParams(FTickAnimationCB, float, FFrameNumber);
DECLARE_DELEGATE(FEndAnimationCB);

//Skel Mesh Recorder to set up and restore various parameters on the skelmesh
struct FSkelMeshRecorderState
{
public:
	FSkelMeshRecorderState() {}
	~FSkelMeshRecorderState() = default;

	UE_API void Init(USkeletalMeshComponent* InComponent);
	UE_API void FinishRecording();


public:
	TWeakObjectPtr<USkeletalMeshComponent> SkelComp;;

	/** Original ForcedLodModel setting on the SkelComp, so we can modify it and restore it when we are done. */
	int CachedSkelCompForcedLodModel;

	/** Used to store/restore update flag when recording */
	EVisibilityBasedAnimTickOption CachedVisibilityBasedAnimTickOption;

	/** Used to store/restore URO when recording */
	bool bCachedEnableUpdateRateOptimizations;
};

enum class FChannelMergeAlgorithm : uint8
{
	/**Average values together*/
	Average,
	/**Add values together*/
	Add,
	/** Override values together*/
	Override,
};

class MovieSceneToolHelpers
{
public:
	/**
	 * Trim section at the given time
	 *
	 * @param Sections The sections to trim
	 * @param Time	The time at which to trim
	 * @param bTrimLeft Trim left or trim right
	 * @param bDeleteKeys Delete keys outside the split ranges
	 */
	static UE_API void TrimSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time, bool bTrimLeft, bool bDeleteKeys);
	static UE_API bool CanTrimSectionLeft(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time);
	static UE_API bool CanTrimSectionRight(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time);

	/**
	 * Trim or extend section at the given time
	 *
	 * @param Track The track that contains the sections to trim
	 * @param RowIndex Optional row index to trim, otherwise trims sections with all row indices
	 * @param Time	The time at which to trim
	 * @param bTrimOrExtendleft Trim or extend left or right
	 * @param bDeleteKeys Delete keys outside the split ranges
	 */
	static UE_API void TrimOrExtendSection(UMovieSceneTrack* Track, TOptional<int32> RowIndex, FQualifiedFrameTime Time, bool bTrimOrExtendLeft, bool bDeleteKeys);

	/**
	 * Splits sections at the given time
	 *
	 * @param Sections The sections to split
	 * @param Time	The time at which to split
	 * @param bDeleteKeys Delete keys outside the split ranges
	 */
	static UE_API void SplitSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time, bool bDeleteKeys);
	static UE_API bool CanSplitSection(const TSet<UMovieSceneSection*>& Sections, FQualifiedFrameTime Time);

	static UE_API FTransform GetTransformOriginForFocusedSequence(TSharedPtr<ISequencer> InSequencer);

	/**
	 * Parse a shot name into its components.
	 *
	 * @param ShotName The shot name to parse
	 * @param ShotPrefix The parsed shot prefix
	 * @param ShotNumber The parsed shot number
	 * @param TakeNumber The parsed take number
	 * @param ShotNumberDigits The number of digits to pad for the shot number
	 * @param TakeNumberDigits The number of digits to pad for the take number
	 * @return Whether the shot name was parsed successfully
	 */
	static UE_API bool ParseShotName(const FString& ShotName, FString& ShotPrefix, uint32& ShotNumber, uint32& TakeNumber, uint32& ShotNumberDigits, uint32& TakeNumberDigits);

	/**
	 * Compose a shot name given its components.
	 *
	 * @param ShotPrefix The shot prefix to use
	 * @param ShotNumber The shot number to use
	 * @param TakeNumber The take number to use
	 * @param ShotNumberDigits The number of digits to pad for the shot number
	 * @param TakeNumberDigits The number of digits to pad for the take number
	 * @return The composed shot name
	 */
	static UE_API FString ComposeShotName(const FString& ShotPrefix, uint32 ShotNumber, uint32 TakeNumber, uint32 ShotNumberDigits, uint32 TakeNumberDigits);

	/**
	 * Generate a new subsequence package
	 *
	 * @param SequenceMovieScene The sequence movie scene for the new subsequence
	 * @param SubsequenceDirectory The directory for the new subsequence
	 * @param NewShotName The new shot name
	 * @return The new subsequence path
	 */
	static UE_API FString GenerateNewSubsequencePath(UMovieScene* SequenceMovieScene, const FString& SubsequenceDirectory, FString& NewShotName);
	
	UE_DEPRECATED(5.3, "GenerateNewShotPath has been deprecated in favor of GenerateNewSubsequencePath that takes a given directory")
	static UE_API FString GenerateNewShotPath(UMovieScene* SequenceMovieScene, FString& NewShotName);


	/**
	 * Generate a new shot name
	 *
	 * @param AllSections All the sections in the given shot track
	 * @param Time The time to generate the new shot name at
	 * @return The new shot name
	 */
	static UE_API FString GenerateNewSubsequenceName(const TArray<UMovieSceneSection*>& AllSections, const FString& SubsequencePrefix, FFrameNumber Time);

	UE_DEPRECATED(5.3, "GenerateNewShotName has been deprecated in favor of GenerateNewSubsequenceName that takes a given prefix")
	static UE_API FString GenerateNewShotName(const TArray<UMovieSceneSection*>& AllSections, FFrameNumber Time);

	/*
	 * Create sequence
	 *
	 * @param NewSequenceName The new sequence name.
	 * @param NewSequencePath The new sequence path. 
	 * @param SectionToDuplicate The section to duplicate.
	 * @param SequenceClass The class of sequence. Used only if SectionToDuplicate is not provided.
	 * @return The new subsequence.
	 */
	static UE_API UMovieSceneSequence* CreateSequence(FString& NewSequenceName, FString& NewSequencePath, UMovieSceneSubSection* SectionToDuplicate = nullptr, TSubclassOf<UMovieSceneSequence> SequenceClass = nullptr);

	/**
	 * Gather takes - level sequence assets that have the same shot prefix and shot number in the same asset path (directory)
	 * 
	 * @param Section The section to gather takes from
	 * @param AssetData The gathered asset take data
	 * @param OutCurrentTakeNumber The current take number of the section
	 */
	static UE_API void GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber);

	/**
	 * Get the take number for the given asset
	 *
	 * @param Section The section to gather the take number from
	 * @param AssetData The take asset to search for
	 * @param OutTakeNumber The take number for the given asset
	 * @return Whether the take number was found
	 */
	static UE_API bool GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber);

	/**
	 * Set the take number for the given asset
	 *
	 * @param Section The section to set the take number on
	 * @param InTakeNumber The take number for the given asset
	 * @return Whether the take number could be set
	 */
	static UE_API bool SetTakeNumber(const UMovieSceneSection* Section, uint32 InTakeNumber);

	/**
	 * Get the next available row index for the section so that it doesn't overlap any other sections in time.
	 *
	 * @param InTrack The track to find the next available row on
	 * @param InSection The section
	 * @param SectionsToDisregard Disregard checking these sections
	 * @return The next available row index
	 */
	static UE_API int32 FindAvailableRowIndex(UMovieSceneTrack* InTrack, UMovieSceneSection* InSection, const TArray<UMovieSceneSection*>& SectionsToDisregard = TArray<UMovieSceneSection*>());

	/**
	 * Does this section overlap any other track section?
	 *
	 * @param InTrack The track to find sections on
	 * @param InSection The section
	 * @param SectionsToDisregard Disregard checking these sections
	 * @return Whether this section overlaps any other track section
	 */
	static UE_API bool OverlapsSection(UMovieSceneTrack* InTrack, UMovieSceneSection* InSection, const TArray<UMovieSceneSection*>& SectionsToDisregard = TArray<UMovieSceneSection*>());

	/**
	 * Generate a combobox for editing enum values
	 *
	 * @param Enum The enum to make the combobox from
	 * @param CurrentValue The current value to display
	 * @param OnSelectionChanged Delegate fired when selection is changed
	 * @return The new widget
	 */
	static UE_API TSharedRef<SWidget> MakeEnumComboBox(const UEnum* Enum, TAttribute<int32> CurrentValue, SEnumComboBox::FOnEnumSelectionChanged OnSelectionChanged);


	/**
	 * Show Import EDL Dialog
	 *
	 * @param InMovieScene The movie scene to import the edl into
	 * @param InFrameRate The frame rate to import the EDL at
	 * @param InOpenDirectory Optional directory path to open from. If none given, a dialog will pop up to prompt the user
	 * @return Whether the import was successful
	 */
	static UE_API bool ShowImportEDLDialog(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InOpenDirectory = TEXT(""));

	/**
	 * Show Export EDL Dialog
	 *
	 * @param InMovieScene The movie scene with the cinematic shot track and audio tracks to export
	 * @param InFrameRate The frame rate to export the EDL at
	 * @param InSaveDirectory Optional directory path to save to. If none given, a dialog will pop up to prompt the user
	 * @param InHandleFrames The number of handle frames to include for each shot.
	 * @param MovieExtension The movie extension for the shot filenames (ie. .avi, .mov, .mp4)
	 * @return Whether the export was successful
	 */
	static UE_API bool ShowExportEDLDialog(const UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InSaveDirectory = TEXT(""), int32 InHandleFrames = 8, FString InMovieExtension = TEXT(".avi"));

	/**
	* Import movie scene formats
	*
	* @param InImporter The movie scene importer.
	* @param InMovieScene The movie scene to import the format into
	* @param InFrameRate The frame rate to import the format at
	* @param InOpenDirectory Optional directory path to open from. If none given, a dialog will pop up to prompt the user
	* @return Whether the import was successful
	*/
	static UE_API bool MovieSceneTranslatorImport(FMovieSceneImporter* InImporter, UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InOpenDirectory = TEXT(""));

	/**
	* Export movie scene formats
	*
	* @param InExporter The movie scene exporter.
	* @param InMovieScene The movie scene with the cinematic shot track and audio tracks to export
	* @param InFrameRate The frame rate to export the AAF at
	* @param InSaveDirectory Optional directory path to save to. If none given, a dialog will pop up to prompt the user
	* @param InHandleFrames The number of handle frames to include for each shot.
	* @return Whether the export was successful
	*/
	static UE_API bool MovieSceneTranslatorExport(FMovieSceneExporter* InExporter, const UMovieScene* InMovieScene, const FMovieSceneCaptureSettings& Settings);

	/** 
	* Log messages and display error message window for MovieScene translators
	*
	* @param InTranslator The movie scene importer or exporter.
	* @param InContext The context used to gather error, warning or info messages during import or export.
	* @param bDisplayMessages Whether to open the message log window after adding the message.
	*/
	static UE_API void MovieSceneTranslatorLogMessages(FMovieSceneTranslator* InTranslator, TSharedRef<FMovieSceneTranslatorContext> InContext, bool bDisplayMessages);

	/**
	* Log error output for MovieScene translators
	*
	* @param InTranslator The movie scene importer or exporter.
	* @param InContext The context used to gather error, warning or info messages during import or export.
	*/
	static UE_API void MovieSceneTranslatorLogOutput(FMovieSceneTranslator* InTranslator, TSharedRef<FMovieSceneTranslatorContext> InContext);

	/**
	* Export FBX
	*
	* @param World The world to export from
	* @param AnimExportSequenceParameters The sequence parameters used for evaluation
	* @param Bindings The sequencer binding map
	* @param Tracks The tracks to export
	* @param NodeNameAdaptor Adaptor to look up actor names.
	* @param Template Movie scene sequence id.
	* @param InFBXFileName the fbx file name.
	* @return Whether the export was successful
	*/
	static UE_API bool ExportFBX(UWorld* World, const FAnimExportSequenceParameters& AnimExportSequenceParameters, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef& Template, const FString& InFBXFileName);

	UE_DEPRECATED(5.5, "ExportFBX taking movie scene has been deprecated in favor of a new function that takes current and root movie scene sequences")
	static UE_API bool ExportFBX(UWorld* World, UMovieScene* MovieScene, IMovieScenePlayer* Player, const TArray<FGuid>& Bindings, const TArray<UMovieSceneTrack*>& Tracks, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef& Template,  const FString& InFBXFileName, FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	* Import FBX with dialog
	*
	* @param InMovieScene The movie scene to import the fbx into
	* @param InObjectBindingNameMap The object binding to name map to map import fbx animation onto
	* @param bCreateCameras Whether to allow creation of cameras if found in the fbx file.
	* @return Whether the import was successful
	*/
	static UE_API bool ImportFBXWithDialog(UMovieSceneSequence* InSequence, ISequencer& InSequencer, const TMap<FGuid, FString>& InObjectBindingNameMap, TOptional<bool> bCreateCameras);

	/**
	* Get FBX Ready for Import. This make sure the passed in file make be imported. After calling this call ImportReadiedFbx. It returns out some parameters that we forcably change so we reset them later.
	*
	* @param ImportFileName The filename to import into
	* @param ImportFBXSettings FBX Import Settings to enforce.
	* @param OutFBXParams Paremter to pass back to ImportReadiedFbx
	* @return Whether the fbx file was ready and is ready to be import.
	*/
	static UE_API bool ReadyFBXForImport(const FString&  ImportFilename, UMovieSceneUserImportFBXSettings* ImportFBXSettings, FFBXInOutParameters& OutFBXParams);

	/**
	* Import into an FBX scene that has been readied already, via the ReadyFBXForImport call. We do this as two pass in case the client want's to do something, like create camera's, before actually
	* loading the data.
	*
	* @param World The world to import the fbx into
	* @param InMovieScene The movie scene to import the fbx into
	* @param ObjectBindingMap Map relating binding id's to track names. 
	* @param TemplateID Id of the sequence that contains the objects being imported onto 
	* @param ImportFBXSettings FBX Import Settings to enforce.
	* @param InFBXParams Paremter from ImportReadiedFbx used to reset some internal fbx settings that we override.
	* @return Whether the fbx file was ready and is ready to be import.
	*/

	static UE_API bool ImportFBXIfReady(UWorld* World, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, TMap<FGuid, FString>& ObjectBindingMap, UMovieSceneUserImportFBXSettings* ImportFBXSettings,
		const FFBXInOutParameters& InFBXParams, ISequencer* Sequencer = nullptr);

	/**
	* Import FBX Camera to existing camera's
	*
	* @param FbxImporter The Fbx importer
	* @param InMovieScene The movie scene to import the fbx into
	* @param Player The player we are getting objects from.
	* @param TemplateID Id of the sequence that contains the objects being imported onto
	* @param InObjectBindingNameMap The object binding to name map to map import fbx animation onto
	* @param bCreateCameras Whether to allow creation of cameras if found in the fbx file.
	* @param bNotifySlate  If an issue show's up, whether or not to notify the UI.
	* @return Whether the import was successful
	*/

	static UE_API void ImportFBXCameraToExisting(UnFbx::FFbxImporter* FbxImporter, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, TMap<FGuid, FString>& InObjectBindingMap, bool bMatchByNameOnly, bool bNotifySlate);

	/**
	* Import FBX Node to existing actor/node
	*
	* @param NodeName Name of fbx node/actor
	* @param CurvesApi Existing FBX curves coming in
	* @param InMovieScene The movie scene to import the fbx into
	* @param Player The player we are getting objects from.
	* @param TemplateID Id of the sequence template ID.
	* @param ObjectBinding Guid of the object we are importing onto.
	* @return Whether the import was successful
	*/
	static UE_API bool ImportFBXNode(FString NodeName, UnFbx::FFbxCurvesAPI& CurveAPI, UMovieSceneSequence* InSequence, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, FGuid ObjectBinding, ISequencer* Sequencer = nullptr);

	/**
	 * Lock the given camera actor to the viewport
	 *
	 * @param Sequencer The Sequencer to set the camera cut enabled for
	 * @param CameraActor The camera actor to lock
	 */
	static UE_API void LockCameraActorToViewport(const TSharedPtr<ISequencer>& Sequencer, ACameraActor* CameraActor);

	/**
	 * Create a new camera cut section for the given camera
	 *
	 * @param MovieScene MovieScene to add Camera.
	 * @param CameraGuid CameraGuid  Guid of the camera that was added.
	 * @param FrameNumber FrameNumber it's added at.
	 */
	static UE_API void CreateCameraCutSectionForCamera(UMovieScene* MovieScene, FGuid CameraGuid, FFrameNumber FrameNumber);

	/**
	* Import FBX Camera to existing camera's
	*
	* @param CameraNode The Fbx camera
	* @param InCameraActor UE actor
	*/
	static UE_API void CopyCameraProperties(fbxsdk::FbxCamera* CameraNode, AActor* InCameraActor);


	/*
	 * Export the SkelMesh to an Anim Sequence for specified MovieScene and Player
	 *
	 * @param AnimSequence The sequence to save to.
	 * @param ExportOptions The options to use when saving.
	 * @param AnimExportSequenceParameters The Sequence parameters like the movie scene sequences, the player, template id, used for evaluation
	 * @param SkelMesh The Player to evaluatee.
	 * @return Whether or not it succeeds

	*/
	static UE_API bool ExportToAnimSequence(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FAnimExportSequenceParameters& AnimExportSequenceParameters,
			USkeletalMeshComponent* SkelMesh);

	UE_DEPRECATED(5.5, "ExportToAnimSequence taking movie scene has been deprecated in favor of a new function that takes current and root movie scene sequences")
	static UE_API bool ExportToAnimSequence(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, UMovieScene* MovieScene, IMovieScenePlayer* Player,
		USkeletalMeshComponent* SkelMesh, FMovieSceneSequenceIDRef& Template, FMovieSceneSequenceTransform& RootToLocalTransform);

	/*
	 * Bake the SkelMesh to a generic object wich implements a set of callbacks
	 *
	 * @param MovieSceneSequence The movie scene sequence that's current
	 * @param RootMovieSceneSequence The root movie scene sequence
	 * @param Player The Player to evaluate
	 * @param SkelMesh The Player to evaluate
	 * @param Template ID of the sequence template.
	 * @param RootToLocalTransform Transform Offset to apply to exported anim sequence.
	* @param ExportOptions The options to use when baking the mesh.
	 * @param InitCallback Callback before it starts running, maybe performance heavy.
	 * @param StartCallback Callback right before starting if needed, should be lightweight.
	 * @param TickCallback Callback per tick where you can bake the skelmesh.
	 * @param EndCallback Callback at end to finalize the baking.
	 * @return Whether or not it succeeds
	
	*/

	static UE_API bool BakeToSkelMeshToCallbacks(const FAnimExportSequenceParameters& AnimExportSequenceParameters, USkeletalMeshComponent* SkelMesh, UAnimSeqExportOption* ExportOptions,
		FInitAnimationCB InitCallback, FStartAnimationCB StartCallback, FTickAnimationCB TickCallback, FEndAnimationCB EndCallback);

	UE_DEPRECATED(5.5, "BakeToSkelMeshToCallbacks taking movie scene has been deprecated in favor of a new function that takes current and root movie scene sequences")
	static UE_API bool BakeToSkelMeshToCallbacks(UMovieScene* MovieScene, IMovieScenePlayer* Player,
		USkeletalMeshComponent* SkelMesh, FMovieSceneSequenceIDRef& Template, FMovieSceneSequenceTransform& RootToLocalTransform, UAnimSeqExportOption* ExportOptions,
		FInitAnimationCB InitCallback, FStartAnimationCB StartCallback, FTickAnimationCB TickCallback, FEndAnimationCB EndCallback);


	/*
	 * @return Whether this object class has hidden mobility and can't be animated
	 */
	static UE_API bool HasHiddenMobility(const UClass* ObjectClass);
	
	/*
	* Get the Active EvaluationTrack for a given track. Will do a recompile if the track isn't valid
	*@param Sequencer The sequencer we are evaluating
	*@aram Track The movie scene track whose evaluation counterpart we want
	*@return Returns the evaluation track for the given movie scene track. May do a re-compile if needed.
	*/
	static UE_API const FMovieSceneEvaluationTrack* GetEvaluationTrack(ISequencer *Sequencer, const FGuid& TrackSignature);


	/*
	* Get the location at time for the specified transform evaluation track
	*@param Track The sequencer we are evaluating
	*@param Object The object that owns this track
	*@param KeyTime the time to evaluate
	*@param KeyPos The position at this time
	*@param KeyRot The rotation at this time
	*@param Sequencer The Sequence that owns this track
	*/
	static UE_API void GetLocationAtTime(const FMovieSceneEvaluationTrack* Track, UObject* Object, FFrameTime KeyTime, FVector& KeyPos, FRotator& KeyRot, const TSharedPtr<ISequencer>& Sequencer);
	
	/* Get the Parents (Scene/Actors) of this object.
	* @param Parents Returned Parents
	* @param InObject Object to find parents for
	*/
	static UE_API void GetParents(TArray<const UObject*>& Parents, const UObject* InObject);
	
	/* Return Reference Frame from the passed in paretns
	* @param Sequencer The Sequence that's driving these parents.
	* @param Parents Parents in sequencer to evaluate to find reference transforms
	* @param KeyTime Time to Evaluate At
	* @return Returns Reference Transform.
	*/
	static UE_API FTransform GetRefFrameFromParents(const TSharedPtr<ISequencer>& Sequencer, const TArray<const UObject*>& Parents, FFrameTime KeyTime);

	/* Return Return ParentTm for current Parent Object
	* @param CurrentRefTM Current Referemnce TM
	* @param Sequencer The Sequence that's driving these parents.
	* @param ParentObject The Parent
	* @param KeyTime Time to Evaluate At
	* @return Returns true if succesful in evaluating the parent in the sequencer and getting a transform.
	*/
	static UE_API bool GetParentTM(FTransform& CurrentRefTM, const TSharedPtr<ISequencer>& Sequencer, UObject* ParentObject, FFrameTime KeyTime);

	/*
	 * Get the fbx cameras from the requested parent node
	 */
	static UE_API void GetCameras(fbxsdk::FbxNode* Parent, TArray<fbxsdk::FbxCamera*>& Cameras);

	/*
	 * Get the fbx camera name
	 */
	static UE_API FString GetCameraName(fbxsdk::FbxCamera* InCamera);

	/*
	 * Import FBX into Control Rig Channels With Dialog
	 */
	static UE_API bool ImportFBXIntoControlRigChannelsWithDialog(const TSharedRef<ISequencer>& InSequencer, TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels);

	/*
	** Export FBX from Control Rig Channels With Dialog
	*/
	static UE_API bool ExportFBXFromControlRigChannelsWithDialog(const TSharedRef<ISequencer>& InSequencer, UMovieSceneTrack* Track);

	/*
	* Import FBX into Control Rig Channels
	*/	
	static UE_API bool ImportFBXIntoControlRigChannels(UMovieScene* MovieScene, const FString& ImportFilename,  UMovieSceneUserImportFBXControlRigSettings *ControlRigSettings,
		TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels, const TArray<FName>& SelectedControlNames, FFrameRate FrameRate);

	/*
	** Export FBX from Control Rig Channels
	*/	
	static UE_API bool ExportFBXFromControlRigChannels(const UMovieSceneSection* Section, const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings,
	                                            const TArray<FName>& SelectedControlNames, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/*
	* Acquire first SkeletalMeshComponent from the Object
	* @param BoundObject Object to get SkeletalMeshComponent from.If actor checks it's components, if component checks itself then child components.
	* @return Returns the USkeletalMeshComponent if one is found
	*/
	static UE_API USkeletalMeshComponent* AcquireSkeletalMeshFromObject(UObject* BoundObject);
	
	/*
	* Get an actors and possible component parents.
	* @param InActorAndComponent Actor and possible component to find parents for
	* @param OutParentActors Returns an array of parents
	*/
	static UE_API void GetActorParents(const FActorForWorldTransforms& Actor,
		TArray<FActorForWorldTransforms>& OutParentActors);

	/*
	* Get an actors and possible component parents using sequencer to test for attachments.
	* @param Sequencer Sequencer to evaluate
	* @param InActorAndComponent Actor and possible component to find parents for
	* @param OutParentActors Returns an array of parents
	*/
	static UE_API void GetActorParentsWithAttachments(ISequencer* Sequencer, const FActorForWorldTransforms& Actor, TArray<FActorForWorldTransforms>& OutParentActors);

	/*
	*  Get an actors and it's parent key frames
	* @param Sequencer Sequencer to evaluate
	* @param Actor The actor and possible component and socket that we want to get the frame for
	* @param StartFrame The first frame to start looking for keys
	* @param EndFrame The last frame to stop looking for keys
	* @param OutFrameMap Sorted map of the frame times found
	*/
	static UE_API void GetActorsAndParentsKeyFrames(ISequencer* Sequencer, const FActorForWorldTransforms& Actor,
		const FFrameNumber& StartFrame, const FFrameNumber& EndFrame, TSortedMap<FFrameNumber, FFrameNumber>& OutFrameMap);

	/*
	*  Get an actors word transforms at the specified times
	* @param Sequencer Sequencer to evaluate
    * @param Actors The actor and possible component and socket that we want to get the world transforms for.
	* @param Frames The times we want to get the world transforms
	* @param OutWorldTransforms The calculated world transforms, one for each specified frame.
	*/
	static UE_API void GetActorWorldTransforms(ISequencer* Sequencer, const FActorForWorldTransforms& Actors, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutWorldTransforms);

	/* Set or add a key onto a float channel.
	* @param ChannelData Channel to set or add
	* @param Time Frame to add or set the value
	* @param Value  Value to Set
	* @param Interpolation Key type to set if added
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& ChannelData, FFrameNumber Time, float Value, const EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto);

	/* Set or add a key onto a double channel.
	* @param ChannelData Channel to set or add
	* @param Time Frame to add or set the value
	* @param Value  Value to Set
	* @param Interpolation Key type to set if added
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& ChannelData, FFrameNumber Time, double Value, const EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto);


	/*
	* Set or add a key onto a float channel based on key value.
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& Curve, FFrameNumber Time, const FMovieSceneFloatValue& Value);


	/*
	* Set or add a key onto a double channel based on key value.
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& ChannelData, FFrameNumber Time, FMovieSceneDoubleValue Value);


	/* 
	* Set or add a key onto a float channel based on rich curve data.
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& Curve, FFrameNumber Time, float Value, 
			float ArriveTangent, float LeaveTangent, ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode,
			FFrameRate FrameRate, ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone, 
			float ArriveTangentWeight = 0.0f, float LeaveTangentWeight = 0.0f);

	/*
	* Set or add a key onto a double channel based on rich curve data.
	*/
	static UE_API void SetOrAddKey(TMovieSceneChannelData<FMovieSceneDoubleValue>& Curve, FFrameNumber Time, double Value, 
			float ArriveTangent, float LeaveTangent, ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode,
			FFrameRate FrameRate, ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone, 
			float ArriveTangentWeight = 0.0f, float LeaveTangentWeight = 0.0f);
	
	/*
	*  Get an actors world transforms at the specified times using a player
	* @param Player Player to evaluate
	* @param InSequence  Sequence to evaluate
	* @param Template  Sequence ID of the template to play
    * @param ActorForWorldTransforms The actor and possible component and socket that we want to get the world transforms for.
	* @param Frames The times we want to get the world transforms
	* @param OutWorldTransforms The calculated world transforms, one for each specified frame.
	*/
	static UE_API void GetActorWorldTransforms(IMovieScenePlayer* Player, UMovieSceneSequence* InSequence, FMovieSceneSequenceIDRef Template,const FActorForWorldTransforms& Actors, const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutWorldTransforms);

	/*
	 * Return whether this asset is valid for the given sequence
	 */
	static UE_API bool IsValidAsset(UMovieSceneSequence* Sequence, const FAssetData& InAssetData);

	/*
	* Collapse all of the sections specified onto the first one
	* @param InSequencer  Sequencer we are collpasing at
	* @param InOwnerTrack  The track that should be owning the sections we are collapsing
	* @param InSections The sections we are collapsing. Sections[0] will remain, te otheer ones will be deleted
	* @param InSettings Baking settings ussed
	* @return Return true if succeeds false if it doesn't
	*/
	static UE_API bool CollapseSection(TSharedPtr<ISequencer>& InSequencer, UMovieSceneTrack* InOwnerTrack, TArray<UMovieSceneSection*> InSections,
		const FBakingAnimationKeySettings& InSettings);

	/*
	* Split set of sections to one containing the BlendType the other not
	* @param InSections The sections we are searching
	* @param OutSections All non-BlendType sections
	* @param OutAbsoluteSections All BlendType sections
	*/
	static UE_API void SplitSectionsByBlendType(EMovieSceneBlendType BlendType, const TArray<UMovieSceneSection*>& InSections, TArray<UMovieSceneSection*>& OutSections, TArray<UMovieSceneSection*>& OutBlendTypeSections);

	/*
	* Get the channel values at the specified time with the specified sections, will get
	* the channels from within the specified start and end indices
	* @param StartIndex Start Index to get values from
	* @param EndIndex End Index to get values from, should be no greater than the number of channels -1
	* @param Sections List of non-absolute(additive and override) sections to evaluate. Note this may not be all of the sections in the track,
	* but just a subset, which allows us to evaluate up to a certain point in the track.
	* @param AbsoluteSections List of absolute channels to evaluate
	* @param FrameTime Frame to evaluate
	* @return Returns the total channel value of those sections for each channel. Remember if OverrideChannelIndex is set this will just be one channel.
	*/
	template<typename ChannelType, typename CurveValueType>
	static TArray<CurveValueType> GetChannelValues(const int32 StartIndex, const int32 EndIndex, const TArray<UMovieSceneSection*>& Sections, const TArray<UMovieSceneSection*>& AbsoluteSections, const FFrameNumber& FrameTime)
	{
		TArray<CurveValueType> Values;
		int32 NumChannels = 0;
		if (Sections.Num() > 0)
		{
			TArrayView<ChannelType*>Channels = Sections[0]->GetChannelProxy().GetChannels<ChannelType>();
			NumChannels = Channels.Num();
		}
		else if(AbsoluteSections.Num() > 0)
		{
			TArrayView<ChannelType*>Channels = AbsoluteSections[0]->GetChannelProxy().GetChannels<ChannelType>();
			NumChannels = Channels.Num();
		}
		else
		{
			UE_LOGF(LogMovieScene, Warning, "GetChannelValues:: Invalid number of channels");
			return Values;
		}
		if (StartIndex < 0 || EndIndex >= NumChannels || EndIndex < StartIndex)
		{
			UE_LOGF(LogMovieScene, Warning, "GetChannelValues:: Invalid Start/End indices");
			return Values;
		}
		
		for (int32 ChannelIndex = StartIndex; ChannelIndex <= EndIndex; ++ChannelIndex)
		{
			CurveValueType Value = 0.0;
			if (AbsoluteSections.Num() > 0)
			{
				for (UMovieSceneSection* AbsoluteSection : AbsoluteSections)
				{
					TArrayView<ChannelType*>Channels = AbsoluteSection->GetChannelProxy().GetChannels<ChannelType>();
					float Weight = AbsoluteSection->GetTotalWeightValue(FrameTime);
					CurveValueType WeightedValue = 0.0;
					ChannelType* Channel = Channels[ChannelIndex];
					Channel->Evaluate(FrameTime, WeightedValue);
					WeightedValue *= ((double)Weight);
					Value += WeightedValue;
				}

				Value /= (double)AbsoluteSections.Num();		
			}

			for (UMovieSceneSection* Section : Sections)
			{
				TArrayView<ChannelType*> Channels = Section->GetChannelProxy().GetChannels<ChannelType>();
				float Weight = Section->GetTotalWeightValue(FrameTime);
				CurveValueType WeightedValue = 0.0;
				ChannelType* Channel = Channels[ChannelIndex];
				Channel->Evaluate(FrameTime, WeightedValue);
				if (Section->GetBlendType().Get() == EMovieSceneBlendType::Additive)
				{
					WeightedValue *= Weight;
					Value += WeightedValue;
				}
				else if (Section->GetBlendType().Get() == EMovieSceneBlendType::Override)
				{
					Value = (Value * (1.0 - Weight)) +
						(WeightedValue * Weight);
				}
			}
			Values.Add(Value);
		}
		return Values;
	}


	/*
	* Merge the value based upon the specified merge algorithm using cached parameters
	* For the algorithm to be relaible it should be done per section blending type.
	* @param InOutValue  Current Value you want to merge with the other sections.
	* @param Channels  Set of channels of this type that we will merge together onto the first specified one.
	* @param Frame  Current Frame to merge
	* @param SectionChannelIndex  The index of the channel of this type in the sections.
	* @param Sections Set of corresponding sections for each channel specified.
	* @param AbsoluteSections Set of absoluate sections for each channel specified.
	* @param OtherSections Set of non-absolute sections for each channel specified.
	* @param OverrideChanneIndex ChannelIndex for an override merge.
	* @param MergeAlgorithm The algorithm to use to blend the channels together.
	* @return Returns true if successful.
	*/
	template<typename ChannelType, typename CurveValueType>
	static bool MergeValue(CurveValueType& InOutValue, TArray<ChannelType*>& Channels, const FFrameNumber& Frame, int32 SectionChannelIndex, const  TArray<UMovieSceneSection*>& Sections, 
		const TArray<UMovieSceneSection*>& AbsoluteSections, const TArray<UMovieSceneSection*>& OtherSections, int32 OverrideChannelIndex, FChannelMergeAlgorithm MergeAlgorithm)
	{
		const FFrameTime FrameTime(Frame);
		if (MergeAlgorithm == FChannelMergeAlgorithm::Average)
		{
			double DNumChannels = 0.0;
			for (int32 WeightIndex = 0; WeightIndex < Sections.Num(); ++WeightIndex)
			{
				if (Sections[WeightIndex]->GetRange().Contains(Frame))
				{
					float Weight = Sections[WeightIndex]->GetTotalWeightValue(FrameTime);
					CurveValueType WeightedValue = 0.0;
					ChannelType* EachChannel = Channels[WeightIndex];
					EachChannel->Evaluate(FrameTime, WeightedValue);
					WeightedValue *= ((double)Weight);
					InOutValue += WeightedValue;
					DNumChannels += 1.0;
				}
			}
			if (DNumChannels > 0.0) //should always happen since base(0) at least be here
			{
				InOutValue /= DNumChannels;
			}
		}
		else if (MergeAlgorithm == FChannelMergeAlgorithm::Add)
		{
			for (int32 WeightIndex = 0; WeightIndex < Sections.Num(); ++WeightIndex)
			{
				if (Sections[WeightIndex]->GetRange().Contains(Frame))
				{
					float Weight = Sections[WeightIndex]->GetTotalWeightValue(FrameTime);
					if (Sections[WeightIndex]->GetBlendType().IsValid() == false ||
						Sections[WeightIndex]->GetBlendType().Get() != EMovieSceneBlendType::Additive)
					{
						Weight = 1.0;
					}
					CurveValueType WeightedValue = 0.0;
					ChannelType* EachChannel = Channels[WeightIndex];
					EachChannel->Evaluate(FrameTime, WeightedValue);
					WeightedValue *= Weight;
					InOutValue += WeightedValue;
				}
			}
		}
		else if (MergeAlgorithm == FChannelMergeAlgorithm::Override)
		{
			//when doing an override merge we need to just get the full value since the new layer will also be
			//an override layer
			if (Sections[1]->GetRange().Contains(Frame))
			{
				TArray<CurveValueType> ChannelValues = MovieSceneToolHelpers::GetChannelValues<ChannelType,
					CurveValueType>(OverrideChannelIndex, OverrideChannelIndex, OtherSections, AbsoluteSections, Frame);
				if (ChannelValues.Num() == 1)
				{
					InOutValue = ChannelValues[0];
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	/*
	* Merge the set of passed in channels from each section at the specified section channel index.
	* For the algorithm to be relaible it should be done per section blending type.
	* @param SectionChannelIndex  The index of the channel of this type in the sections.
	* @param Channels  Set of channels of this type that we will merge together onto the first specified one.
	* @param Sections Set of corresponding sections for each channel specified.
	* @param Range Minimal range of time over which we will merge, we will always extend to key extents if possible
	* @param MergeAlgorithm The algorithm to use to blend the channels together
	* @param TrackSections Sections in this track
	* @param Increment Optional increment index, if specfied we bake over the range using the increment value, otherwise by default we just merge onto the existing keys
	* @return Returns true if successful
	*/	
	template<typename ChannelType>
	static bool MergeChannels(int32 SectionChannelIndex, TArray<ChannelType*>& Channels, const  TArray<UMovieSceneSection*>& Sections, const TRange<FFrameNumber>& Range,
		FChannelMergeAlgorithm MergeAlgorithm, const TArray<UMovieSceneSection*>& TrackSections, const int32* Increment = nullptr)
	{
		if (Channels.Num() < 2 || Channels.Num() != Sections.Num())
		{
			UE_LOGF(LogMovieScene, Warning, "MergeChannels:: Invalid number of channels");
			return false;
		}
		//for overrides since the need to evaluate the full value we only support 2 sections
		if (MergeAlgorithm == FChannelMergeAlgorithm::Override && Sections.Num() != 2)
		{
			UE_LOGF(LogMovieScene, Warning, "MergeChannels:: Can only do Override Blend with two sections");
			return false;
		}
		using ChannelValueType = typename ChannelType::ChannelValueType;
		using CurveValueType = typename ChannelType::CurveValueType;

		TArray<UMovieSceneSection*> OtherSections;
		TArray<UMovieSceneSection*> AbsoluteSections;
		int32 OverrideChannelIndex = SectionChannelIndex;
		int32 NumChannelsInSection = INDEX_NONE;
		if (MergeAlgorithm == FChannelMergeAlgorithm::Override) //if we are overriding we need to get the absolute value at the frame!
	{
			MovieSceneToolHelpers::SplitSectionsByBlendType(EMovieSceneBlendType::Absolute,TrackSections, OtherSections, AbsoluteSections);
			TArrayView<ChannelType*> OverrideChannels = Sections[1]->GetChannelProxy().GetChannels<ChannelType>();
			NumChannelsInSection = OverrideChannels.Num();
		}
		//base channel we set values on
		ChannelType* BaseChannel = Channels[0];
		TMovieSceneChannelData<ChannelValueType> BaseChannelData = BaseChannel->GetData();

		//use following to get key extents
		TArray<FFrameNumber> KeyTimes;
		TArray<FKeyHandle> Handles;
		const FFrameNumber Min = TNumericLimits<FFrameNumber>::Lowest();
		const FFrameNumber Max = TNumericLimits<FFrameNumber>::Max();
		TRange<FFrameNumber> MaxRange(Min, Max);
		//similar but slightly different implementations based upon whether or not doing per key or by frame.
		//for performance we do this outside of the loop
		if (Increment != nullptr && *Increment > 0)
		{
			//iterate over each frame
			if (UMovieScene* MovieScene = Sections[0]->GetTypedOuter<UMovieScene>())
			{
				TRange<FFrameNumber> RangeToUse = Range;
				for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
				{
					KeyTimes.Reset();
					Handles.Reset();
					ChannelType* Channel = Channels[ChannelIndex];
					Channel->GetKeys(MaxRange, &KeyTimes, &Handles);
					if (KeyTimes.Num() > 0)
					{
						if (KeyTimes[0] < RangeToUse.GetLowerBoundValue())
						{
							RangeToUse.SetLowerBoundValue(KeyTimes[0]);
						}
						if (KeyTimes[KeyTimes.Num() - 1] > RangeToUse.GetUpperBoundValue())
						{
							RangeToUse.SetUpperBoundValue(KeyTimes[KeyTimes.Num() - 1]);
						}
					}
					
				}
				TArray<FFrameNumber> Frames;
				CalculateFramesBetween(MovieScene, RangeToUse.GetLowerBoundValue(), RangeToUse.GetUpperBoundValue(), *Increment, Frames);
				
			
				TArray<TPair< FFrameNumber, CurveValueType>> ValuesToSet;
				for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
				{
					ChannelType* Channel = Channels[ChannelIndex];
					for (const FFrameNumber& Frame : Frames)
					{
						if (Sections[0]->GetRange().Contains(Frame) == false)  //frame is outside base range so skip
						{
							continue;
						}
						CurveValueType Value = 0.0;
						MergeValue(Value, Channels, Frame, SectionChannelIndex, Sections, AbsoluteSections,
							OtherSections, OverrideChannelIndex, MergeAlgorithm);
						ValuesToSet.Add(TPair<FFrameNumber, CurveValueType>(Frame, Value));
					}
				}
				for (TPair<FFrameNumber, CurveValueType>& ValueToSet : ValuesToSet)
				{
					MovieSceneToolHelpers::SetOrAddKey(BaseChannelData, ValueToSet.Key, ValueToSet.Value);
				}
			}
		}
		else
		{
			//cached set that we set at the end
			TArray<TPair< FFrameNumber, ChannelValueType>> KeysToSet;
			for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
			{
				KeyTimes.Reset();
				Handles.Reset();
				ChannelType* Channel = Channels[ChannelIndex];
				Channel->GetKeys(MaxRange, &KeyTimes, &Handles);
				for (int32 FrameIndex = 0; FrameIndex < KeyTimes.Num(); ++FrameIndex)
				{
					const FFrameNumber& Frame = KeyTimes[FrameIndex];
					if (Sections[0]->GetRange().Contains(Frame) == false)  //frame is outside base range so extend it
					{
						if (Sections[0]->HasEndFrame() && Sections[0]->GetExclusiveEndFrame() <= Frame)
						{
							if (Sections[0]->GetExclusiveEndFrame() != Frame)
							{
								Sections[0]->SetEndFrame(Frame);
							}
						}
						else
						{
							Sections[0]->SetStartFrame(Frame);
						}
					}
					const FFrameTime FrameTime(Frame);
					int32 KeyIndex = Channel->GetData().GetIndex(Handles[FrameIndex]);
					ChannelValueType Value = Channel->GetData().GetValues()[KeyIndex];
					//got value with tangents and times, now we perform the operation
					Value.Value = 0.0; //zero out the value we calculate it 

					MergeValue(Value.Value, Channels, Frame, SectionChannelIndex, Sections, AbsoluteSections,
						OtherSections, OverrideChannelIndex, MergeAlgorithm);
					KeysToSet.Add(TPair<FFrameNumber, ChannelValueType>(Frame, Value));
				}
			}
			for (TPair<FFrameNumber, ChannelValueType>& KeyToSet : KeysToSet)
			{
				MovieSceneToolHelpers::SetOrAddKey(BaseChannelData, KeyToSet.Key, KeyToSet.Value);
			}
		}
		return true;
	}

	struct FBaseTopSections
	{
		FBaseTopSections(UMovieSceneSection* InBaseSection,
						 UMovieSceneSection* InTopSection,
						 int32 InBaseStartIndex,
						 int32 InBaseEndIndex,
						 int32 InTopStartIndex,
						 int32 InTopEndIndex) :
			BaseSection(InBaseSection),
			TopSection(InTopSection),
			BaseStartIndex(InBaseStartIndex),
			BaseEndIndex(InBaseEndIndex),
			TopStartIndex(InTopStartIndex),
			TopEndIndex(InTopEndIndex)
		{}
		UMovieSceneSection* BaseSection;
		UMovieSceneSection* TopSection;
		int32 BaseStartIndex;
		int32 BaseEndIndex;
		int32 TopStartIndex; 
		int32 TopEndIndex;
	};
	/*
	* Merge the TopSection onto the BaseSection. This function should be used when merging
	* a mix of Override/Additive and Absolute section Blend Types.
	* @param BaseAndTopSections The sections we are merging with index information
	* @param Range Range of time over which we will merge  Control Rig sections and don't want to merge the last weight float channel
	* @param TrackSections Sections in this track
	* @param Increment Optional increment index, if specfied we bake over the range using the increment value, otherwise by default we just merge onto the existing keys
	* @return Returns true if successful
	*/	
	template<typename ChannelType>
	static bool MergeTwoSections(const FBaseTopSections& BT, const TRange<FFrameNumber>& Range, const TArray<UMovieSceneSection*>& TrackSections, const int32* Increment = nullptr)
	{
		TArrayView<ChannelType*> BaseChannels = BT.BaseSection->GetChannelProxy().GetChannels<ChannelType>();
		TArrayView<ChannelType*> TopChannels = BT.TopSection->GetChannelProxy().GetChannels<ChannelType>();
		if (TopChannels.Num() != BaseChannels.Num())
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid number of channels");
			return false;
		}
		if (BT.BaseStartIndex < 0 || BT.BaseEndIndex >= BaseChannels.Num() || BT.BaseEndIndex < BT.BaseStartIndex)
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid Start/End indices");
			return false;
		}

		if (BT.TopStartIndex < 0 || BT.TopEndIndex >= BaseChannels.Num() || BT.TopEndIndex < BT.TopStartIndex)
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid Start/End indices");
			return false;
		}

		if ((BT.BaseEndIndex - BT.BaseStartIndex) != (BT.TopEndIndex - BT.TopStartIndex))
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid Start/End indices");
			return false;
		}

		TArray<ChannelType*> Channels;
		int32 ChannelIndex = 0;
		TArray<UMovieSceneSection*> Sections;
		Sections.Add(BT.BaseSection);
		Sections.Add(BT.TopSection);
		FChannelMergeAlgorithm MergeAlgorithm = FChannelMergeAlgorithm::Add;
		//if either setion is override we do an override blend since the result will be override
		if ((BT.TopSection->GetBlendType().IsValid() && BT.TopSection->GetBlendType().Get() == EMovieSceneBlendType::Override) ||
			(BT.BaseSection->GetBlendType().IsValid() && BT.BaseSection->GetBlendType().Get() == EMovieSceneBlendType::Override))
		{
			MergeAlgorithm = FChannelMergeAlgorithm::Override;
		}
		else if (BT.TopSection->GetBlendType().IsValid() && BT.TopSection->GetBlendType().Get() == EMovieSceneBlendType::Absolute)

		{
			MergeAlgorithm = FChannelMergeAlgorithm::Average;
		}

		int32 TopChannelIndex = BT.TopStartIndex;
		for (ChannelIndex = BT.BaseStartIndex; ChannelIndex <= BT.BaseEndIndex; ++ChannelIndex)
		{
			Channels.Reset();
			Channels.Add(BaseChannels[ChannelIndex]);
			Channels.Add(TopChannels[TopChannelIndex]);
			++TopChannelIndex;
			if (MergeChannels(ChannelIndex, Channels, Sections, Range,
				MergeAlgorithm, TrackSections, Increment) == false)
			{
				UE_LOGF(LogMovieScene, Warning, "MergeSections:: Could not merge channels");
				return false;
			}
		}
		
		for (ChannelType* Channel : BaseChannels)
		{
			Channel->AutoSetTangents();
		}
		
		return true;
	}

	template<typename ChannelType>
	static bool MergeSections(UMovieSceneSection* BaseSection, UMovieSceneSection* TopSection,
		int32 StartIndex, int32 EndIndex, const TRange<FFrameNumber>& Range, const TArray<UMovieSceneSection*>& TrackSections, const int32* Increment = nullptr)
	{
		FBaseTopSections BT(BaseSection, TopSection, StartIndex, EndIndex, StartIndex, EndIndex);
		return MergeTwoSections<ChannelType>(BT, Range, TrackSections, Increment);
	}

	
	/*
	* Merge the following set of sections. Should be used when blending just Absolute with additive sections. If Merging with
	* overrides, use the above MergeSections function
	* @param BaseSection  The section to merge onto
	* @param AbsoluteSections  Set of absolute functions to merge
	* @param AddditveSections  Set of Additive functions to merge
	* @param StartIndex Start Index of the channels we are merging
	* @param EndIndex End Index of the channels we are merging, should be no greater than the number of channels -1
	* @param Range Range of time over which we will merge Control Rig sections and don't want to merge the last weight float channel
	* @param Increment Optional increment index, if specfied we bake over the range using the increment value, otherwise by default we just merge onto the existing keys
	* @return Returns true if successful
	*/	
	template<typename ChannelType>
	static bool MergeSections(UMovieSceneSection* BaseSection, TArray<UMovieSceneSection*>& AbsoluteSections, TArray<UMovieSceneSection*>& AdditiveSections,
		int32 StartIndex, int32 EndIndex, const TRange<FFrameNumber>& Range, const int32* Increment = nullptr)
	{
		TArrayView<ChannelType*> BaseChannels = BaseSection->GetChannelProxy().GetChannels<ChannelType>();
		if (StartIndex < 0 || EndIndex >= BaseChannels.Num() || EndIndex < StartIndex)
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid Start/End indices");
			return false;
		}
		if (BaseChannels.Num() > 0)
		{
			int32 SectionIndex = 0;
			//sanity check to make sure channels are the same size
			for (SectionIndex = 0; SectionIndex < AbsoluteSections.Num(); ++SectionIndex)
			{
				TArrayView<ChannelType*>Channels = AbsoluteSections[SectionIndex]->GetChannelProxy().GetChannels<ChannelType>();
				if (Channels.Num() != BaseChannels.Num())
				{
					UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid number of channels");
					return false;
				}
			}
			for (SectionIndex = 0; SectionIndex < AdditiveSections.Num(); ++SectionIndex)
			{
				TArrayView<ChannelType*>Channels = AdditiveSections[SectionIndex]->GetChannelProxy().GetChannels<ChannelType>();
				if (Channels.Num() != BaseChannels.Num())
				{
					UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid number of channels");
					return false;
				}
			}
			TArray<UMovieSceneSection*> TrackSections;
			if (UMovieSceneTrack* OwnerTrack = BaseSection->GetTypedOuter<UMovieSceneTrack>())
			{
				TrackSections = OwnerTrack->GetAllSections();
			}
			else
			{
				UE_LOGF(LogMovieScene, Warning, "MergeSections:: No Owner Track");
				return false;
			}
			TArray<ChannelType*> Channels;
			
			BaseSection->Modify();

			for (int32 ChannelIndex = StartIndex; ChannelIndex <= EndIndex; ++ChannelIndex)
			{
				if (AbsoluteSections.Num() > 0)
				{
					Channels.Reset();
					for (SectionIndex = 0; SectionIndex < AbsoluteSections.Num(); ++SectionIndex)
					{
						TArrayView<ChannelType*>OurChannels = AbsoluteSections[SectionIndex]->GetChannelProxy().GetChannels<ChannelType>();
						Channels.Add(OurChannels[ChannelIndex]);
					}
					//now blend them
					if (MergeChannels(ChannelIndex, Channels, AbsoluteSections, Range,
						FChannelMergeAlgorithm::Average, TrackSections, Increment) == false)
					{
						UE_LOGF(LogMovieScene, Warning, "MergeSections:: Could not merge channels");
						return false;
					}
				}
				if (AdditiveSections.Num() > 0)
				{
					//now do additives
					Channels.Reset();
					for (SectionIndex = 0; SectionIndex < AdditiveSections.Num(); ++SectionIndex)
					{
						TArrayView<ChannelType*>OurChannels = AdditiveSections[SectionIndex]->GetChannelProxy().GetChannels<ChannelType>();
						Channels.Add(OurChannels[ChannelIndex]);
					}
					//now blend them
					if (MergeChannels(ChannelIndex,Channels, AdditiveSections, Range,
						FChannelMergeAlgorithm::Add, TrackSections, Increment) == false)
					{
						UE_LOGF(LogMovieScene, Warning, "MergeSections:: Could not merge channels");
						return false;
					}
				}
			}
			for (ChannelType* Channel : BaseChannels)
			{
				Channel->AutoSetTangents();
			}
		}
		else
		{
			UE_LOGF(LogMovieScene, Warning, "MergeSections:: Invalid number of channels");
			return false;
		}
		return true;
	}


	static UE_API bool OptimizeSection(const FKeyDataOptimizationParams& InParams, UMovieSceneSection* InSection);

	/** Returns the frame numbers between start and end. */
	static UE_API void CalculateFramesBetween(
		const UMovieScene* MovieScene,
		FFrameNumber StartFrame,
		FFrameNumber EndFrame,
		int FrameIncrement,
		TArray<FFrameNumber>& OutFrames);

	/** Returns the transform section for that guid. */
	static UE_API UMovieScene3DTransformSection* GetTransformSection(
		const ISequencer* InSequencer,
		const FGuid& InGuid,
		const FTransform& InDefaultTransform = FTransform::Identity);

	/** Adds transform keys to the section based on the channels filters. */
	static UE_API bool AddTransformKeys(
		const UMovieScene3DTransformSection* InTransformSection,
		const TArray<FFrameNumber>& Frames,
		const TArray<FTransform>& InLocalTransforms,
		const EMovieSceneTransformChannel& InChannels);

	/**
	* Adds transform keys to a specific object, using it's current transform

	* @param Object The object to add keys to.
	* @param ChannelToKey The channels to add keys to.
*	*/
	static UE_API void AddTransformKeysForObject(UObject* Object, EMovieSceneTransformChannel ChannelToKey);

	/** Import an animation sequence's root transforms into a transform section */
	static UE_API void ImportAnimSequenceTransforms(const TSharedRef<ISequencer>& Sequencer, const FAssetData& Asset, UMovieScene3DTransformTrack* TransformTrack);

	/** Import an animation sequence's root transforms into a transform section */
	static UE_API void ImportAnimSequenceTransformsEnterPressed(const TSharedRef<ISequencer>& Sequencer, const TArray<FAssetData>& Asset, UMovieScene3DTransformTrack* TransformTrack);

	/* Executes tasks at specified times.
	* @param Sequencer the sequencer to evaluate.
	* @param World the world context (used to retrieve constraints, can be null).
	* @param Frames the times at which the sequencer should be evaluated.
	* @param Tasks the tasks executed at each frame.
	*/
	static UE_API void RunSequencerTasks(
		ISequencer* Sequencer,
		UWorld* World,
		const TArray<FFrameNumber>& Frames,
		const TArrayView<TFunction<void(int32)>>& Tasks);

	/* Executes a task at specified times.
	* @param Sequencer the sequencer to evaluate.
	* @param World the world context (used to retrieve constraints, can be null).
	* @param Frames the times at which the sequencer should be evaluated.
	* @param Task the task executed at each frame.
	*/
	static UE_API void RunSequencerTask(
		ISequencer* Sequencer,
		UWorld* World,
		const TArray<FFrameNumber>& Frames,
		const TFunctionRef<void(int32)>& Task);
};

struct FSpawnableRestoreState
{
	UE_DEPRECATED(5.5, "This constructor is deprecated in favor of passing in shared playback state")
	UE_API FSpawnableRestoreState(UMovieScene* MovieScene);

	UE_DEPRECATED(5.7, "FSpawnableRestoreState has been deprecated in favor of FAllSpawnableRestoreState that turns off and restores spawnables for the whole sequence")
	UE_API FSpawnableRestoreState(UMovieScene* MovieScene, TSharedPtr<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState);
	UE_DEPRECATED(5.7, "FSpawnableRestoreState has been deprecated in favor of FAllSpawnableRestoreState that turns off and restores spawnables for the whole sequence")
	UE_API ~FSpawnableRestoreState();

	bool bWasChanged;
	TMap<FGuid, ESpawnOwnership> SpawnOwnershipMap;
	TWeakObjectPtr<UMovieScene> WeakMovieScene;
	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState;

};

// Helper to make all spawnables in all nested sub sequences persist throughout the export process and then restore properly afterwards
// You should use this when you may have multiple nested sequences, in particularly when using sequencer as the player
// If you create a player you need to pass in a MovieScene to use
struct FAllSpawnableRestoreState
{
	UE_API FAllSpawnableRestoreState(TSharedPtr<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState, UMovieScene* OptionalMovieScene = nullptr);
	UE_API ~FAllSpawnableRestoreState();
	bool bWasChanged = false;

	//helper function to set up the SpawnOwnershipMap given the binding, will update if the spawn track was updated
	static void SaveStateForSpawnable(bool& bInOutWasChanged, TMap<FGuid, ESpawnOwnership>& InOutSpawnOwnershipMap, TWeakObjectPtr<UMovieScene>& WeakMovieScene, FGuid BindingGuid, ESpawnOwnership PreviousSpawnOwnership, TFunction<void(ESpawnOwnership)> SpawnOwnershipSetter);

private:
	struct FPerMovieScene
	{
		TMap<FGuid, ESpawnOwnership> SpawnOwnershipMap;
		TWeakObjectPtr<UMovieScene> WeakMovieScene;
		FMovieSceneSequenceID SequenceID;
	};
	TMap <TWeakObjectPtr<UMovieScene>, FPerMovieScene> OwnershipMaps;
	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState;
};

class FTrackEditorBindingIDPicker : public FMovieSceneObjectBindingIDPicker
{
public:
	FTrackEditorBindingIDPicker(FMovieSceneSequenceID InLocalSequenceID, TWeakPtr<ISequencer> InSequencer)
		: FMovieSceneObjectBindingIDPicker(InLocalSequenceID, InSequencer)
	{
		Initialize();
	}

	DECLARE_EVENT_OneParam(FTrackEditorBindingIDPicker, FOnBindingPicked, FMovieSceneObjectBindingID)
	FOnBindingPicked& OnBindingPicked()
	{
		return OnBindingPickedEvent;
	}

	using FMovieSceneObjectBindingIDPicker::GetPickerMenu;

private:

	virtual UMovieSceneSequence* GetSequence() const override { return WeakSequencer.Pin()->GetFocusedMovieSceneSequence(); }
	virtual void SetCurrentValue(const FMovieSceneObjectBindingID& InBindingId) override { OnBindingPickedEvent.Broadcast(InBindingId); }
	virtual FMovieSceneObjectBindingID GetCurrentValue() const override { return FMovieSceneObjectBindingID(); }

	FOnBindingPicked OnBindingPickedEvent;
};


#undef UE_API

