// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/Guid.h"
#include "Curves/RichCurve.h"
#include "Containers/UnrealString.h"
#include "IMovieSceneTools.h"
#include "IMovieSceneRootMotionOffsetProvider.h"

#define UE_API MOVIESCENETOOLS_API

struct FAssetData;

class UK2Node;
class UBlueprint;
class UMovieScene;
class UMovieSceneSection;
class UMovieSceneEventSectionBase;
class IMovieSceneToolsTrackImporter;
class ULevelSequence;
class IStructureDetailsView;

class IMovieSceneToolsTakeData
{
public:
	virtual bool GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber) = 0;
	virtual bool GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber) = 0;
	virtual bool SetTakeNumber(const UMovieSceneSection*, uint32 InTakeNumber) = 0;
};

//Interface to get notifications when an animation bake happens in case in needs to run custom code
class IMovieSceneToolsAnimationBakeHelper
{
public:
	virtual void StartBaking(UMovieScene* MovieScene) {};
	virtual void PreEvaluation(UMovieScene* MovieScene, FFrameNumber Frame) {};
	virtual void PostEvaluation(UMovieScene* MovieScene, FFrameNumber Frame) {};
	virtual void StopBaking(UMovieScene* MovieScene) {};
};

struct FMovieSceneToolsAnimationBakingWrapper
{
	FMovieSceneToolsAnimationBakingWrapper(UMovieScene* InMovieScene);

	void Start() const;
	void PreEvaluate(const FFrameNumber& Frame) const;
	void PostEvaluate(const FFrameNumber& Frame) const;
	void Stop() const;
	
private:
	UMovieScene* MovieScene = nullptr;
	const TArray<IMovieSceneToolsAnimationBakeHelper*>& BakeHelpers;
};

// Interface to allow external modules to register additional key struct instanced property type customizations
class IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer
{
public:
	virtual void RegisterKeyStructInstancedPropertyTypeCustomization(TSharedRef<IStructureDetailsView> StructureDetailsView, TWeakObjectPtr<UMovieSceneSection> WeakOwningSection) {};
};

// Wraps UMovieSceneSkeletalAnimationSystem as an IMovieSceneRootMotionOffsetProvider
// so the transform track editor can subtract root motion during auto-key.
class FSkeletalAnimationRootMotionOffsetProvider : public IMovieSceneRootMotionOffsetProvider
{
public:
	UE_API virtual FTransform GetRootMotionOffset(const UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) const override;
	UE_API virtual void UpdateRootMotionOffset(UMovieSceneEntitySystemLinker* Linker, UObject* AnimatedObject) override;
};

/**
* Implements the MovieSceneTools module.
*/
class FMovieSceneToolsModule
	: public IMovieSceneTools
{
public:

	static inline FMovieSceneToolsModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FMovieSceneToolsModule >("MovieSceneTools");
	}

	// IModuleInterface interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API void RegisterAnimationBakeHelper(IMovieSceneToolsAnimationBakeHelper* BakeHelper);
	UE_API void UnregisterAnimationBakeHelper(IMovieSceneToolsAnimationBakeHelper* BakeHelper);
	const TArray<IMovieSceneToolsAnimationBakeHelper*>& GetAnimationBakeHelpers() { return BakeHelpers; }
	UE_API void RegisterTakeData(IMovieSceneToolsTakeData*);
	UE_API void UnregisterTakeData(IMovieSceneToolsTakeData*);

	UE_API bool GatherTakes(const UMovieSceneSection* Section, TArray<FAssetData>& AssetData, uint32& OutCurrentTakeNumber);
	UE_API bool GetTakeNumber(const UMovieSceneSection* Section, FAssetData AssetData, uint32& OutTakeNumber);
	UE_API bool SetTakeNumber(const UMovieSceneSection* Section, uint32 InTakeNumber);

	UE_API void RegisterTrackImporter(IMovieSceneToolsTrackImporter*);
	UE_API void UnregisterTrackImporter(IMovieSceneToolsTrackImporter*);

	UE_API bool ImportAnimatedProperty(const FString& InPropertyName, const FRichCurve& InCurve, FGuid InBinding, UMovieScene* InMovieScene);
	UE_API bool ImportStringProperty(const FString& InPropertyName, const FString& InPropertyValue, FGuid InBinding, UMovieScene* InMovieScene);

	UE_API void RegisterKeyStructInstancedPropertyTypeCustomizer(IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer*);
	UE_API void UnregisterKeyStructInstancedPropertyTypeCustomizer(IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer*);

	// Called By SKeyEditInterface to allow external modules to add key struct instanced property type customizations
	UE_API void CustomizeKeyStructInstancedPropertyTypes(TSharedRef<IStructureDetailsView> StructureDetailsView, TWeakObjectPtr<UMovieSceneSection> Section);

private:

	UE_API void RegisterClipboardConversions();

	static UE_API void FixupPayloadParameterNameForSection(UMovieSceneEventSectionBase* Section, UK2Node* InNode, FName OldPinName, FName NewPinName);
	static UE_API void FixupPayloadParameterNameForDynamicBinding(UMovieScene* MovieScene, UK2Node* InNode, FName OldPinName, FName NewPinName);
	static UE_API bool UpgradeLegacyEventEndpointForSection(UMovieSceneEventSectionBase* Section);
	static UE_API void PostDuplicateEventSection(UMovieSceneEventSectionBase* Section);
	static UE_API void RemoveForCookEventSection(UMovieSceneEventSectionBase* Section);
	static UE_API bool IsTrackClassAllowed(UClass* InClass);
	static UE_API bool IsCustomBindingClassAllowed(UClass* InClass);
	static UE_API bool IsConditionClassAllowed(const UClass* InClass);
	static UE_API void PostDuplicateEvent(ULevelSequence* LevelSequence);
	static UE_API void FixupDynamicBindingsEvent(ULevelSequence* LevelSequence);
	static UE_API void FixupPayloadParameterNameForDirectorBlueprintCondition(UMovieScene* MovieScene, UK2Node* InNode, FName OldPinName, FName NewPinName);

private:

	/** Registered delegate handles */
	FDelegateHandle BytePropertyTrackCreateEditorHandle;
	FDelegateHandle RotatorPropertyTrackCreateEditorHandle;
	FDelegateHandle VisibilityPropertyTrackCreateEditorHandle;
	FDelegateHandle ActorReferencePropertyTrackCreateEditorHandle;
	FDelegateHandle StringPropertyTrackCreateEditorHandle;
	FDelegateHandle TextPropertyTrackCreateEditorHandle;
	FDelegateHandle ObjectTrackCreateEditorHandle;

	FDelegateHandle AnimationTrackCreateEditorHandle;
	FDelegateHandle AttachTrackCreateEditorHandle;
	FDelegateHandle AudioTrackCreateEditorHandle;
	FDelegateHandle EventTrackCreateEditorHandle;
	FDelegateHandle ParticleTrackCreateEditorHandle;
	FDelegateHandle ParticleParameterTrackCreateEditorHandle;
	FDelegateHandle PathTrackCreateEditorHandle;
	FDelegateHandle CameraCutTrackCreateEditorHandle;
	FDelegateHandle CinematicShotTrackCreateEditorHandle;
	FDelegateHandle SlomoTrackCreateEditorHandle;
	FDelegateHandle SubTrackCreateEditorHandle;
	FDelegateHandle TransformTrackCreateEditorHandle;
	FDelegateHandle ComponentMaterialTrackCreateEditorHandle;
	FDelegateHandle FadeTrackCreateEditorHandle;
	FDelegateHandle SpawnTrackCreateEditorHandle;
	FDelegateHandle LevelVisibilityTrackCreateEditorHandle;
	FDelegateHandle DataLayerTrackCreateEditorHandle;
	FDelegateHandle CameraShakeTrackCreateEditorHandle;
	FDelegateHandle MPCTrackCreateEditorHandle;
	FDelegateHandle PrimitiveMaterialCreateEditorHandle;
	FDelegateHandle CameraShakeSourceShakeCreateEditorHandle;
	FDelegateHandle CVarTrackCreateEditorHandle;
	FDelegateHandle CustomPrimitiveDataTrackCreateEditorHandle;
	FDelegateHandle BindingLifetimeTrackCreateEditorHandle;
	FDelegateHandle TimeWarpTrackCreateEditorHandle;
	FDelegateHandle RegisteredPropertyTrackCreateEditorHandle;

	FDelegateHandle CameraCutTrackModelHandle;
	FDelegateHandle CinematicShotTrackModelHandle;
	FDelegateHandle BindingLifetimeTrackModelHandle;
	FDelegateHandle TimeWarpTrackModelHandle;

	FDelegateHandle GenerateEventEntryPointsHandle;
	FDelegateHandle FixupDynamicBindingPayloadParameterNameHandle;
	FDelegateHandle FixupEventSectionPayloadParameterNameHandle;
	FDelegateHandle UpgradeLegacyEventEndpointHandle;
	FDelegateHandle FixupDynamicBindingsHandle;
	FDelegateHandle FixupDirectorBlueprintConditionPayloadParameterNameHandle;

	FDelegateHandle OnObjectsReplacedHandle;

	TArray<IMovieSceneToolsTakeData*> TakeDatas;
	TArray<IMovieSceneToolsTrackImporter*> TrackImporters;

	TArray<IMovieSceneToolsAnimationBakeHelper*> BakeHelpers;
	TArray<IMovieSceneToolsKeyStructInstancedPropertyTypeCustomizer*> KeyStructInstancedPropertyTypeCustomizers;

	FSkeletalAnimationRootMotionOffsetProvider SkeletalAnimRootMotionProvider;
};

#undef UE_API
