// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneToolsProjectSettings.generated.h"

#define UE_API MOVIESCENETOOLS_API

UENUM()
enum class EMovieSceneToolsPropertyTrackType
{
	FloatTrack,
	DoubleTrack
};

USTRUCT()
struct FMovieSceneToolsPropertyTrackSettings
{
	GENERATED_BODY()

	/** Optional ActorComponent tag (when keying a component property). */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString ComponentName;

	/** Name to the keyed property within the Actor or ActorComponent. */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString PropertyName;
};

USTRUCT()
struct FMovieSceneToolsFbxSettings
{
	GENERATED_BODY()

    /** The name of the fbx property */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	FString FbxPropertyName;

	/** The property track setting to map to */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	FMovieSceneToolsPropertyTrackSettings PropertyPath;

	/** Type of property track to map to */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	EMovieSceneToolsPropertyTrackType PropertyType = EMovieSceneToolsPropertyTrackType::FloatTrack;
};

// Settings for the level sequences
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UMovieSceneToolsProjectSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovieSceneToolsProjectSettings();

	/** The default start time for new level sequences, in seconds. */
	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(Units=s))
	float DefaultStartTime;

	/** The default duration for new level sequences in seconds. */
	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ClampMin=0.00001f, Units=s))
	float DefaultDuration;

	/** The default directory for the subsequences. */
	UPROPERTY(config, EditAnywhere, Category=Shots)
	FString SubsequenceDirectory;

	/** The default directory for the shots. */
	UPROPERTY(config, EditAnywhere, Category=Shots)
	FString ShotDirectory;

	/** The default prefix for subsequence names. */
	UPROPERTY(config, EditAnywhere, Category=Shots)
	FString SubsequencePrefix;

	/** The default prefix for shot names. */
	UPROPERTY(config, EditAnywhere, Category=Shots)
	FString ShotPrefix;

	/** The first shot number. */
	UPROPERTY(config, EditAnywhere, Category=Shots, meta = (UIMin = "1", UIMax = "100"))
	uint32 FirstShotNumber;

	/** The default shot increment. */
	UPROPERTY(config, EditAnywhere, Category=Shots, meta = (UIMin = "1", UIMax = "100"))
	uint32 ShotIncrement;

	/** The number of digits for the shot number. */
	UPROPERTY(config, EditAnywhere, Category=Shots, meta = (UIMin = "1", UIMax = "10"))
	uint32 ShotNumDigits;

	/** The number of digits for the take number. */
	UPROPERTY(config, EditAnywhere, Category=Shots, meta = (UIMin = "1", UIMax = "10"))
	uint32 TakeNumDigits;

	/** The first take number. */
	UPROPERTY(config, EditAnywhere, Category=Shots, meta = (UIMin = "1", UIMax = "10"))
	uint32 FirstTakeNumber;

	/** Character separator between the shot number and the take number. */
	UPROPERTY(config, EditAnywhere, Category=Shots)
	FString TakeSeparator;

	/** Mapping between fbx property name and property track path */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	TArray<FMovieSceneToolsFbxSettings> FbxSettings;
};

#undef UE_API
