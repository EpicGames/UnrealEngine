// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneToolHelpers.h"
#include "SceneTypes.h"
#include "Exporters/FbxExportOption.h"

#include "MovieSceneToolsUserSettings.generated.h"

#define UE_API MOVIESCENETOOLS_API

UENUM()
enum class EThumbnailQuality : uint8
{
	Draft,
	Normal,
	Best,
};

UCLASS(MinimalAPI, config=EditorSettings)
class UMovieSceneUserThumbnailSettings : public UObject
{
public:
	UE_API UMovieSceneUserThumbnailSettings(const FObjectInitializer& Initializer);
	
	GENERATED_BODY()

	/** Whether to draw thumbnails or not */
	UPROPERTY(EditAnywhere, config, Category=General)
	bool bDrawThumbnails;

	/** Whether to draw a single thumbnail for this section or as many as can fit */
	UPROPERTY(EditAnywhere, config, Category=General, meta=(EditCondition=bDrawThumbnails))
	bool bDrawSingleThumbnails;

	/** Size at which to draw thumbnails on thumbnail sections */
	UPROPERTY(EditAnywhere, config, Category=General, meta=(ClampMin=1, ClampMax=1024, EditCondition=bDrawThumbnails))
	FIntPoint ThumbnailSize;

	/** Quality to render the thumbnails with */
	UPROPERTY(EditAnywhere, config, Category=General, meta=(EditCondition=bDrawThumbnails))
	EThumbnailQuality Quality;

	/** Temporal history for the view required for advanced features(e.g., eye adaptation) on all thumbnails*/
	FSceneViewStateReference ViewState;

	DECLARE_EVENT(UMovieSceneUserThumbnailSettings, FOnForceRedraw)
	FOnForceRedraw& OnForceRedraw() { return OnForceRedrawEvent; }
	void BroadcastRedrawThumbnails() const { OnForceRedrawEvent.Broadcast(); }

	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& InCollector);

private:
	FOnForceRedraw OnForceRedrawEvent;
};

UCLASS(MinimalAPI, config=EditorSettings, BlueprintType)
class UMovieSceneUserImportFBXSettings : public UObject
{
public:
	UE_API UMovieSceneUserImportFBXSettings(const FObjectInitializer& Initializer);
	
	GENERATED_BODY()

	/** Whether to match fbx node names to sequencer node names. */
	UPROPERTY(EditAnywhere, config, Category=Import, meta= (ToolTip = "Match fbx node names to sequencer node names"))
	bool bMatchByNameOnly;

	/** Whether to force the front axis to be align with X instead of -Y. */
	UPROPERTY(EditAnywhere, config, Category=Import, meta= (ToolTip = "Convert the scene from FBX coordinate system to UE coordinate system with front X axis instead of -Y"))
	bool bForceFrontXAxis;

	/** Convert the scene from FBX unit to UE unit(centimeter)*/
	UPROPERTY(EditAnywhere, config, Category = Import, meta = (ToolTip = "Convert the scene from FBX unit to UE unit(centimeter)"))
	bool bConvertSceneUnit;

	/** Import Uniform Scale*/
	UPROPERTY(EditAnywhere, config, Category = Import, meta = (ToolTip = "Import Uniform Scale"))
	float ImportUniformScale;

	/** Whether to create cameras if they don't already exist in the level. */
	UPROPERTY(EditAnywhere, config, Category=Import)
	bool bCreateCameras;

	/** Whether to replace the existing transform track or create a new track/section */
	UPROPERTY(EditAnywhere, config, Category=Import)
	bool bReplaceTransformTrack;

	/** When enabled, the current transform origin of the focused sequence will be removed from any imported transforms.
	 *  If the FBX was exported with a transform origin, this will prevent it from being applied twice.
	 */
	UPROPERTY(EditAnywhere, config, Category=Import)
	bool bCorrectForTransformOrigin;
	
	/** Whether to remove keyframes within a tolerance from the imported tracks */
	UPROPERTY(EditAnywhere, config, Category=Import)
	bool bReduceKeys;

	/** The tolerance for reduce keys */
	UPROPERTY(EditAnywhere, config, Category=Import, meta=(EditCondition=bReduceKeys))
	float ReduceKeysTolerance;
};


/** Enumeration specifying the control type and channel*/
UENUM(BlueprintType)
enum class FControlRigChannelEnum : uint8
{
	/**Bool*/
	Bool,
	/**Enum*/
	Enum,
	/**Integer*/
	Integer,
	/**Float*/
	Float,
	/**Vector2D.X*/
	Vector2DX,
	/**Vector2D.Y*/
	Vector2DY,
	/**Position.X*/
	PositionX,
	/**Position.Y*/
	PositionY,
	/**Position.Z*/
	PositionZ,
	/**Rotator.X*/
	RotatorX,
	/**Rotator.Y*/
	RotatorY,
	/**Rotator.Z*/
	RotatorZ,
	/**Scale.X*/
	ScaleX,
	/**Scale.Y*/
	ScaleY,
	/**Scale.Z*/
	ScaleZ,
};

/** Enumeration specifying the transfrom channel  */
UENUM(BlueprintType)
enum class FTransformChannelEnum : uint8
{
	TranslateX,
	TranslateY,
	TranslateZ,
	RotateX,
	RotateY,
	RotateZ,
	ScaleX,
	ScaleY,
	ScaleZ,
};

USTRUCT(BlueprintType)
struct FControlToTransformMappings
{
	GENERATED_BODY()

	FControlToTransformMappings() :  ControlChannel(FControlRigChannelEnum::Float), FBXChannel(FTransformChannelEnum::TranslateX), bNegate(false)
	{}

	/** The channel of the control to map */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Control To Transform Mappings")
	FControlRigChannelEnum ControlChannel;

	/** The channel of the fbx transofrm node to map  */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Controle To Transform Mappings")
	FTransformChannelEnum FBXChannel;

	/** Whether to negate the value */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Control To Transform Mappings")
	bool bNegate;
};

USTRUCT(BlueprintType)
struct FControlFindReplaceString
{
	GENERATED_BODY()

	FControlFindReplaceString()
	{}

	/** The string to find in the imported data*/
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Imported File Information")
	FString Find;

	/** The string to replace in the imported data*/
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Imported File Information")
	FString Replace;
};

UCLASS(MinimalAPI, BlueprintType,config = EditorSettings)
class UMovieSceneUserImportFBXControlRigSettings : public UObject
{
public:
	UE_API UMovieSceneUserImportFBXControlRigSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()
	/** Imported File Name */
	UPROPERTY(VisibleAnywhere, Category = "Imported File Information")
	FString ImportedFileName;

	/** Imported File Duration in Seconds */
	UPROPERTY(VisibleAnywhere, Category = "Imported File Information")
	FFrameNumber ImportedStartTime;

	/** Imported File  */
	UPROPERTY(VisibleAnywhere, Category = "Imported File Information")
	FFrameNumber ImportedEndTime;

	/** List Of Imported Names in FBX File*/
	UPROPERTY(VisibleAnywhere,Category = "Imported File Information")
	TArray<FString> ImportedNodeNames;

	/** Incoming File Frame Rate*/
	UPROPERTY(VisibleAnywhere, Category = "Imported File Information")
	FString ImportedFrameRate;

	/** Strings To Find in the Import and Replace With */
	UPROPERTY(EditAnywhere, config, Category = "String Matching Options", meta = (ToolTip = "Strings In Imported Node To Find And Replace"))
	TArray< FControlFindReplaceString> FindAndReplaceStrings;

	/** Strip namespaces from FBX node names */
	UPROPERTY(EditAnywhere, config, Category = "Import Options", meta = (ToolTip = "Will strip any namespace from the FBX node names"))
	bool bStripNamespace = true;
	
	/** Whether to force the front axis to be align with X instead of -Y. */
	UPROPERTY(EditAnywhere, config, Category = "Import Options", meta = (ToolTip = "Convert the scene from FBX coordinate system to UE coordinate system with front X axis instead of -Y"))
	bool bForceFrontXAxis;

	/** Convert the scene from FBX unit to UE unit(centimeter)*/
	UPROPERTY(EditAnywhere, config, Category = Import, meta = (ToolTip = "Convert the scene from FBX unit to UE unit(centimeter)"))
	bool bConvertSceneUnit;

	/** Import Uniform Scale*/
	UPROPERTY(EditAnywhere, config, Category = Import, meta = (ToolTip = "Import Uniform Scale"))
	float ImportUniformScale;

	/** Whether or not import onto selected controls or all controls*/
	UPROPERTY(EditAnywhere, config, Category = "Import Options")
	bool bImportOntoSelectedControls;

	/** Time that we insert or replace the imported animation*/
	UPROPERTY(EditAnywhere, config, Category = "Import Options")
	FFrameNumber TimeToInsertOrReplaceAnimation;

	/** Whether or not we insert or replace, by default we insert*/
	UPROPERTY(EditAnywhere, config, Category = "Import Options")
	bool bInsertAnimation;

	/** Whether to import over specific Time Range*/
	UPROPERTY(EditAnywhere, config, Category = "Import Options")
	bool bSpecifyTimeRange;

	/**Start Time Range To Import*/
	UPROPERTY(EditAnywhere, config, Category = "Import Options", meta = (EditCondition = bSpecifyTimeRange))
	FFrameNumber StartTimeRange;

	/**End Time Range To Import */
	UPROPERTY(EditAnywhere, config, Category = "Import Options", meta = (EditCondition = bSpecifyTimeRange))
	FFrameNumber EndTimeRange;

	/** Mappings for how Control Rig Control Attributes Map to the incoming Transforms*/
	UPROPERTY(Config, EditAnywhere, Category = "Control Attribute Mappings")
	TArray<FControlToTransformMappings> ControlChannelMappings;

public:
	/** Load the default or metahuman preset into the current mappings */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	UE_API void LoadControlMappingsFromPreset(bool bMetaHumanPreset);
};

UCLASS(MinimalAPI, BlueprintType, config = EditorSettings)
class UMovieSceneUserExportFBXControlRigSettings : public UObject
{
public:
	UE_API UMovieSceneUserExportFBXControlRigSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()
	/** Imported File Name */
	UPROPERTY(EditAnywhere, Category = "Imported File Information")
	FString ExportFileName;
	
	/** This will set the fbx sdk compatibility when exporting to fbx file. The default value is 2013 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Exporter)
	EFbxExportCompatibility FbxExportCompatibility = EFbxExportCompatibility::FBX_2018;

	/** If enabled, save as ascii instead of binary */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Exporter)
	uint32 bASCII : 1;

	/** Whether to force the front axis to be align with X instead of -Y. */
	UPROPERTY(EditAnywhere, config, Category = Exporter, meta = (ToolTip = "Convert the scene from FBX coordinate system to UE coordinate system with front X axis instead of -Y"))
	bool bForceFrontXAxis = false;

	/** Whether or not import onto selected controls or all controls*/
	UPROPERTY(EditAnywhere, config, Category = "Control Rig")
	bool bExportOnlySelectedControls = false;

	/** Mappings for how Control Rig Control Attributes Map to the incoming Transforms*/
	UPROPERTY(Config, EditAnywhere, Category = "Control Rig")
	TArray<FControlToTransformMappings> ControlChannelMappings;

	/** If enabled, export sequencer animation in its local time, relative to its sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Animation)
	uint32 bExportLocalTime : 1;

public:
	/** Load the default or metahuman preset into the current mappings */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	UE_API void LoadControlMappingsFromPreset(bool bMetaHumanPreset);
};

#undef UE_API
