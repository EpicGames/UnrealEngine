// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Logging/TokenizedMessage.h"
#include "UObject/MetaData.h"
#include "Sound/SoundBase.h"
#include "MovieSceneExportMetadata.h"

#define UE_API MOVIESCENETOOLS_API

/**
* MovieSceneTranslator context class.
*/
class FMovieSceneTranslatorContext : public TSharedFromThis<FMovieSceneTranslatorContext>
{
public:
	/** Constructor */
	FMovieSceneTranslatorContext() {}
	/** Destructor */
	virtual ~FMovieSceneTranslatorContext() = default;

	/** Initialize the context */
	UE_API void Init();

	/** Add message. */
	UE_API void AddMessage(EMessageSeverity::Type MessageSeverity, FText ErrorMessage);

	/** Reset all messages. */
	UE_API void ClearMessages();

	/** Returns true if specified type of message */
	UE_API bool ContainsMessageType(EMessageSeverity::Type InSeverity) const;

	/** Get error messages. */
	UE_API const TArray<TSharedRef<FTokenizedMessage>>& GetMessages() const;

private:

	/** Error messages **/
	TArray<TSharedRef<FTokenizedMessage>> Messages;
};

struct FMovieSceneExportSectionData
{
	const UMovieSceneSection* MovieSceneSection;
	int32 RowIndex;

	FString DisplayName;
	FString SourceFilename;
	FString SourceFilePath;

	FFrameNumber StartFrame;
	FFrameNumber EndFrame;
	bool bWithinPlaybackRange;
	bool bEnabled;
};

struct FMovieSceneExportCinematicSectionData : public FMovieSceneExportSectionData
{
};

struct FMovieSceneExportAudioSectionData : public FMovieSceneExportSectionData
{
	int32 NumChannels;
	int32 Depth;
	int32 SampleRate;
};

struct FMovieSceneExportCinematicTrackData
{
	/** This indicates the sub-track's row index in the track */
	int32 RowIndex;
	TArray< TSharedPtr<FMovieSceneExportCinematicSectionData> > CinematicSections;
};

struct FMovieSceneExportAudioTrackData
{
	int32 SampleRate;

	/** This indicates the sub-track's row index in the track */
	int32 RowIndex;
	TArray< TSharedPtr<FMovieSceneExportAudioSectionData> > AudioSections;
};

struct FMovieSceneExportTrackData
{
	const UMovieSceneTrack* MovieSceneTrack;
};

struct FMovieSceneExportAudioData : public FMovieSceneExportTrackData
{
	int32 SampleRate;

	/** Array of all sections in order they appear in UMovieSceneAudioTrack*/
	TArray< TSharedPtr<FMovieSceneExportAudioSectionData> > AudioSections;

	/** Array of sorted audio sub tracks, containing pointers to sections within the sub track row*/
	TArray< TSharedPtr<FMovieSceneExportAudioTrackData> > AudioTracks;

};

struct FMovieSceneExportCinematicData : public FMovieSceneExportTrackData
{
	/** Array of all sections in order they appear in UMovieSceneCinematicTrack*/
	TArray< TSharedPtr<FMovieSceneExportCinematicSectionData> > CinematicSections;

	/** Array of sorted movie sub tracks, containing pointers to sections within the sub track row */
	TArray< TSharedPtr<FMovieSceneExportCinematicTrackData> > CinematicTracks;
};

struct FMovieSceneExportMovieSceneData
{
	FString Name;
	FString Path;
	FFrameRate TickResolution;
	int32 Duration;
	FFrameNumber PlaybackRangeStartFrame;
	FFrameNumber PlaybackRangeEndFrame;
	TSharedPtr<FMovieSceneExportCinematicData> CinematicData;
	TArray< TSharedPtr<FMovieSceneExportAudioData> > AudioData;
	FString MovieExtension;
};

enum class EMovieSceneTranslatorSectionType : int32
{
	Cinematic = 0,
	Audio = 1
};

/** 
 * The FMovieSceneExportData class aggregates intermediate data from Sequencer classes to be used for timeline exports 
 */
class FMovieSceneExportData : public TSharedFromThis<FMovieSceneExportData>
{
public:
	/** Constructor */
	UE_API FMovieSceneExportData(const UMovieScene* InMovieScene, FFrameRate InFrameRate, uint32 InResX, uint32 InResY, int32 InHandleFrames, FString InSaveFilename, TSharedPtr<FMovieSceneTranslatorContext> InContext, FString InMovieExtension);
	/** Default constructor, necessary for shared ref - should not be used */
	UE_API FMovieSceneExportData();
	/** Destructor */
	UE_API ~FMovieSceneExportData();

	/** Gets export filename */
	UE_API FString GetFilename() const;
	/** Gets export filename with full path */
	UE_API FString GetFilenamePath() const;
	/** Gets the shot movie extension */
	UE_API FString GetMovieExtension() const;

	/** Gets export frame rate */
	UE_API FFrameRate GetFrameRate() const;
	/** Gets x resolution */
	UE_API uint32 GetResX() const;
	/** Gets y resolution */
	UE_API uint32 GetResY() const;
	/** Returns true if frame rate is a non-integral frame rate */
	UE_API bool GetFrameRateIsNTSC() const;
	/** Returns the nearest integral frame rate */
	UE_API uint32 GetNearestWholeFrameRate() const;

	/** Gets the frame handle */
	UE_API int32 GetHandleFrames() const;
	/** Gets default audio sample rate */
	UE_API int32 GetDefaultAudioSampleRate() const;
	/** Gets default audio depth */
	UE_API int32 GetDefaultAudioDepth() const;

	/** True when the export data was successfully constructed. */
	UE_API bool IsExportDataValid() const;

	/** Find audio sections */
	UE_API bool FindAudioSections(const FString& InSoundPathName, TArray<TSharedPtr<FMovieSceneExportAudioSectionData>>& OutFoundSections) const;

private:

	/** Entry point for creating the intermediate data to use when exporting. */
	UE_API bool ConstructData(const UMovieScene* InMovieScene);
	/** Loads intermediate movie scene data from Sequencer. */
	UE_API bool ConstructMovieSceneData(const UMovieScene* InMovieScene);
	/** Loads intermediate cinematic data from Sequencer. */
	UE_API bool ConstructCinematicData(const UMovieScene* InMovieScene, const UMovieSceneCinematicShotTrack* InCinematicTrack);
	/** Loads intermediate cinematic data from Sequencer. */
	UE_API bool ConstructCinematicTrackData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, int32 InRowIndex);
	/** Loads intermediate audio data from Sequencer. */
	UE_API bool ConstructAudioData(const UMovieScene* InMovieScene, const UMovieSceneAudioTrack* InAudioTrack, TMap<int32, TSharedPtr<FMovieSceneExportAudioData>>& InAudioDataMap);
	/** Loads intermediate audio data from Sequencer. */
	UE_API bool ConstructAudioData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportAudioData> InAudioData, int32 InRowIndex);
	/** Loads intermediate cinematic section data from Sequencer. */
	UE_API bool ConstructCinematicSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportCinematicData> InCinematicData, const UMovieSceneCinematicShotSection* InCinematicSection);
	/** Loads intermediate audio section data from Sequencer. */
	UE_API bool ConstructAudioSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportAudioData> InAudioData, const UMovieSceneAudioSection* InAudioSection);
	/** Loads intermediate common section data from Sequencer. */
	UE_API bool ConstructSectionData(const UMovieScene* InMovieScene, TSharedPtr<FMovieSceneExportSectionData> InSectionData, const UMovieSceneSection* InSection, EMovieSceneTranslatorSectionType InSectionType, const FString& InSectionDisplayName);

	/** Context for messages */
	TSharedPtr<FMovieSceneTranslatorContext> ExportContext;

public:

	/** Intermediate data loaded from Sequencer to be used for export */
	TSharedPtr<FMovieSceneExportMovieSceneData> MovieSceneData;

private:

	FFrameRate FrameRate;
	uint32 ResX;
	uint32 ResY;
	int32 HandleFrames;
	FString SaveFilename;
	FString SaveFilenamePath;
	bool bExportDataIsValid;
	int32 DefaultAudioSampleRate;
	int32 DefaultAudioDepth;
	FString MovieExtension;

	TMap<FString, int32> DuplicateShotNameCounts;
};

struct FMovieSceneImportCinematicSectionData 
{
	UMovieSceneCinematicShotSection* CinematicSection;
};

struct FMovieSceneImportAudioSectionData
{
	UMovieSceneAudioSection* AudioSection;
	FString SourceFilename;
	FString SourceFilePath;
};

struct FMovieSceneImportCinematicTrackData
{
	int32 RowIndex;
	TArray< TSharedPtr<FMovieSceneImportCinematicSectionData> > CinematicSections;
};

struct FMovieSceneImportAudioTrackData
{
	int32 RowIndex;
	TArray< TSharedPtr<FMovieSceneImportAudioSectionData> > AudioSections;
};

struct FMovieSceneImportTrackData
{
	UMovieSceneTrack* MovieSceneTrack;
};
        
struct FMovieSceneImportAudioData : public FMovieSceneImportTrackData
{
	/** Array of all sections in order they appear in UMovieSceneAudioTrack*/
	TArray< TSharedPtr<FMovieSceneImportAudioSectionData> > AudioSections;

	/** Array of sorted audio sub tracks, containing pointers to sections within the sub track row*/
	TArray< TSharedPtr<FMovieSceneImportAudioTrackData> > AudioTracks;

	/** Max row index existing in this track */
	int32 MaxRowIndex;
};

struct FMovieSceneImportCinematicData : public FMovieSceneImportTrackData
{
	/** Array of all sections in order they appear in UMovieSceneCinematicTrack*/
	TArray< TSharedPtr<FMovieSceneImportCinematicSectionData> > CinematicSections;

	/** Array of sorted movie sub tracks, containing pointers to sections within the sub track row */
	TArray< TSharedPtr<FMovieSceneImportCinematicTrackData> > CinematicTracks;
};

struct FMovieSceneImportMovieSceneData
{
	UMovieScene* MovieScene;
	TSharedPtr<FMovieSceneImportCinematicData> CinematicData;
	TArray< TSharedPtr<FMovieSceneImportAudioData> > AudioData;
};

/**
* The FMovieSceneImportData class aggregates intermediate data from Sequencer classes to be used for timeline imports
*/
class FMovieSceneImportData : public TSharedFromThis<FMovieSceneImportData>
{
public:
	/** Constructor */
	FMovieSceneImportData(UMovieScene* InMovieScene, TSharedPtr<FMovieSceneTranslatorContext> InContext);
	/** Default constructor, necessary for shared ref - should not be used */
	FMovieSceneImportData();
	/** Destructor */
	~FMovieSceneImportData();

	/** True when the export data was successfully constructed. */
	bool IsImportDataValid() const;

	/** Returns the cinematic data pointer or nullptr if one does not exist */
	TSharedPtr<FMovieSceneImportCinematicData> GetCinematicData(bool CreateTrackIfNull);
	/** Find cinematic section */
	TSharedPtr<FMovieSceneImportCinematicSectionData> FindCinematicSection(const FString& InSectionPathName);
	/** Create cinematic section */
	TSharedPtr<FMovieSceneImportCinematicSectionData> CreateCinematicSection(FString InName, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame);
	/** Set cinematic section */
	bool SetCinematicSection(TSharedPtr<FMovieSceneImportCinematicSectionData> InSection, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, TOptional<FFrameNumber> InStartOffsetFrame);

	/** Returns the audio data pointer or nullptr if one does not exist */
	TSharedPtr<FMovieSceneImportAudioData> GetAudioData();
	/** Find audio sections */
	TSharedPtr<FMovieSceneImportAudioSectionData>  FindAudioSection(const FString& InSectionPathName, TSharedPtr<FMovieSceneImportAudioData>& OutAudioData);
	/** Create audio section */
	TSharedPtr<FMovieSceneImportAudioSectionData> CreateAudioSection(FString InFilenameOrAssetPathName, bool bIsPathName, TSharedPtr<FMovieSceneImportAudioData> InAudioData, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame);
	/** Set audio section */
	bool SetAudioSection(TSharedPtr<FMovieSceneImportAudioSectionData> InSection, int32 InRow, FFrameRate InFrameRate, FFrameNumber InStartFrame, FFrameNumber InEndFrame, FFrameNumber InStartOffsetFrame);
	/** Move audio section */
	bool MoveAudioSection(TSharedPtr<FMovieSceneImportAudioSectionData> InAudioSectionData, TSharedPtr<FMovieSceneImportAudioData> InFromAudioData, TSharedPtr<FMovieSceneImportAudioData> InToAudioData, int32 InToRowIndex);

private:

	/** Entry point for setting up intermediate data for use when importing. */
	TSharedPtr<FMovieSceneImportMovieSceneData> ConstructMovieSceneData(UMovieScene* InMovieScene);
	/** Gets cinematic data from Sequencer. */
	TSharedPtr<FMovieSceneImportCinematicData> ConstructCinematicData(UMovieSceneCinematicShotTrack* InCinematicTrack);
	/** Gets cinematic track data from Sequencer. */
	TSharedPtr<FMovieSceneImportCinematicTrackData> ConstructCinematicTrackData(UMovieSceneCinematicShotTrack* InCinematicTrack, int32 InRowIndex);
	/** Gets audio data from Sequencer. */
	TSharedPtr<FMovieSceneImportAudioData> ConstructAudioData(UMovieSceneAudioTrack* InAudioTrack);
	/** Gets audio track data from Sequencer. */
	TSharedPtr<FMovieSceneImportAudioTrackData> ConstructAudioTrackData(UMovieSceneAudioTrack* InAudioTrack, int32 InRowIndex);
	/** Gets cinematic section data from Sequencer. */
	TSharedPtr<FMovieSceneImportCinematicSectionData> ConstructCinematicSectionData(UMovieSceneCinematicShotSection* InCinematicSection);
	/** Gets audio section data from Sequencer. */
	TSharedPtr<FMovieSceneImportAudioSectionData>  ConstructAudioSectionData(UMovieSceneAudioSection* InAudioSection);

	/** Context for messages */
	TSharedPtr<FMovieSceneTranslatorContext> ImportContext;

public:

	/** Intermediate data loaded from Sequencer to be used for export */
	TSharedPtr<FMovieSceneImportMovieSceneData> MovieSceneData;
};

/** Abstract base class for importer/exporter */
class FMovieSceneTranslator
{
public:
	FMovieSceneTranslator() {}
	virtual ~FMovieSceneTranslator() = default;

	/** Error log window title. */
	virtual FName GetMessageLogWindowTitle() const = 0;
	/** Error log list label. */
	virtual FText GetMessageLogLabel() const = 0;
};

/** Abstract base class for movie scene importers */
class FMovieSceneImporter : public FMovieSceneTranslator
{
public:
	FMovieSceneImporter() : FMovieSceneTranslator() {}

	virtual ~FMovieSceneImporter() = default;

	/** Format description. */
	virtual FText GetFileTypeDescription() const = 0;
	/** Import window title. */
	virtual FText GetDialogTitle() const = 0;
	/** Scoped transaction description. */
	virtual FText GetTransactionDescription() const = 0;

public:
	/*
	* Import movie scene
	*
	* @param InMovieScene The movie scene to import the XML file into
	* @param InFrameRate The frame rate to import the XML at
	* @param InFilename The filename to import
	* @param OutError The return error message
	* @return Whether the import was successful
	*/
	virtual bool Import(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InFilename, TSharedRef<FMovieSceneTranslatorContext> InContext) = 0;
};

/** Abstract base class for movie scene exporters */
class FMovieSceneExporter : public FMovieSceneTranslator
{
public:
	FMovieSceneExporter() : FMovieSceneTranslator() {}

	virtual ~FMovieSceneExporter() = default;

	/** Format description. */
	virtual FText GetFileTypeDescription() const = 0;
	/** Export dialog window title. */
	virtual FText GetDialogTitle() const = 0;
	/** Default format file extension. */
	virtual FText GetDefaultFileExtension() const = 0;
	/** Notification when export completes. */
	virtual FText GetNotificationExportFinished() const = 0;
	/** Notification hyperlink to exported file path. */
	virtual FText GetNotificationHyperlinkText() const = 0;


public:
	/*
	* Export movie scene
	*
	* @param InMovieScene The movie scene with the cinematic shot track and audio tracks to export
	* @param InFilenameFormat The specified filename format.
	* @param InFrameRate The frame rate for export.
	* @param InResX Sequence resolution x.
	* @param InResY Sequence resolution y.
	* @param InHandleFrames The number of handle frames to include for each shot.
	* @param InSaveFilename The file path to save to.
	* @param OutError The return error message
	* @param MovieExtension The movie extension for the shot filenames (ie. .avi, .mov, .mp4)
	* @param InMetadata (optional) Metadata from export to override movie output file list
	* @return Whether the export was successful
	*/
	virtual bool Export(const UMovieScene* InMovieScene, FString InFilenameFormat, FFrameRate InFrameRate, uint32 InResX, uint32 InResY, int32 InHandleFrames, FString InSaveFilename, TSharedRef<FMovieSceneTranslatorContext> InContext, FString InMovieExtension, const FMovieSceneExportMetadata* InMetadata=nullptr) = 0;
};

#undef UE_API
