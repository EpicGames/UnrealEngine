// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MOVIESCENETOOLS_API

struct FFrameRate;
struct FFrameNumber;
class UMovieScene;

class MovieSceneTranslatorEDL
{
public:

	/**
	 * Import EDL
	 *
	 * @param InMovieScene The movie scene to import the edl into
	 * @param InFrameRate The frame rate to import the EDL at
	 * @param InFilename The filename to import
	 * @return Whether the import was successful
	 */
	static UE_API bool ImportEDL(UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InFilename);

	/**
	 * Export EDL
	 *
	 * @param InMovieScene The movie scene with the cinematic shot track and audio tracks to export
	 * @param InFrameRate The frame rate to export the EDL at
	 * @param InSaveFilename The file path to save to.
	 * @param InHandleFrames The number of handle frames to include for each shot.
	 * @param MovieExtension The movie extension for the shot filenames (ie. .avi, .mov, .mp4)
	 * @return Whether the export was successful
	 */
	static UE_API bool ExportEDL(const UMovieScene* InMovieScene, FFrameRate InFrameRate, FString InSaveFilename, int32 InHandleFrames, FString InMovieExtension);
};

#undef UE_API
