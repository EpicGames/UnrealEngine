// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "MovieSceneObjectBindingID.h"
#include "UObject/NameTypes.h"

class UMovieSceneSequence;


/**
 * Owned by an FSequencer instance, this class tracks tags for object bindings, and maintains a reverse lookup map from binding to tag(s)
 * along with other information for the tags such as color tints.
 */
class FObjectBindingTagCache
{
public:

	using TagIterator = TMultiMap<UE::MovieScene::FFixedObjectBindingID, FName>::TConstKeyIterator;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdated, const FObjectBindingTagCache*);

	/** Public event that is broadcast when this cache changes in some way */
	FOnUpdated OnUpdatedEvent;

	/*
	 * Conditionally update this cache based on the state of the specified root sequence
	 */
	MOVIESCENETOOLS_API void ConditionalUpdate(UMovieSceneSequence* RootSequence);


	/*
	 * Retrieve a color for the specified tag name
	 */
	FLinearColor GetTagColor(const FName& TagName) const
	{
		const FLinearColor* Color = ExposedNameColors.Find(TagName);
		return Color ? *Color : FLinearColor::White;
	}


	/*
	 * Check whether the specified binding ID has the specified tag
	 */
	bool HasTag(const UE::MovieScene::FFixedObjectBindingID& BindingID, const FName& TagName) const
	{
		return ExposedNameReverseLUT.FindPair(BindingID, TagName) != nullptr;
	}


	/*
	 * Iterate all the tags for the specified binding ID
	 */
	TagIterator IterateTags(const UE::MovieScene::FFixedObjectBindingID& BindingID) const
	{
		return ExposedNameReverseLUT.CreateConstKeyIterator(BindingID);
	}

private:

	/** Sorted map of colors by tag name */
	TSortedMap<FName, FLinearColor, FDefaultAllocator, FNameFastLess> ExposedNameColors;

	/** The signature of the root sequence last time we were updated */
	FGuid RootSequenceSignature;

	/** Multi map (1->many) of a binding ID to any tags that are associated with it */
	TMultiMap<UE::MovieScene::FFixedObjectBindingID, FName> ExposedNameReverseLUT;
};