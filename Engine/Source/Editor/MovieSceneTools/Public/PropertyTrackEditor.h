// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "KeyPropertyParams.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "UObject/Package.h"
#include "MovieSceneTrackEditor.h"
#include "KeyframeTrackEditor.h"
#include "ISequencerObjectChangeListener.h"
#include "AnimatedPropertyKey.h"
#include "Tracks/MovieScenePropertyTrack.h"

#define LOCTEXT_NAMESPACE "PropertyTrackEditor"

/**
* Tools for animatable property types such as floats ands vectors
*/
template<typename TrackType>
class FPropertyTrackEditor
	: public FKeyframeTrackEditor<TrackType>
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FPropertyTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FKeyframeTrackEditor<TrackType>( InSequencer )
	{ }

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 * @param InWatchedPropertyTypes A list of property types that this editor can animate
	 */
	FPropertyTrackEditor( TSharedRef<ISequencer> InSequencer, TArrayView<const FAnimatedPropertyKey> InWatchedPropertyTypes )
		: FKeyframeTrackEditor<TrackType>( InSequencer )
	{
		for (const FAnimatedPropertyKey& Key : InWatchedPropertyTypes)
		{
			AddWatchedProperty(Key);
		}
	}

	~FPropertyTrackEditor()
	{
		TSharedPtr<ISequencer> SequencerPtr = FMovieSceneTrackEditor::GetSequencer();
		if ( SequencerPtr.IsValid() )
		{
			ISequencerObjectChangeListener& ObjectChangeListener = SequencerPtr->GetObjectChangeListener();
			for ( FAnimatedPropertyKey PropertyKey : WatchedProperties )
			{
				ObjectChangeListener.GetOnAnimatablePropertyChanged( PropertyKey ).RemoveAll( this );
			}
		}
	}

public:

	//~ ISequencerTrackEditor interface

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("PropertyTrackEditor_DisplayName", "Property");
	}

    virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override
    {
        return true;
    }

	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override
	{
		return Type == TrackType::StaticClass();
	}

protected:

	/**
	* Generates keys based on the new value from the property property change parameters.
	*
	* @param PropertyChangedParams Parameters associated with the property change.
	* @param OutGeneratedKeys Array of keys that are generated from the changed property
	*/
	virtual void GenerateKeysFromPropertyChanged( const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys ) = 0;

	/** When true, this track editor will only be used on properties which have specified it as a custom track class. This is necessary to prevent duplicate
		property change handling in cases where a custom track editor handles the same type of data as one of the standard track editors. */
	virtual bool ForCustomizedUseOnly() { return false; }

	/** 
	 * Initialized values on a track after it's been created, but before any sections or keys have been added.
	 * @param NewTrack The newly created track.
	 * @param PropertyChangedParams The property change parameters which caused this track to be created.
	 */
	virtual void InitializeNewTrack( TrackType* NewTrack, FPropertyChangedParams PropertyChangedParams )
	{
		const FProperty* ChangedProperty = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();
		if (ChangedProperty)
		{
			NewTrack->SetPropertyNameAndPath(ChangedProperty->GetFName(), PropertyChangedParams.GetPropertyPathString());

#if WITH_EDITORONLY_DATA
			FText DisplayText;
			const FPropertyPath NamingPropertyPath = BuildNamingPropertyPath(PropertyChangedParams.PropertyPath);
			const FProperty* LeafNameProperty = NamingPropertyPath.GetLeafMostProperty().Property.Get();
			const int32 NumProperties = NamingPropertyPath.GetNumProperties();

			// Set up the appropriate name for the track from an array/nested struct index if necessary
			for (int32 PropertyIndex = NumProperties - 1; PropertyIndex >= 0; --PropertyIndex)
			{
				const FPropertyInfo& Info = NamingPropertyPath.GetPropertyInfo(PropertyIndex);
				const FArrayProperty* ParentArrayProperty = PropertyIndex > 0 ? CastField<FArrayProperty>(NamingPropertyPath.GetPropertyInfo(PropertyIndex - 1).Property.Get()) : nullptr;

				const FProperty* ArrayInnerProperty = Info.Property.Get();
				if (ArrayInnerProperty && Info.ArrayIndex != INDEX_NONE)
				{
					DisplayText = FText::Format(LOCTEXT("DisplayTextArrayFormat", "{0} ({1}[{2}])"),
						LeafNameProperty->GetDisplayNameText(),
						(ParentArrayProperty ? ParentArrayProperty : ArrayInnerProperty)->GetDisplayNameText(),
						FText::AsNumber(Info.ArrayIndex)
					);
					break;
				}
			}

			if (DisplayText.IsEmpty())
			{
				for (int32 PropertyIndex = NumProperties - 1; PropertyIndex >= 0; --PropertyIndex)
				{
					const FStructProperty* ParentStructProperty = PropertyIndex > 0 ? CastField<FStructProperty>(PropertyChangedParams.PropertyPath.GetPropertyInfo(PropertyIndex - 1).Property.Get()) : nullptr;
					if (ParentStructProperty)
					{
						DisplayText = FText::Format(LOCTEXT("DisplayTextStructFormat", "{0} ({1})"),
							LeafNameProperty->GetDisplayNameText(),
							ParentStructProperty->GetDisplayNameText()
						);
						break;
					}
				}
			}

			if (DisplayText.IsEmpty())
			{
				DisplayText = LeafNameProperty->GetDisplayNameText();
			}

			NewTrack->SetDisplayName(DisplayText);
#endif
		}
	}

	virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override
	{
		UMovieSceneTrack* Track = FocusedMovieScene->AddTrack(TrackClass, ObjectHandle); 
		return Track;
	}

protected:

	/** Adds a callback for property changes for the supplied property type name. */
	void AddWatchedProperty( FAnimatedPropertyKey PropertyKey )
	{
		FMovieSceneTrackEditor::GetSequencer()->GetObjectChangeListener().GetOnAnimatablePropertyChanged( PropertyKey ).AddRaw( this, &FPropertyTrackEditor::OnAnimatedPropertyChanged );
		WatchedProperties.Add( PropertyKey );
	}

	/**
	* Called by the details panel when an animatable property changes
	*
	* @param InObjectsThatChanged	List of objects that changed
	* @param KeyPropertyParams		Parameters for the property change.
	*/
	virtual void OnAnimatedPropertyChanged( const FPropertyChangedParams& PropertyChangedParams )
	{
		FMovieSceneTrackEditor::AnimatablePropertyChanged( FOnKeyProperty::CreateRaw( this, &FPropertyTrackEditor::OnKeyProperty, PropertyChangedParams ) );
	}

private:

	/** Adds a callback for property changes for the supplied property type name. */
	void AddWatchedPropertyType( FName WatchedPropertyTypeName )
	{
		AddWatchedProperty(FAnimatedPropertyKey::FromPropertyTypeName(WatchedPropertyTypeName));
	}

	/** 
	 * Builds a property path that omits properties that are "insignificant" for computing track names. 
	 * IMPORTANT: the result path is only for looking up names, and is non functional!
	 */
	FPropertyPath BuildNamingPropertyPath(const FPropertyPath& InPropertyPath)
	{
		FPropertyPath Result;
		for (int32 Index = 0; Index < InPropertyPath.GetNumProperties(); ++Index)
		{
			const FPropertyInfo& Info = InPropertyPath.GetPropertyInfo(Index);
			if (const FProperty* Property = Info.Property.Get())
			{
				if (Info.ArrayIndex == INDEX_NONE && Property->GetBoolMetaData(TEXT("SequencerUseParentPropertyName")))
				{
					continue;
				}
			}
			Result.AddProperty(Info);
		}
		return Result;
	}

	/** Get a customized track class from the property if there is one, otherwise return nullptr. */
	TSubclassOf<UMovieSceneTrack> GetCustomizedTrackClass( const FProperty* Property )
	{
		// Look for a customized track class for this property on the meta data
		const FString& MetaSequencerTrackClass = Property->GetMetaData( TEXT( "SequencerTrackClass" ) );
		if ( !MetaSequencerTrackClass.IsEmpty() )
		{
			UClass* MetaClass = UClass::TryFindTypeSlow<UClass>(MetaSequencerTrackClass);
			if ( !MetaClass )
			{
				MetaClass = LoadObject<UClass>( nullptr, *MetaSequencerTrackClass );
			}
			return MetaClass;
		}
		return nullptr;
	}

	/** Adds a key based on a property change. */
	FKeyPropertyResult OnKeyProperty( FFrameNumber KeyTime, FPropertyChangedParams PropertyChangedParams )
	{
		FKeyPropertyResult KeyPropertyResult;

		const FProperty* Property = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();
		if (!Property)
		{
			return KeyPropertyResult;
		}

		TSubclassOf<UMovieSceneTrack> CustomizedClass = GetCustomizedTrackClass( Property );
		TSubclassOf<UMovieSceneTrack> TrackClass;
		
		if (CustomizedClass != nullptr)
		{
			TrackClass = CustomizedClass;
		}
		else
		{
			TrackClass = TrackType::StaticClass();
		}

		FName UniqueName(*PropertyChangedParams.PropertyPath.ToString(TEXT(".")));

		// If the track class has been customized for this property then it's possible this track editor doesn't support it, 
		// also check for track editors which should only be used for customization.
		if ( SupportsType( TrackClass ) && ( ForCustomizedUseOnly() == false || *CustomizedClass != nullptr) )
		{
			auto GenerateKeys = [this, PropertyChangedParams](UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys)
			{
				this->GenerateKeysFromPropertyChanged(PropertyChangedParams, Section, OutGeneratedKeys);
			};

			return this->AddKeysToObjects(
				PropertyChangedParams.ObjectsThatChanged,
				KeyTime,
				PropertyChangedParams.KeyMode,
				TrackClass,
				UniqueName,
				[&](TrackType* NewTrack) { InitializeNewTrack(NewTrack, PropertyChangedParams); },
				GenerateKeys
			);
		}
		else
		{
			return KeyPropertyResult;
		}
	}

private:

	/** An array of property type names which are being watched for changes. */
	TArray<FAnimatedPropertyKey> WatchedProperties;
};

#undef LOCTEXT_NAMESPACE
