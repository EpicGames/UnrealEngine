// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"
#include "EditorUndoClient.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MOVIESCENETOOLS_API

struct FPropertyChangedEvent;

class ISequencer;
class FStructOnScope;
class UMovieSceneSection;
class IPropertyTypeCustomization;
class IStructureDetailsView;

struct FKeyEditData
{
	TSharedPtr<FStructOnScope> KeyStruct;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
	/** Channel's actual owning UObject (e.g. a decoration). Falls back to OwningSection for section-owned channels. */
	TWeakObjectPtr<UObject> OwningObject;
};

/**
 * Widget that represents a details panel that refreshes on undo, and handles modification of the section on edit
 */
class SKeyEditInterface : public SCompoundWidget, public FEditorUndoClient, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SKeyEditInterface){}
		SLATE_ATTRIBUTE(FKeyEditData, EditData)
	SLATE_END_ARGS()

	UE_API ~SKeyEditInterface();

	UE_API void Construct(const FArguments& InArgs, TSharedRef<ISequencer> InSequencer);

	/**
	 * (Re)Initialize this widget's details panel
	 */
	UE_API void Initialize();

private:

	/**
	 * Create a binding ID customization for the details panel
	 */
	UE_API TSharedRef<IPropertyTypeCustomization> CreateBindingIDCustomization();
	UE_API TSharedRef<IPropertyTypeCustomization> CreateFrameNumberCustomization();
	UE_API TSharedRef<IPropertyTypeCustomization> CreateEventCustomization();

	/**
	 * Called when a property has been changed on the UI
	 */
	UE_API void OnFinishedChangingProperties(const FPropertyChangedEvent& ChangeEvent, TSharedPtr<FStructOnScope> KeyStruct);

private:

	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	void ModifySectionAndOwner();

	UE_API virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;

private:
	TAttribute<FKeyEditData> EditDataAttribute;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TWeakObjectPtr<UObject> WeakOwningObject;
	TWeakPtr<ISequencer> WeakSequencer;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
};

#undef UE_API
