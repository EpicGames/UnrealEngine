// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "MovieSceneObjectBindingID.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FObjectBindingTagCache;
class FText;
class SEditableTextBox;

/** A horizontally grouped collection of tags that apply to a single object binding. This widget automatically updates when the tags for a binding change. */
class SObjectBindingTags : public SCompoundWidget
{
public:

	/** A simple delegate that is passed the tag name applicable to the operation */
	DECLARE_DELEGATE_OneParam(FTagEvent, FName)

	SLATE_BEGIN_ARGS(SObjectBindingTags){}

		/** (Optional) When bound, will show a delete button on the tags, and call this delegate when clicked */
		SLATE_EVENT(FTagEvent, OnTagDeleted)

	SLATE_END_ARGS()

	/** Construct this widget. Binding cache can be retrieved through FSequencer::GetObjectTagCache()*/
	MOVIESCENETOOLS_API void Construct(const FArguments& InArgs, const UE::MovieScene::FFixedObjectBindingID& InBindingID, FObjectBindingTagCache* BindingCache);

private:

	/** Called when the binding tag cache that holds information about the binding's tags changes */
	void OnBindingCacheUpdated(const FObjectBindingTagCache* BindingCache);

	/** Called when a tag is to be deleted */
	void OnTagDeleted(FName TagName);

private:

	FTagEvent OnTagDeletedEvent;
	UE::MovieScene::FFixedObjectBindingID BindingID;
};

/** A single named tag widget for an object binding within a sequence, represented as a rounded 'pill' */
class SObjectBindingTag : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnCreateNew, FName)

	SLATE_BEGIN_ARGS(SObjectBindingTag) : _ColorTint(FLinearColor::White) {}

		/** (Optional) When bound, shows a small delete button on the tag that invokes this delegate when clicked */
		SLATE_EVENT(FSimpleDelegate, OnDeleted)

		/** (Optional) When bound, the text portion of the tag will be a text input allowing the user to create a new named tag */
		SLATE_EVENT(FOnCreateNew, OnCreateNew)

		/** (Optional) When bound, the whole tag will be clickable, and invoke this delegate when clicked */
		SLATE_EVENT(FSimpleDelegate, OnClicked)

		/** Text to display on the tag (when OnCreateNew is not specified) */
		SLATE_ATTRIBUTE(FText, Text)

		/** Tool tip text for this whole widget */
		SLATE_ATTRIBUTE(FText, ToolTipText)

		/** Color tint for the whole widget */
		SLATE_ATTRIBUTE(FSlateColor, ColorTint)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 */
	MOVIESCENETOOLS_API void Construct(const FArguments& InArgs);

private:

	void OnNewTextCommitted(const FText& InNewText, ETextCommit::Type CommitType);

	FReply HandleCreateButtonClicked();

	FReply HandleDeleteButtonClicked();

	FReply HandlePillClicked();

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		const float DesiredX = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier).X;
		return FVector2D(DesiredX, 24.f);
	}

private:
	FOnCreateNew OnCreateNew;
	FSimpleDelegate OnDeleted;
	FSimpleDelegate OnClicked;

	TSharedPtr<SEditableTextBox> EditableText;
};