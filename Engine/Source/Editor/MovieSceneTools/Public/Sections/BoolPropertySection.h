// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ISequencerSection.h"

class FSequencerSectionPainter;
class UMovieSceneSection;

/**
* An implementation of bool sections.
*/
class FBoolPropertySection
	: public FSequencerSection
{
public:

	/**
	* Creates a new bool section for editing bool sections.
	*
	* @param InSectionObject The section object which is being displayed and edited.
	* @param InDisplayName A display name for the section being displayed and edited.
	*/
	FBoolPropertySection(UMovieSceneSection& InSectionObject)
		: FSequencerSection(InSectionObject)
	{}

public:

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
};
