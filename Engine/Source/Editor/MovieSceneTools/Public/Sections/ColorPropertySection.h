// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "ISequencerSection.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneCommonHelpers.h"
#include "Templates/SharedPointer.h"

class FSequencerSectionPainter;
class ISequencer;
class UMovieSceneSection;
struct FKeyHandle;

/**
* A color section implementation
*/
class FColorPropertySection
	: public FSequencerSection
{
public:

	/**
	* Creates a new color property section.
	*
	* @param InSectionObject The section object which is being displayed and edited.
	* @param InObjectBindingID The ID of the object this section is bound to
	* @param InSequencer The sequencer this section is for
	*/
	FColorPropertySection(UMovieSceneSection& InSectionObject, const FGuid& InObjectBindingID, TWeakPtr<ISequencer> InSequencer);

private:

	virtual FReply OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles ) override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

	/** Get the current value of the object's property as a linear color */
	FLinearColor GetPropertyValueAsLinearColor() const;

private:

	/** The bound objects ID */
	FGuid ObjectBindingID;

	/** Weak pointer to the sequencer this section is for */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Property bindings used for retrieving object properties */
	mutable TOptional<FTrackInstancePropertyBindings> PropertyBindings;
};
