// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/SharedPointerFwd.h"

class ISequencer;
class FSequencerSectionPainter;
class UMovieSceneSection;

/**
 * Helper class for event section interfaces.
 */
class FEventSectionHelper
{
public:

	/** Paints an event name. */
	MOVIESCENETOOLS_API static void PaintEventName(
			FSequencerSectionPainter& Painter, 
			int32 LayerId, 
			const FString& EventString, 
			float PixelPosition,
			bool bIsEventValid = true);

	/** Returns whether the given section is selected. */
	MOVIESCENETOOLS_API static bool IsSectionSelected(TSharedRef<ISequencer> Sequencer, UMovieSceneSection* InSection);
};

