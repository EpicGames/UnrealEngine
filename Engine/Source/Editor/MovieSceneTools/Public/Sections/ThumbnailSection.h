// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "TrackEditorThumbnail/TrackEditorThumbnail.h"

#define UE_API MOVIESCENETOOLS_API

class FLevelEditorViewportClient;
class FMenuBuilder;
class FSceneViewport;
class FSequencerSectionPainter;
class FTrackEditorThumbnailPool;
class ISectionLayoutBuilder;
struct FSlateBrush;

struct FThumbnailCameraSettings
{
	float AspectRatio;
};


/**
 * Thumbnail section, which paints and ticks the appropriate section.
 */
class FThumbnailSection
	: public ISequencerSection
	, public TSharedFromThis<FThumbnailSection>
{
public:

	/** Create and initialize a new instance. */
	UE_API FThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, IViewportThumbnailClient* InViewportThumbanilClient, UMovieSceneSection& InSection);
	UE_API FThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, ICustomThumbnailClient* InCustomThumbnailClient, UMovieSceneSection& InSection);

	/** Destructor. */
	UE_API ~FThumbnailSection();

public:

	/** @return The sequencer widget owning the MovieScene section. */
	TSharedRef<SWidget> GetSequencerWidget()
	{
		return SequencerPtr.Pin()->GetSequencerWidget();
	}

	/** Enter rename mode for the section */
	UE_API void EnterRename();

	/** Get whether the text is renameable */
	virtual bool CanRename() const { return false; }

	/** Callback for getting the text of the track name text block. */
	virtual FText HandleThumbnailTextBlockText() const { return FText::GetEmpty(); }

	/** Callback for when the text of the track name text block has changed. */
	virtual void HandleThumbnailTextBlockTextCommitted(const FText& NewThumbnailName, ETextCommit::Type CommitType) { }

public:

	/** Set this thumbnail section to draw a single thumbnail at the specified time */
	virtual void SetSingleTime(double GlobalTime) = 0;

public:

	//~ ISequencerSection interface

	UE_API virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	UE_API virtual void BuildSectionSidebarMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;
	UE_API virtual TSharedRef<SWidget> GenerateSectionWidget() override;
	UE_API virtual float GetSectionGripSize() const override;
	UE_API virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	UE_API virtual UMovieSceneSection* GetSectionObject() override;
	UE_API virtual FText GetSectionTitle() const override;
	UE_API virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const FGeometry& ParentGeometry, const double InCurrentTime, const float InDeltaTime ) override;

protected:

	/** Called to force a redraw of this section's thumbnails */
	UE_API void RedrawThumbnails();

	/** Get the range that is currently visible in the section's time space */
	UE_API TRange<double> GetVisibleRange() const;

	/** Get the total range that thumbnails are to be generated for in the section's time space */
	UE_API TRange<double> GetTotalRange() const;

	/** Get rename visibility */
	UE_API EVisibility GetRenameVisibility() const;

	/** When renaming is started, show the rename widget */
	UE_API void OnEnterEditingMode();

	/** When renaming is completed, hide the rename widget */
	UE_API void OnExitEditingMode();

	UE_API void BuildThumbnailsMenu(FMenuBuilder& InMenuBuilder);

protected:

	/** The section we are visualizing. */
	UMovieSceneSection* Section;

	/** The parent sequencer we are a part of. */
	TWeakPtr<ISequencer> SequencerPtr;

	/** A list of all thumbnails this section has. */
	FTrackEditorThumbnailCache ThumbnailCache;

	/** Saved playback status. Used for restoring state when rendering thumbnails */
	EMovieScenePlayerStatus::Type SavedPlaybackStatus;
	
	/** Rename widget */
	TSharedPtr<SInlineEditableTextBlock> NameWidget;

	/** Fade brush. */
	const FSlateBrush* WhiteBrush;

	/** Additional draw effects */
	ESlateDrawEffect AdditionalDrawEffect;

	enum class ETimeSpace
	{
		Global,
		Local,
	};

	/** Enumeration value specifyin in which time-space to generate thumbnails */
	ETimeSpace TimeSpace;

	FDelegateHandle RedrawThumbnailDelegateHandle;
};

/**
 * Thumbnail section, which paints and ticks the appropriate section.
 */
class FViewportThumbnailSection
	: public FThumbnailSection
	, public IViewportThumbnailClient
{
public:

	/** Create and initialize a new instance. */
	UE_API FViewportThumbnailSection(TSharedPtr<ISequencer> InSequencer, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, UMovieSceneSection& InSection);

	//~ IViewportThumbnailClient interface
	UE_API virtual void PreDraw(FTrackEditorThumbnail& Thumbnail) override;
	UE_API virtual void PostDraw(FTrackEditorThumbnail& Thumbnail) override;
};

#undef UE_API
