// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectKey.h"

#define UE_API MOVIESCENETOOLS_API

class UMovieScene;
class UMovieSceneSequence;

/** Node that represents an object binding, or a sub sequence (where the guid is zero) */
struct FSequenceBindingNode
{
	FSequenceBindingNode(FText InDisplayString, const UE::MovieScene::FFixedObjectBindingID& InBindingID, FSlateIcon InIcon)
		: BindingID(InBindingID)
		, ParentID(FGuid(), MovieSceneSequenceID::Invalid)
		, DisplayString(InDisplayString)
		, Icon(InIcon)
		, bIsSpawnable(false)
	{}

	/** Add a child */
	void AddChild(TSharedRef<FSequenceBindingNode> Child)
	{
		Child->ParentID = BindingID;
		Children.Add(Child);
	}

	/** This object's ID, and its parent's */
	UE::MovieScene::FFixedObjectBindingID BindingID, ParentID;
	/** The display string that represents this node */
	FText DisplayString;
	/** A representative icon for the node */
	FSlateIcon Icon;
	/** Whether this is a spawnable or not */
	bool bIsSpawnable;
	/** Array holding this node's children */
	TArray<TSharedRef<FSequenceBindingNode>> Children;
};

/** Data structure used internally to represent the bindings of a sequence recursively */
struct FSequenceBindingTree
{
	/**
	 * Conditionally reconstruct the tree structure from the specified sequence if any of the sequences have changed since the last population
	 *
	 * @param InSequence			The sequence to generate the tree for
	 * @param InActiveSequence		A sequence at which point we can start to generate localally resolving IDs
	 * @param InActiveSequenceID	The sequence ID for the above sequence within the root context
	 * @retrun True if this tree was rebuilt, false otherwise
	 */
	UE_API bool ConditionalRebuild(UMovieSceneSequence* InSequence, FObjectKey InActiveSequence, FMovieSceneSequenceID InActiveSequenceID);

	/**
	 * Construct the tree structure from the specified sequence.
	 *
	 * @param InSequence			The sequence to generate the tree for
	 * @param InActiveSequence		A sequence at which point we can start to generate localally resolving IDs
	 * @param InActiveSequenceID	The sequence ID for the above sequence within the root context
	 */
	UE_API void ForceRebuild(UMovieSceneSequence* InSequence, FObjectKey InActiveSequence, FMovieSceneSequenceID InActiveSequenceID);

	/** Get the root of the tree */
	TSharedRef<FSequenceBindingNode> GetRootNode() const
	{
		return TopLevelNode.ToSharedRef();
	}

	/** Find a node in the tree */
	TSharedPtr<FSequenceBindingNode> FindNode(UE::MovieScene::FFixedObjectBindingID BindingID) const
	{
		return Hierarchy.FindRef(BindingID);
	}

	bool IsEmpty() const
	{
		return bIsEmpty;
	}

private:

	struct FSequenceIDStack;

	/** Recursive sort helper for a sequence binding node */
	static UE_API void Sort(TSharedRef<FSequenceBindingNode> Node);

	/** Recursive builder function that iterates into sub sequences */
	UE_API void Build(UMovieSceneSequence* InSequence, FSequenceIDStack& SequenceIDStack);

	/** Ensure that a parent node exists for the specified object */
	UE_API TSharedRef<FSequenceBindingNode> EnsureParent(const FGuid& InParentGuid, UMovieScene* InMovieScene, FMovieSceneSequenceID SequenceID);

private:

	/** The ID of the currently 'active' sequence from which to generate relative IDs */
	FMovieSceneSequenceID ActiveSequenceID;
	/** The currently 'active' sequence from which to generate relative IDs */
	FObjectKey ActiveSequence;
	/** The node relating to the currently active sequence ID (if any) */
	TSharedPtr<FSequenceBindingNode> ActiveSequenceNode;
	/** The top level (root) node in the tree */
	TSharedPtr<FSequenceBindingNode> TopLevelNode;
	/** Map of hierarchical information */
	TMap<UE::MovieScene::FFixedObjectBindingID, TSharedPtr<FSequenceBindingNode>> Hierarchy;
	/** Map of sequence to its signature the last time we were built */
	TMap<FObjectKey, FGuid> CachedSequenceSignatures;
	/** Whether the tree is considered empty */
	bool bIsEmpty;
};

#undef UE_API
