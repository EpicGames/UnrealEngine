// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetExportTask.h"
#include "SequencerExportTask.generated.h"

/**
 * Contains data for a group of assets to import
 */ 
UCLASS(Transient, BlueprintType)
class USequencerExportTask : public UAssetExportTask
{
	GENERATED_BODY()

public:

	/* A UWorld for LevelSequences, UUserWidget for WidgetAnimations, or AActor for Actor Sequences, etc... */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TObjectPtr<UObject> SequencerContext;
};
