// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "HAL/ThreadSafeBool.h"
#include "Animation/CurveSequence.h"
#include "Rendering/RenderingCommon.h"
#include "MovieSceneToolsUserSettings.h"
#include "RHI.h"
#include "Misc/FrameTime.h"
#include "Slate/SlateTextures.h"

#define UE_API MOVIESCENETOOLS_API

struct FMinimalViewInfo;

class FLevelEditorViewportClient;
class FSceneViewport;
class FSlateShaderResource;
class FSlateTexture2DRHIRef;
class FThumbnailViewportClient;
class FTrackEditorThumbnail;
class FTrackEditorThumbnailPool;
class UCameraComponent;


DECLARE_DELEGATE_OneParam(FOnThumbnailDraw, FTrackEditorThumbnail&);


/**
 * Track Editor Thumbnail, which keeps a Texture to be displayed by a viewport.
 */
class FTrackEditorThumbnail
	: public ISlateViewport
	, public TSharedFromThis<FTrackEditorThumbnail>
{
public:

	/** Create and initialize a new instance. */
	UE_API FTrackEditorThumbnail(const FOnThumbnailDraw& InOnDraw, const FIntPoint& InDesiredSize, TRange<double> InTimeRange, double InPosition);

	/** Virtual destructor. */
	UE_API virtual ~FTrackEditorThumbnail();

public:

	/**
	 * Assign this thumbnail from a slate texture.
	 */
	UE_API void AssignFrom(TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> InTextureData);

	/**
	 * Ensure that this thumbnail has a render target of the specified size
	 */
	UE_API void ResizeRenderTarget(const FIntPoint& InSize);

	/**
	 * Access the (potentially null) render target to be used for rendering onto this thumbnail
	 */
	FSlateTextureRenderTarget2DResource* GetRenderTarget() const
	{
		return ThumbnailRenderTarget;
	}

	/**
	 * Get the desired size for this thumbnail on the UI
	 */
	FIntPoint GetDesiredSize() const
	{
		return DesiredSize;
	}

	/** Renders the thumbnail to the texture. */
	UE_API void DrawThumbnail();

	/** Prompt this thumbnail to fade in */
	UE_API void SetupFade(const TSharedRef<SWidget>& InWidget);
	UE_API void PlayFade();

	/** Gets the curve for fading in the thumbnail. */
	UE_API float GetFadeInCurve() const;

	/** Get the full time-range that this thumbnail occupies */
	const TRange<double> GetTimeRange() const { return TimeRange; }

	/** Get the time at which this thumbnail should be drawn */
	double GetEvalPosition() const { return Position; }

public:

	// ISlateViewport interface

	UE_API virtual FIntPoint GetSize() const override;
	UE_API virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	UE_API virtual bool RequiresVsync() const override;

	/** Destroy the texture on the next tick */
	UE_API void DestroyTexture();

public:

	/** Sort order for this thumbnail */
	int32 SortOrder;

	/** True when this thumbnail has been drawn, false otherwise */
	FThreadSafeBool bHasFinishedDrawing;

	/** True to ignore alpha on this thumbnail */
	bool bIgnoreAlpha;

private:

	/** Delegate to use to draw the thumbnail. */
	FOnThumbnailDraw OnDraw;

	/** The desired size of the thumbnail on the actual UI (Not necessarily the same size as the texture) */
	FIntPoint DesiredSize;

	/** The Texture RHI that holds the thumbnail. */
	FSlateTexture2DRHIRef* ThumbnailTexture;
	/** The texture render target used for 3D rendering on to the texture. May be null. */
	FSlateTextureRenderTarget2DResource* ThumbnailRenderTarget;

	/** Where in time this thumbnail is a rendering of. */
	TRange<double> TimeRange;

	/** The position we should actually render (within the above time range). */
	double Position;

	/** Fade curve to display while the thumbnail is redrawing. */
	FCurveSequence FadeInCurve;
};

/** Client interface for thumbanils that render the current world from a viewport */
struct IViewportThumbnailClient
{
	virtual UCameraComponent* GetViewCamera() { return nullptr; }
	virtual void PreDraw(FTrackEditorThumbnail& TrackEditorThumbnail) { }
	virtual void PostDraw(FTrackEditorThumbnail& TrackEditorThumbnail) { }
};

/** Custom thumbnail drawing client interface */
struct ICustomThumbnailClient
{
	virtual void Setup() {}
	virtual void Draw(FTrackEditorThumbnail& TrackEditorThumbnail) { }
};

/** Cache data */
struct FThumbnailCacheData
{
	FThumbnailCacheData() : VisibleRange(0), TimeRange(0), AllottedSize(0,0), DesiredSize(0, 0), Quality(EThumbnailQuality::Normal) {}

	bool operator==(const FThumbnailCacheData& RHS) const
	{
		return
			AllottedSize == RHS.AllottedSize && 
			VisibleRange == RHS.VisibleRange &&
			TimeRange == RHS.TimeRange &&
			DesiredSize == RHS.DesiredSize &&
			Quality == RHS.Quality &&
			SingleReferenceFrame == RHS.SingleReferenceFrame;
	}

	bool operator!=(const FThumbnailCacheData& RHS) const
	{
		return
			AllottedSize != RHS.AllottedSize ||
			VisibleRange != RHS.VisibleRange ||
			TimeRange != RHS.TimeRange ||
			DesiredSize != RHS.DesiredSize ||
			Quality != RHS.Quality ||
			SingleReferenceFrame != RHS.SingleReferenceFrame;
	}

	/** The visible range of our thumbnails we can see on the UI */
	TRange<double> VisibleRange;
	/** The total range to generate thumbnails for */
	TRange<double> TimeRange;
	/** Physical size of the thumbnail area */
	FIntPoint AllottedSize;
	/** Desired frame size constraint */
	FIntPoint DesiredSize;
	/** Thumbnail quality */
	EThumbnailQuality Quality;
	/** Set when we want to render a single reference frame */
	TOptional<double> SingleReferenceFrame;
};

class FTrackEditorThumbnailCache
{
public:
	UE_API FTrackEditorThumbnailCache(const TSharedPtr<FTrackEditorThumbnailPool>& ThumbnailPool, IViewportThumbnailClient* InViewportThumbnailClient);
	UE_API FTrackEditorThumbnailCache(const TSharedPtr<FTrackEditorThumbnailPool>& ThumbnailPool, ICustomThumbnailClient* InCustomThumbnailClient);
	
	UE_API ~FTrackEditorThumbnailCache();

	void ForceRedraw() { bForceRedraw = true; }

	UE_API void SetSingleReferenceFrame(TOptional<double> InReferenceFrame);
	TOptional<double> GetSingleReferenceFrame() const { return CurrentCache.SingleReferenceFrame; }

	UE_API void Update(const TRange<double>& NewRange, const TRange<double>& VisibleRange, const FIntPoint& AllottedSize, const FIntPoint& InDesiredSize, EThumbnailQuality InQuality, double InCurrentTime);

	UE_API void Revalidate(double InCurrentTime);

	const TArray<TSharedPtr<FTrackEditorThumbnail>>& GetThumbnails() const
	{
		return Thumbnails;
	}

protected:

	UE_API void DrawThumbnail(FTrackEditorThumbnail& TrackEditorThumbnail);

	UE_API void DrawViewportThumbnail(FTrackEditorThumbnail& TrackEditorThumbnail) const;

	UE_API void ComputeNewThumbnails();

	UE_API void Setup();

	UE_API bool ShouldRegenerateEverything() const;

	UE_API FIntPoint CalculateTextureSize(const FMinimalViewInfo& ViewInfo) const;

	UE_API void UpdateSingleThumbnail();

	UE_API void UpdateFilledThumbnails();

	UE_API void GenerateFront(const TRange<double>& Boundary);

	UE_API void GenerateBack(const TRange<double>& Boundary);

	UE_API void SetupViewportEngineFlags();

protected:

	/** Thumbnail client used for paint notifications */
	IViewportThumbnailClient* ViewportThumbnailClient;
	ICustomThumbnailClient* CustomThumbnailClient;

	/** The thumbnail pool that we are sending all of our thumbnails to. */
	TWeakPtr<FTrackEditorThumbnailPool> ThumbnailPool;

	FThumbnailCacheData CurrentCache;

	FThumbnailCacheData PreviousCache;

	TArray<TSharedPtr<FTrackEditorThumbnail>> Thumbnails;
	TArray<TSharedPtr<FTrackEditorThumbnail>> ThumbnailsNeedingRedraw;

	double LastComputationTime;
	bool bNeedsNewThumbnails;

	/** Whether to force a redraw or not */
	bool bForceRedraw;
};

#undef UE_API
