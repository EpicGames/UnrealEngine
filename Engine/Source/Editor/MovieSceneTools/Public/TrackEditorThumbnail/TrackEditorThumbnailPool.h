// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MOVIESCENETOOLS_API

class FTrackEditorThumbnail;
class ISequencer;

/**
 * Track Editor Thumbnail pool, which keeps a list of thumbnails that
 * need to be drawn and draws them incrementally.
 */
class FTrackEditorThumbnailPool
{
public:

	UE_API FTrackEditorThumbnailPool(TSharedPtr<ISequencer> InSequencer);

public:

	/** Requests that the passed in thumbnails need to be drawn */
	UE_API void AddThumbnailsNeedingRedraw(const TArray<TSharedPtr<FTrackEditorThumbnail>>& InThumbnails);

	/** Draws a small number of thumbnails that are enqueued for drawing */
	/* @return Whether thumbnails were drawn */
	UE_API bool DrawThumbnails();

	/** Informs the pool that the thumbnails passed in no longer need to be drawn */
	UE_API void RemoveThumbnailsNeedingRedraw(const TArray< TSharedPtr<FTrackEditorThumbnail>>& InThumbnails);

private:

	/** Parent sequencer we're drawing thumbnails for */
	TWeakPtr<ISequencer> Sequencer;

	/** Thumbnails enqueued for drawing */
	TArray< TSharedPtr<FTrackEditorThumbnail> > ThumbnailsNeedingDraw;

	/** Thumbnails that are currently being drawn */
	TArray< TSharedPtr<FTrackEditorThumbnail> > ThumbnailsBeingDrawn;

	double TimeOfLastDraw;
	double TimeOfLastUpdate;

	/** Whether we need to sort */
	bool bNeedsSort;
};

#undef UE_API
