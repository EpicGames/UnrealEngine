// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"
#include "Math/MathFwd.h"
#include "Misc/FrameTime.h"

class FSceneInterface;
class FRenderTarget;
enum class EThumbnailQuality : uint8;
struct FMinimalViewInfo;
struct FPostProcessSettings;

namespace UE::MoveSceneTools
{
	/** Prepares the sequencer so a thumbnail can be captured from it. */
	MOVIESCENETOOLS_API void PreDrawThumbnailSetupSequencer(ISequencer& Sequencer, FFrameTime CaptureFrame);
	inline void PreDrawThumbnailSetupSequencer(ISequencer& Sequencer, double ThumbnailEvalPosition)
	{
		PreDrawThumbnailSetupSequencer(Sequencer, ThumbnailEvalPosition * Sequencer.GetLocalTime().Rate);
	}
	
	/** Cleans up the sequencer from the consequences of PreDrawThumbnailSetupSequencer. */
	MOVIESCENETOOLS_API void PostDrawThumbnailCleanupSequencer(ISequencer& Sequencer);

	/** Renders the thumbnail's render target in the current Scene. */
	MOVIESCENETOOLS_API void DrawViewportThumbnail(
		FRenderTarget& ThumbnailRenderTarget,
		const FIntPoint& RenderTargetSize,
		FSceneInterface& Scene,
		const FMinimalViewInfo& ViewInfo,
		EThumbnailQuality Quality,
		const FPostProcessSettings* OverridePostProcessSettings = nullptr
		);
}
