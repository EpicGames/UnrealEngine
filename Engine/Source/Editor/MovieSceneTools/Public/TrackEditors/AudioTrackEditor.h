// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Misc/Guid.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneTrack.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SubclassOf.h"
#include "TimeToPixel.h"
#include "Widgets/SWidget.h"

#define UE_API MOVIESCENETOOLS_API

struct FAssetData;
class FAudioThumbnail;
class FDelegateHandle;
class FMenuBuilder;
class FSequencerSectionPainter;
class USoundWave;
class UMovieSceneAudioTrack;

/**
 * Tools for audio tracks
 */
class FAudioTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	UE_API FAudioTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FAudioTrackEditor();

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void OnInitialize() override;
	UE_API virtual void OnRelease() override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual bool IsResizable(UMovieSceneTrack* InTrack) const override;
	UE_API virtual void Resize(float NewSize, UMovieSceneTrack* InTrack) override;
	UE_API virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;
	
protected:

	/** Delegate for AnimatablePropertyChanged in HandleAssetAdded for sounds */
	UE_API FKeyPropertyResult AddNewSound(FFrameNumber KeyTime, class USoundBase* Sound, UMovieSceneAudioTrack* Track, int32 RowIndex);

	/** Delegate for AnimatablePropertyChanged in HandleAssetAdded for attached sounds */
	UE_API FKeyPropertyResult AddNewAttachedSound(FFrameNumber KeyTime, class USoundBase* Sound, UMovieSceneAudioTrack* Track, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo);

private:

	/** Callback for executing the "Add Audio Track" menu entry. */
	UE_API void HandleAddAudioTrackMenuEntryExecute();

	/** Callback for executing the "Add Audio Track" menu entry on an actor */
	UE_API void HandleAddAttachedAudioTrackMenuEntryExecute(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);

	/** Audio sub menu */
	UE_API TSharedRef<SWidget> BuildAudioSubMenu(FOnAssetSelected OnAssetSelected, FOnAssetEnterPressed OnAssetEnterPressed);

	/** Audio asset selected */
	UE_API void OnAudioAssetSelected(const FAssetData& AssetData, UMovieSceneTrack* Track);

	/** Audio asset enter pressed */
	UE_API void OnAudioAssetEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* Track);

	/** Attached audio asset selected */
	UE_API void OnAttachedAudioAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings);

	/** Attached audio asset enter pressed */
	UE_API void OnAttachedAudioEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings);

	/** Registers a delegate with the sequencer for monitoring edits */
	UE_API void RegisterMovieSceneChangedDelegate();

	/** Movie Scene Data Changed Delegate */
	UE_API void OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType);

	/** Returns true if the given Sequence or any subsequence contains an audio track */
	UE_API bool SequenceContainsAudioTrack(const UMovieSceneSequence* InSequence);

	/** Will return true if a sequence contains an audio track and the user was notified about the potential clock source issue */
	UE_API bool CheckSequenceClockSource();

	/** Prompts user and potentially modifies settings pref USequencerSettings::bAutoSelectAudioClockSource */
	UE_API void PromptUserForClockSource();

	/** Sets the clock source for the given sequence to use the audio clock */
	UE_API void SetClockSoureToAudioClock();

private:

	FDelegateHandle MovieSceneChangedDelegate;
};


/**
 * Class for audio sections, handles drawing of all waveform previews.
 */
class FAudioSection
	: public ISequencerSection
	, public TSharedFromThis<FAudioSection>
{
public:

	/** Constructor. */
	UE_API FAudioSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FAudioSection();

public:

	// ISequencerSection interface

	UE_API virtual UMovieSceneSection* GetSectionObject() override;
	UE_API virtual FText GetSectionTitle() const override;
	UE_API virtual FText GetSectionToolTip() const override;
	UE_API virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	UE_API virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const FGeometry& ParentGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual void BeginResizeSection() override;
	UE_API virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	UE_API virtual void BeginSlipSection() override;
	UE_API virtual void SlipSection(FFrameNumber SlipTime) override;
	UE_API virtual TOptional<FFrameTime> GetSectionTime(FSequencerSectionPainter& InPainter) const override;
	UE_API virtual void CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const override;
	
private:

	/* Re-creates the texture used to preview the waveform. */
	UE_API void RegenerateWaveforms(TRange<float> DrawRange, int32 XOffset, int32 XSize, const FColor& ColorTint, float DisplayScale);

private:

	/** The section we are visualizing. */
	UMovieSceneSection& Section;

	mutable TOptional<FTimeToPixel> TimeToPixel;

	/** The waveform thumbnail render object. */
	TSharedPtr<class FAudioThumbnail> WaveformThumbnail;

	/** Stored data about the waveform to determine when it is invalidated. */
	TRange<float> StoredDrawRange;
	FFrameNumber StoredStartOffset;
	int32 StoredXOffset;
	int32 StoredXSize;
	FColor StoredColor;
	float StoredSectionHeight;
	bool bStoredRepeating;

	/** Stored sound wave to determine when it is invalidated. */
	TWeakObjectPtr<USoundWave> StoredSoundWave;

	TWeakPtr<ISequencer> Sequencer;

	/** Cached start offset value valid only during resize */
	FFrameNumber InitialStartOffsetDuringResize;
	
	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;
};

#undef UE_API
