// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "MovieSceneTrack.h"
#include "SubTrackEditor.h"
#include "ISequencerTrackEditor.h"

#define UE_API MOVIESCENETOOLS_API

class AActor;
class FMenuBuilder;
class FTrackEditorThumbnailPool;
class UMovieSceneCinematicShotSection;
class UMovieSceneCinematicShotTrack;
class UMovieSceneSubSection;

/**
 * Tools for cinematic shots.
 */
class FCinematicShotTrackEditor : public FSubTrackEditor
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	UE_API FCinematicShotTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCinematicShotTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer .
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	UE_DEPRECATED(5.5, "Use FCameraCutPlaybackCapability::LastViewTargetCamera instead.")
	UE_API TWeakObjectPtr<AActor> GetCinematicShotCamera() const;

public:

	// ISequencerTrackEditor interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

	/*
	 * Render shots. 
	 *
	 * @param Sections The sections to render
	 */
	UE_API void RenderShots(const TArray<UMovieSceneCinematicShotSection*>& Sections);

public:

	// FSubTrackEditor interface
	UE_API virtual FText GetSubTrackName() const override;
	UE_API virtual FText GetSubTrackToolTip() const override;
	UE_API virtual FName GetSubTrackBrushName() const override;
	UE_API virtual FString GetSubSectionDisplayName(const UMovieSceneSubSection* Section) const override;
	UE_API virtual FString GetDefaultSubsequenceName() const override;
	UE_API virtual FString GetDefaultSubsequenceDirectory() const override;
	UE_API virtual TSubclassOf<UMovieSceneSubTrack> GetSubTrackClass() const;

protected:

	UE_API virtual bool HandleAddSubTrackMenuEntryCanExecute() const override;
	UE_API virtual bool CanHandleAssetAdded(UMovieSceneSequence* Sequence) const override;

private:

	/** Delegate for shots button lock state */
	UE_API ECheckBoxState AreShotsLocked() const;

	/** Delegate for locked shots button */
	UE_API void OnLockShotsClicked(ECheckBoxState CheckBoxState);
	
	/** Delegate for shots button lock tooltip */
	UE_API FText GetLockShotsToolTip() const;

	/** Callback for ImportEDL. */
	UE_API void ImportEDL();
	
	/** Callback for ExportEDL. */
	UE_API void ExportEDL();

	/** Callback for ImportFCPXML. */
	UE_API void ImportFCPXML();

	/** Callback for ExportFCPXML. */
	UE_API void ExportFCPXML();

private:

	/** The Thumbnail pool which draws all the viewport thumbnails for the shot track. */
	TSharedPtr<FTrackEditorThumbnailPool> ThumbnailPool;
};

#undef UE_API
