// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"
#include "SequencerCoreFwd.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"
#include "IMovieSceneLinkedAnimTrackProvider.h"
#include "LevelSequenceAnimSequenceLink.h"

#define UE_API MOVIESCENETOOLS_API

struct FAssetData;
struct FMovieSceneTimeWarpChannel;
struct FMovieSceneSequenceTransform;

class FMenuBuilder;
class FSequencerSectionPainter;
class SHorizontalBox;
class UAnimSeqExportOption;
class UAnimSequenceBase;
class UMovieSceneCommonAnimationTrack;
class UMovieSceneSkeletalAnimationSection;
class UMovieSceneSkeletalAnimationTrack;
class USkeletalMeshComponent;
class USkeleton;

namespace UE::Sequencer
{

class ITrackExtension;

/**
 * Tools for animation tracks
 */
class FCommonAnimationTrackEditor
	: public FMovieSceneTrackEditor
	, public FGCObject
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	UE_API FCommonAnimationTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FCommonAnimationTrackEditor() = default;

	//~ FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCommonAnimationTrackEditor");
	}

	/**
	* Keeps track of how many skeletal animation track editors we have*
	*/
	static UE_API int32 NumberActive;

	static UE_API USkeletalMeshComponent* AcquireSkeletalMeshFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr);
	static UE_API USkeleton* AcquireSkeletonFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr);
	static UE_API bool AcquireAllBoundSkeletalMeshObjectsFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr, TArray<USkeletalMeshComponent*>& OutSkelMeshComponents,
		TArray<FGuid>& OutBindingIds);
public:

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	UE_API virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	UE_API virtual void BuildObjectBindingColumnWidgets(
		TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox,
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding,
		const UE::Sequencer::FCreateOutlinerViewParams& InParams,
		const FName& InColumnName) override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual void OnInitialize() override;
	UE_API virtual void OnRelease() override;

protected:

	// Whether to activate FSkeletalAnimationTrackEditMode for skeleton/trail visualization.
	// Override and return false if the subclass provides its own edit mode.
	virtual bool ShouldActivateSkeletalAnimEditMode() const { return true; }

	virtual TSubclassOf<UMovieSceneCommonAnimationTrack> GetTrackClass() const = 0;

	/** Animation sub menu */
	UE_API TSharedRef<SWidget> BuildAddAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel);
	UE_API TSharedRef<SWidget> BuildAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneTrack* Track);
	UE_API void AddAnimationSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, USkeleton* Skeleton, UMovieSceneTrack* Track, int32 RowIndex = INDEX_NONE);

	/** Filter only compatible skeletons */
	UE_API bool FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton);

	/** Animation sub menu filter function */
	UE_API bool ShouldFilterAsset(const FAssetData& AssetData);

	/** Animation asset selected */
	UE_API void OnAnimationAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, int32 RowIndex = INDEX_NONE);

	/** Animation asset enter pressed */
	UE_API void OnAnimationAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, int32 RowIndex = INDEX_NONE);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	UE_API FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UAnimSequenceBase* AnimSequence, UMovieSceneTrack* Track, int32 RowIndex);
	
	/** Construct the binding menu*/
	UE_API void ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);

	/** Callback to Create the Animation Asset, pop open the dialog */
	UE_API void HandleCreateAnimationSequence(USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton, FGuid Binding, bool bCeateSoftLink);

	/** Callback to Creae the Animation Asset after getting the name*/
	UE_API bool CreateAnimationSequence(const TArray<UObject*> NewAssets,USkeletalMeshComponent* SkelMeshComp, FGuid Binding, bool bCreateSoftLink);

	/** Open the linked Anim Sequence*/
	UE_API void OpenLinkedAnimSequence(FGuid Binding);

	/** Can Open the linked Anim Sequence*/
	UE_API bool CanOpenLinkedAnimSequence(FGuid Binding);

	/** Resolve the linked AnimSequence for the given binding and unlink it from the LevelSequence*/
	UE_API void HandleDeleteLinkedAnimSequence(FGuid Binding);

	/** Whether a linked AnimSequence exists for the given binding*/
	UE_API bool CanDeleteLinkedAnimSequence(FGuid Binding);

	/** If we can open a linked anim sequence on any binding*/
	UE_API bool CanOpenLinkedAnimSequences(TArray<FGuid> Bindings);

	/** If we can isolate baked anim tracks since they exists*/
	UE_API bool HasNoLinkedTracks(TArray<FGuid> Bindings);

	/** Whether any binding has a linked track with bAutoBake == true */
	UE_API bool HasAutoBakedLinkedTracks(TArray<FGuid> Bindings);

	/** Remove all auto-baked linked sequences and the linked anim track for each binding */
	UE_API void HandleRemoveAutoBake(TArray<FGuid> Bindings);

	friend class FMovieSceneSkeletalAnimationParamsDetailCustomization;

public:

	/** Create a linked animation sequence for the binding. Used by linked anim track providers. */
	UE_API void CreateLinkedAnimationSequence(USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton, FGuid Binding, bool bCreateSoftLink);

	/** Remove the bidirectional link between an AnimSequence and its owning LevelSequence. */
	UE_API static bool UnlinkLinkedAnimSequence(UAnimSequence* AnimSequence);

	/** Remove linked AnimSequence link(s) and the LinkedAnimTrack for the given binding.
	 *  If bOnlyAutoBaked is true, only removes items where bAutoBake == true. */
	UE_API static bool DeleteLinkedAnimTrackForBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bOnlyAutoBaked);

	UE_API static TArray<FLevelSequenceAnimSequenceLinkItem*> GetLinkedAnimSequences(const TWeakPtr<ISequencer>& InSequencer, FGuid Binding);

	//isolate the linked anim track by muting other tracks here, return true if items were changed
	UE_API static bool IsolateLinkedAnimTrack(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID, bool bIsolate);

	// update linked anim sections ranges after bake in case playback range changed
	UE_API static void UpdateLinkedAnimSectionRange(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID);

	/** Find or create the linked animation track for the given binding. If none exists, creates the
	 *  necessary track structure, linked anim sequence, section, and disables evaluation on the track. */
	UE_API static UMovieSceneTrack* GetOrCreateLinkedAnimTrack(
		const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID);

	/** Find the first linked anim track provider that can handle this binding. */
	UE_API static IMovieSceneLinkedAnimTrackProvider* FindProviderForBinding(const TWeakPtr<ISequencer>& InSequencer, FGuid BindingID);

	/** If bIsolate is true it will activate any linked and autobaked anim tracks, and deactivate control rig or non linked anim tracks, if false does opposite */
	UE_API static  bool IsolateLinkedAnimTracks(const TWeakPtr<ISequencer>& InSequencer, TArray<FGuid> Bindings, bool bIsolate);

	/** Set up auto-bake recorders for any linked anim sequences on the given bindings. */
	UE_API static void ConfigureAutoBake(const TWeakPtr<ISequencer>& InSequencer, TArray<FGuid> Bindings);

	/** Find an empty skeletal animation track on the binding. Used by linked anim track providers. */
	UE_API UMovieSceneSkeletalAnimationTrack* FindEmptyAnimationTrack(FGuid BindingID);


private:
	//save link
	void SaveAnimSequenceLink(FLevelSequenceAnimSequenceLinkItem& Item);
	UMovieSceneTrack* GetLinkedAnimTrack(FGuid Binding, bool bCreateIfMissing);

	ECheckBoxState IsLinkedAnimTrackIsolated(FGuid ObjectGuid) const;
	void OnIsolateAnimTrackClicked(ECheckBoxState NewState, FGuid ObjectGuid);


protected:
	/* Was part of the thesection but should be at the track level since it takes the final blended result at the current time, not the section instance value*/
	UE_API bool CreatePoseAsset(const TArray<UObject*> NewAssets, FGuid InObjectBinding);
	UE_API void HandleCreatePoseAsset(FGuid InObjectBinding);
	UE_API bool CanCreatePoseAsset(FGuid InObjectBinding) const;

protected:
	/* For Anim Sequence UI Option with be gc'd*/
	TObjectPtr<UAnimSeqExportOption> AnimSeqExportOption;


protected:
	/* Delegate to handle sequencer changes for auto baking of anim sequences*/
	FDelegateHandle SequencerSavedHandle;
	UE_API void OnSequencerSaved(ISequencer& InSequence);
	FDelegateHandle SequencerChangedHandle;
	UE_API void OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType);
};


/** Class for animation sections */
class FCommonAnimationSection
	: public ISequencerSection
	, public TSharedFromThis<FCommonAnimationSection>
{
public:

	/** Constructor. */
	UE_API FCommonAnimationSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FCommonAnimationSection();

public:

	// ISequencerSection interface

	UE_API virtual UMovieSceneSection* GetSectionObject() override;
	UE_API virtual FText GetSectionTitle() const override;
	UE_API virtual FText GetSectionToolTip() const override;
	UE_API virtual TOptional<FFrameTime> GetSectionTime(FSequencerSectionPainter& InPainter) const override;
	UE_API virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	UE_API virtual FMargin GetContentPadding() const override;
	UE_API virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	UE_API virtual void BeginResizeSection() override;
	UE_API virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	UE_API virtual void BeginSlipSection() override;
	UE_API virtual void SlipSection(FFrameNumber SlipTime) override;
	UE_API virtual void CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const override;
	UE_API virtual void BeginDilateSection() override;
	UE_API virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;
	UE_API virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;
	UE_API virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding) override;

	// Whether to show the legacy bone matching UI from MovieSceneCommonAnimationTrack.
	// Override to return false when using a different bone matching system (e.g. AnimMixer).
	virtual bool ShouldShowLegacyBoneMatching() const { return true; }

protected:
	UE_API void FindBestBlendSection(FGuid InObjectBinding);
protected:

	/** The section we are visualizing */
	TWeakObjectPtr<UMovieSceneSkeletalAnimationSection> WeakSection;

	/** Used to draw animation frame, need selection state and local time*/
	TWeakPtr<ISequencer> Sequencer;

	TUniquePtr<FMovieSceneSequenceTransform> InitialDragTransform;
	TUniquePtr<FMovieSceneTimeWarpChannel> PreDilateChannel;
	double PreDilatePlayRate;
};

} // namespace UE::Sequencer

#undef UE_API
