// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Materials/MaterialParameters.h"
#include "MovieSceneTrackEditor.h"

#define UE_API MOVIESCENETOOLS_API

class UMovieSceneCustomPrimitiveDataTrack;

using FGetStartIndexDelegate = TFunction<uint8()>;

/**
 * Track editor for custom primitive data tracks
 */
class FCustomPrimitiveDataTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/** Constructor. */
	UE_API FCustomPrimitiveDataTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FCustomPrimitiveDataTrackEditor() = default;

	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual bool GetDefaultExpansionState(UMovieSceneTrack* InTrack) const override; 
	UE_API virtual void ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;

protected:

private:


	/** Provides the contents of the outliner edit widget */
	UE_API TSharedRef<SWidget> OnGetAddMenuContent(FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, int32 TrackInsertRowIndex);

	/** Provides the contents of the add parameter menu. */
	UE_API TSharedRef<SWidget> OnGetAddParameterMenuContent(FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack);

	/** Provides the contents of the add parameter menu. */
	UE_API void OnBuildAddParameterMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UMovieSceneCustomPrimitiveDataTrack* CPDTrack);

	/** Adds a scalar parameter and initial key to a custom primitive data track.
	 * @param ObjectBinding The object binding which owns the custom primitive data track.
	 * @param CPDTrack The track in which to look for sections to add the parameter to.
	 * @param PrimitiveDataIndex The index for the primitive data to animate
	 */
	UE_API void AddScalarParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate);

	/** Adds a Vector2D parameter and initial key to a custom primitive data track.
	 * @param ObjectBinding The object binding which owns the custom primitive data track.
	 * @param CPDTrack The track in which to look for sections to add the parameter to.
	 * @param PrimitiveDataStartIndex The start index in primitive data where we are animating a Vector2D
	 */
	UE_API void AddVector2DParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate);

	/** Adds a Vector parameter and initial key to a custom primitive data track.
	 * @param ObjectBinding The object binding which owns the custom primitive data track.
	 * @param CPDTrack The track in which to look for sections to add the parameter to.
	 * @param PrimitiveDataStartIndex The start index in primitive data where we are animating a Vector
	 */
	UE_API void AddVectorParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate);

	/** Adds a color parameter and initial key to a custom primitive data track.
	 * @param ObjectBinding The object binding which owns the custom primitive data track.
	 * @param CPDTrack The track in which to look for sections to add the parameter to.
	 * @param PrimitiveDataStartIndex The start index in primitive data where we are animating a Color
	 */
	UE_API void AddColorParameter(UPrimitiveComponent* Component, UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate);

	/** Returns whether we can add a parameter starting at this index of size ParameterSize. 
	 * Will return false if another parameter added overlaps that index.
	 */
	UE_API bool CanAddParameter(UMovieSceneCustomPrimitiveDataTrack* CPDTrack, FGetStartIndexDelegate GetStartIndexDelegate, int ParameterSize);

	UE_API void ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);

	UE_API void HandleAddCustomPrimitiveDataTrackExecute(UPrimitiveComponent* Component);

	uint8 StartIndex = 0;
};

#undef UE_API
