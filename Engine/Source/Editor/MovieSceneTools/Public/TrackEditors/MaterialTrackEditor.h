// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Materials/MaterialParameters.h"
#include "MovieSceneTrackEditor.h"

#define UE_API MOVIESCENETOOLS_API

class UMaterial;
class UMaterialInterface;
class UMovieSceneMaterialTrack;
class USceneComponent;
struct FComponentMaterialInfo;

/**
 * Track editor for material parameters.
 */
class FMaterialTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/** Constructor. */
	UE_API FMaterialTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FMaterialTrackEditor() = default;

public:

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget( const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params ) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;

protected:

	/** Gets a material interface for a specific object binding and material track. */
	virtual UMaterialInterface* GetMaterialInterfaceForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack ) = 0;

private:

	/** Provides the contents of the outliner edit widget */
	UE_API TSharedRef<SWidget> OnGetAddMenuContent( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, int32 TrackInsertRowIndex );

	/** Provides the contents of the add parameter menu. */
	UE_API TSharedRef<SWidget> OnGetAddParameterMenuContent( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack );

	/** Provides the contents of the add parameter menu. */
	UE_API void OnBuildAddParameterMenu( FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack );

	/** Gets a material for a specific object binding and track */
	UE_API UMaterial* GetMaterialForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack );

	/** Adds a scalar parameter and initial key to a material track.
	 * @param ObjectBinding The object binding which owns the material track.
	 * @param MaterialTrack The track in which to look for sections to add the parameter to.
	 * @param ParameterName The name of the parameter to add an initial key for.
	 */
	UE_API void AddScalarParameter( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, FMaterialParameterInfo ParameterInfo, FString InLayerName, FString InAssetName );

	/** Adds a color parameter and initial key to a material track.
	* @param ObjectBinding The object binding which owns the material track.
	 * @param MaterialTrack The track in which to look for sections to add the parameter to.
	* @param ParameterName The name of the parameter to add an initial key for.
	*/
	UE_API void AddColorParameter( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack, FMaterialParameterInfo ParameterInfo, FString InLayerName, FString InAssetName);
};


/**
 * A specialized material track editor for component materials
 */
class FComponentMaterialTrackEditor
	: public FMaterialTrackEditor
{
public:

	FComponentMaterialTrackEditor( TSharedRef<ISequencer> InSequencer );

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface
	virtual void ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual bool GetDefaultExpansionState(UMovieSceneTrack* InTrack) const override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

protected:

	// FMaterialtrackEditor interface

	virtual UMaterialInterface* GetMaterialInterfaceForTrack( FGuid ObjectBinding, UMovieSceneMaterialTrack* MaterialTrack ) override;

private:

	void ConstructObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings);
	/** Callback for executing the add component material track. */
	void HandleAddComponentMaterialActionExecute(USceneComponent* Component, FComponentMaterialInfo MaterialInfo);
	/** Callback for rebinding a component material track to a different material slot */
	void FillRebindMaterialTrackMenu(FMenuBuilder& MenuBuilder, class UMovieSceneComponentMaterialTrack* MaterialTrack, class UPrimitiveComponent* Component, FGuid ObjectBinding);

};

#undef UE_API
