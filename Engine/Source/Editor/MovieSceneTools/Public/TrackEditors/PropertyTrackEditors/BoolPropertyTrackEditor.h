// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "Containers/Array.h"
#include "PropertyTrackEditor.h"
#include "Tracks/MovieSceneBoolTrack.h"

UE_DEPRECATED_HEADER(5.7, "This header is no longer in use.")

/**
 * A property track editor for Booleans.
 */
class UE_DEPRECATED(5.7, "This class is no longer in use") FBoolPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneBoolTrack>
{
public:

	/**
	 * Constructor.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	FBoolPropertyTrackEditor( TSharedRef<ISequencer> InSequencer )
		: FPropertyTrackEditor( InSequencer, GetAnimatedPropertyTypes() )
	{ }

	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromPropertyTypeName(NAME_BoolProperty) });
	}
};
