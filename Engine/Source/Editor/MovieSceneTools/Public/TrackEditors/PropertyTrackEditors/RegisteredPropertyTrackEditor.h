// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PropertyTrackEditor.h"

namespace UE::Sequencer
{

struct IRegisteredProperty
{
	virtual bool AppliesToTrack(UMovieSceneTrack* Track) const = 0;
	virtual bool AppliesToSection(UMovieSceneSection* Section) const = 0;

	virtual TSharedPtr<ISequencerSection> TryMakeSectionInterface(UMovieSceneSection* SectionObject, UMovieSceneTrack* Track, const FGuid& ObjectBinding, TWeakPtr<ISequencer> WeakSequencer) { return nullptr; }
};

template<typename TrackType, typename SectionType>
struct TRegisteredProperty : IRegisteredProperty
{
	using MakeSectionInterfacePtr = TSharedPtr<ISequencerSection> (*)(UMovieSceneSection*, UMovieSceneTrack*, const FGuid&, TWeakPtr<ISequencer>) ;
	TRegisteredProperty(MakeSectionInterfacePtr InMakeSectionInterface = nullptr)
		: MakeSectionInterface(InMakeSectionInterface)
	{
	}

	bool AppliesToTrack(UMovieSceneTrack* Track) const override
	{
		return Track->IsA<TrackType>();
	}
	bool AppliesToSection(UMovieSceneSection* Section) const override
	{
		return Section->IsA<SectionType>();
	}
	TSharedPtr<ISequencerSection> TryMakeSectionInterface(UMovieSceneSection* SectionObject, UMovieSceneTrack* Track, const FGuid& ObjectBinding, TWeakPtr<ISequencer> WeakSequencer) override
	{
		if (MakeSectionInterface)
		{
			return MakeSectionInterface(SectionObject, Track, ObjectBinding, WeakSequencer);
		}
		return nullptr;
	}

	MakeSectionInterfacePtr MakeSectionInterface;
};

void RegisterPropertyType(TSharedPtr<IRegisteredProperty> RegisteredProperty);

class FRegisteredPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieScenePropertyTrack>
{
public:

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	FRegisteredPropertyTrackEditor(TSharedRef<ISequencer> InSequencer);

	static void GenerateKeysFromPropertyChanged(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, ISequencer& Sequencer, const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys);

	static UE::MovieScene::FIntermediatePropertyValue RecomposeValue(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, ISequencer& Sequencer, const UE::MovieScene::FIntermediatePropertyValue& InCurrentValue, UObject* AnimatedObject, UMovieSceneSection* Section);

	static const UE::MovieScene::FPropertyDefinition* FindMatchingPropertyDefinition(const FProperty* InProperty);

private:

	void InitializeNewTrack(UMovieScenePropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* PropertyDefinition);
	void InitializeNewTrack(UMovieScenePropertyTrack* NewTrack, FPropertyChangedParams PropertyChangedParams) override;
	void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;
	TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults) override;
	bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;

private:

	void HandlePropertyChanged(const FPropertyChangedParams& PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* Property);
	TSubclassOf<UMovieSceneTrack> GetCustomizedTrackClass(const FProperty* Property) const;
	FKeyPropertyResult KeyProperty(FFrameNumber KeyTime, FPropertyChangedParams PropertyChangedParams, const UE::MovieScene::FPropertyDefinition* Property);
	FOnAnimatablePropertyChanged FindPropertyChangedHandler(const FProperty*) const;
	void ProcessKeyOperation(UObject* ObjectToKey, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults);
};



} // namespace UE::Sequencer