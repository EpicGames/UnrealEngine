// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Misc/Guid.h"
#include "MovieSceneTrack.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "KeyframeTrackEditor.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "UObject/NameTypes.h"

#define UE_API MOVIESCENETOOLS_API

class FMenuBuilder;
class ISequencer;
class ISequencerTrackEditor;
class SWidget;
class UClass;
class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneTrack;
class UObject;
struct FBuildEditWidgetParams;
struct FGuid;

/**
 * A property track editor for controlling the lifetime of a sapwnable object
 */
class FSpawnTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneSpawnTrack>
{
public:

	/**
	 * Factory function to create an instance of this class (called by a sequencer).
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSequencer The sequencer instance to be used by this tool.
	 */
	UE_API FSpawnTrackEditor(TSharedRef<ISequencer> InSequencer);

public:

	// ISequencerTrackEditor interface

	UE_API virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override;
	UE_API virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override  { return TSharedPtr<SWidget>(); }
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override { return false; }
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

private:

	/** Callback for executing the "Add Spawn Track" menu entry. */
	UE_API void HandleAddSpawnTrackMenuEntryExecute(TArray<FGuid> ObjectBindings);
	UE_API bool CanAddSpawnTrack(FGuid ObjectBinding) const;
};

#undef UE_API
