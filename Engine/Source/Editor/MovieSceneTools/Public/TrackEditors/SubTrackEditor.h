// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "KeyframeTrackEditor.h"
#include "MovieSceneTrackEditor.h"

#define UE_API MOVIESCENETOOLS_API

class AActor;
struct FAssetData;
class FMenuBuilder;
class UMovieSceneSubTrack;

namespace UE::Sequencer { struct FAddKeyResult; }

/**
 * Tools for subsequences
 */
class FSubTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneSubTrack>
{
public:

	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	UE_API FSubTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FSubTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static UE_API TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual bool IsResizable(UMovieSceneTrack* InTrack) const override;
	UE_API virtual void OnInitialize() override;
	UE_API virtual void OnRelease() override;
	UE_API virtual void Resize(float NewSize, UMovieSceneTrack* InTrack) override;
	UE_API virtual bool GetDefaultExpansionState(UMovieSceneTrack* InTrack) const override;
	virtual bool HasTransformKeyBindings() const override { return true; }
	UE_API virtual bool CanAddTransformKeysForSelectedObjects() const override;
	UE_API virtual void OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) override;
	UE_API virtual void OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) override;

public:
	
	/** Insert sequence into this track */
	UE_API virtual void InsertSection(UMovieSceneTrack* Track);

	/** Duplicate the section into this track */
	UE_API virtual void DuplicateSection(UMovieSceneSubSection* Section);

	/** Create a new take of the given section */
	UE_API virtual void CreateNewTake(UMovieSceneSubSection* Section);

	/** Switch the selected section's take sequence */
	UE_API virtual void ChangeTake(UMovieSceneSequence* Sequence);

	/** Generate a menu for takes for this section */
	UE_API virtual void AddTakesMenu(UMovieSceneSubSection* Section, FMenuBuilder& MenuBuilder);

	/** Edit the section's metadata */
	UE_API virtual void EditMetaData(UMovieSceneSubSection* Section);

	/** Update the current active edit mode when a subtrack or section is selected.*/
	UE_API void UpdateActiveMode();

	/**
	 * Check whether the given sequence can be added as a sub-sequence.
	 *
	 * The purpose of this method is to disallow circular references
	 * between sub-sequences in the focused movie scene.
	 *
	 * @param Sequence The sequence to check.
	 * @return true if the sequence can be added as a sub-sequence, false otherwise.
	 */
	UE_API bool CanAddSubSequence(const UMovieSceneSequence& Sequence) const;

public:

	/** Get the name of the sub track */
	UE_API virtual FText GetSubTrackName() const;

	/** Get the tooltip for this sub track editor */
	UE_API virtual FText GetSubTrackToolTip() const;

	/** Get the brush used for the sub track editor */
	UE_API virtual FName GetSubTrackBrushName() const;

	/** Get the display name for the sub section */
	UE_API virtual FString GetSubSectionDisplayName(const UMovieSceneSubSection* Section) const;

	/** Get the default sub sequence name */
	UE_API virtual FString GetDefaultSubsequenceName() const;

	/** Get the sub sequence directory */
	UE_API virtual FString GetDefaultSubsequenceDirectory() const;

	/** Get the UMovieSceneSubTrack class */
	UE_API virtual TSubclassOf<UMovieSceneSubTrack> GetSubTrackClass() const;

	/** Called when the editor mode has made external changes to the origin data. */
	UE_API void UpdateOrigin(FVector InPosition, FRotator InRotation);

	/** Called when sequence playback updates to revert preview data on modified secitons */
	UE_API void ResetSectionPreviews();
	void ResetSectionPreviews(FMovieSceneSequenceIDRef IDRef) { ResetSectionPreviews(); }
	void ResetSectionPreviews(const TArray<FFrameNumber>& Frames, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection* InSection) { ResetSectionPreviews(); }

	/** Query's the channel data directly (does not take parent transforms into account). Used for setting keyframes. */
	UE_API FTransform GetTransformOriginDataForSubSection(const UMovieSceneSubSection* SubSection) const;

	/** Helper function that finds the previous key. Used to "unwind" rotators */
	UE_API int32 GetPreviousKey(FMovieSceneDoubleChannel& Channel, FFrameNumber Time);

	/** Helper function to fix-up Euler rotations if they would go over 180 degrees due to interpolation. */
	UE_API double UnwindChannel(const double& OldValue, double NewValue);

protected:

	/** Get the list of supported sequence class paths */
	UE_API virtual void GetSupportedSequenceClassPaths(TArray<FTopLevelAssetPath>& OutClassPaths) const;

	/** Callback for executing the "Add Subsequence" menu entry. */
	UE_API virtual void HandleAddSubTrackMenuEntryExecute();

	/** Callback for determining whether the "Add Subsequence" menu entry can execute. */
	virtual bool HandleAddSubTrackMenuEntryCanExecute() const { return true; }

	/** Whether to handle this asset being dropped onto the sequence as opposed to a specific track. */
	UE_API virtual bool CanHandleAssetAdded(UMovieSceneSequence* Sequence) const;

	UE_DEPRECATED(5.3, "CreateNewTrack has been deprecated, please implement GetSubTrackClass")
	UE_API virtual UMovieSceneSubTrack* CreateNewTrack(UMovieScene* MovieScene) const;

	/** Find or create a sub track. If the given track is a subtrack, it will be returned. */
	UE_API UMovieSceneSubTrack* FindOrCreateSubTrack(UMovieScene* MovieScene, UMovieSceneTrack* Track) const;

	/** Callback for generating the menu of the "Add Sequence" combo button. */
	UE_API TSharedRef<SWidget> HandleAddSubSequenceComboButtonGetMenuContent(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> ViewModel);

	/** Add the input UMovieSceneSequence to the input UMovieSceneTrack */
	UE_API void AddSequenceToTrack(UMovieSceneSequence* InMovieSceneSequence, UMovieSceneTrack* InTrack);

private:

	/** Callback for executing a menu entry in the "Add Sequence" combo button. */
	UE_API void HandleAddSubSequenceComboButtonMenuEntryExecute(const FAssetData& AssetData, UMovieSceneTrack* InTrack);

	/** Callback for executing a menu entry in the "Add Sequence" combo button when enter pressed. */
	UE_API void HandleAddSubSequenceComboButtonMenuEntryEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneTrack* InTrack);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	UE_API FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UMovieSceneSequence* InMovieSceneSequence, UMovieSceneTrack* InTrack, int32 RowIndex);

	/** Callback for AnimatablePropertyChanged in HandleAssetAdded. */
	UE_API FKeyPropertyResult HandleSequenceAdded(FFrameNumber KeyTime, UMovieSceneSequence* Sequence, UMovieSceneTrack* Track, int32 RowIndex);

	/** Handles adding keys to section.**/
	UE_API void ProcessKeyOperationInternal(TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& Sequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr);

	/** Helper for creating new keys.*/
	UE_API void GetOriginKeys(const FVector& CurrentPosition, const FRotator& CurrentRotation, EMovieSceneTransformChannel ChannelsToKey, UMovieSceneSection* Section, FGeneratedTrackKeys& OutGeneratedKeys);

	/** Helper for getting the sections that should be keyed. **/
	UE_API void GetSectionsToKey(TArray<UMovieSceneSubSection*>& OutSectionsToKey) const;

	/** Sections this editor has added preview data to for keyframing. Used to revert data when other edits are made, before adding a key */
	TArray<UMovieSceneSubSection*> SectionsWithPreviews;
};

#undef UE_API
