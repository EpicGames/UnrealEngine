// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "ISequencerSection.h"
#include "Templates/UniquePtr.h"
#include "Sections/MovieSceneSubSection.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencer.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieScene.h"
#include "MovieSceneMetaData.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Rendering/DrawElements.h"
#include "SequencerSectionPainter.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "TimerManager.h"

#define UE_API MOVIESCENETOOLS_API

struct FMovieSceneTimeWarpChannel;
struct FMovieSceneSequenceTransform;
class ISequencer;

/**
 * Parameters for painting a sub-section.
 */
struct FSubSectionPainterParams
{
    FSubSectionPainterParams() : bShowTrackNum(true)
    {}
    FSubSectionPainterParams(FMargin InContentPadding)
	: ContentPadding(InContentPadding), bShowTrackNum(true)
    {}

    FMargin ContentPadding;
    bool bShowTrackNum;
};

/**
 * Enum indicating the result of painting the sub-section.
 */
enum FSubSectionPainterResult
{
    FSSPR_Success,
    FSSPR_InvalidSection,
    FSSPR_NoInnerSequence
};

/**
 * Utility class for drawing sub-sequence sections.
 */
class FSubSectionPainterUtil
{
public:
    /** Paints the sub-section, mostly by painting loop boundaries, if appropriate */
    static UE_API FSubSectionPainterResult PaintSection(TSharedPtr<const ISequencer> Sequencer, const UMovieSceneSubSection& SectionObject, FSequencerSectionPainter& InPainter, FSubSectionPainterParams Params);

private:
    static UE_API void PaintSectionBounds(TSharedPtr<const ISequencer> Sequencer, const UMovieSceneSubSection& SectionObject, const UMovieSceneSequence& InnerSequence, FSequencerSectionPainter& InPainter, ESlateDrawEffect DrawEffects);
};

/**
 * Utility class for editing (resize/slip) sub-sequence sections.
 */
class FSubSectionEditorUtil
{
public:
    UE_API FSubSectionEditorUtil(UMovieSceneSubSection& InSection);
    UE_API ~FSubSectionEditorUtil();

    /** Starts a resize operation */
    UE_API void BeginResizeSection();
    /** Resizes the section, returns the new resize time */
    UE_API FFrameNumber ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime);

    /** Starts a new slip operation */
    UE_API void BeginSlipSection();
    /** Slips the section, returns the new slip time */
    UE_API FFrameNumber SlipSection(FFrameNumber SlipTime);

    /** Starts a new dilate operation */
    UE_API void BeginDilateSection();
    /** Dilates the section */
    UE_API void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor);

private:
    /** The section object this utility class is editing */
    UMovieSceneSubSection& SectionObject;

	TUniquePtr<FMovieSceneSequenceTransform> InitialDragTransform;

	TUniquePtr<FMovieSceneTimeWarpChannel> PreDilateChannel;

	/** Cached time scale valid only during dilate */
	double PreDilateTimeScale;
};

class FSubTrackEditorUtil
{
public:
	/**
	 * Check whether the given sequence can be added as a sub-sequence.
	 *
	 * The purpose of this method is to disallow circular references
	 * between sub-sequences in the focused movie scene.
	 *
	 * @param Sequence The sequence to check.
	 * @return true if the sequence can be added as a sub-sequence, false otherwise.
	 */
	static UE_API bool CanAddSubSequence(const UMovieSceneSequence* CurrentSequence, const UMovieSceneSequence& SubSequence);

	static UE_API UMovieSceneMetaData* FindOrAddMetaData(UMovieSceneSequence* Sequence);

	static UE_API FText GetMetaDataText(const UMovieSceneSequence* Sequence);
};

/**
 * Mixin class for sub-sequence section interfaces.
 *
 * This mixin effectively wraps the 2 utility classes above, hooks them up to the ISequencerSection interface,
 * and implements a few other ISequencerSection interface methods. As such, it's mostly handy boilerplate code,
 * which you can avoid (if it doesn't fit your use-case) in favour of using the uility classes above directly.
 */
template<typename ParentSectionClass=ISequencerSection>
class TSubSectionMixin
	: public ParentSectionClass
{
public:
    template<typename... ParentCtorParams>
    TSubSectionMixin(TSharedPtr<ISequencer> InSequencer, UMovieSceneSubSection& InSection, ParentCtorParams&&... Params);

    // ISequencerSection interface
    virtual UMovieSceneSection* GetSectionObject() override;
    virtual FText GetSectionTitle() const override;
	virtual FText GetSectionToolTip() const override;
	virtual TOptional<FFrameTime> GetSectionTime(FSequencerSectionPainter& InPainter) const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
    virtual bool IsReadOnly() const override;
    virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const override;
    virtual FReply OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent) override;
    virtual void BeginResizeSection() override;
    virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
    virtual void BeginSlipSection() override;
    virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual void BeginDilateSection() override;
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;
	virtual bool RequestDeleteKeyArea( const TArray<FName>& KeyAreaNamePath ) override;

protected:
	static const float TrackHeight;
    TSharedPtr<ISequencer> GetSequencer() const { return SequencerPtr.Pin(); }
    UMovieSceneSubSection& GetSubSectionObject() { return SubSectionObject; }
    const UMovieSceneSubSection& GetSubSectionObject() const { return SubSectionObject; }
    FSubSectionEditorUtil& GetEditorUtil() { return EditorUtil; }

    template<typename SectionType>
    SectionType& GetSectionObjectAs() { return *CastChecked<SectionType>(&SubSectionObject); }
    template<typename SectionType>
    const SectionType& GetSectionObjectAs() const { return *CastChecked<SectionType>(&SubSectionObject); }

private:
    /** The Sequencer this section belongs to */
    TWeakPtr<ISequencer> SequencerPtr;

    /** The section object this interface represents */
    UMovieSceneSubSection& SubSectionObject;

    /** Utility class for resizing/slipping sub-sequence sections */
    FSubSectionEditorUtil EditorUtil;
};

template<typename ParentSectionClass>
const float TSubSectionMixin<ParentSectionClass>::TrackHeight = 50.f;

template<typename ParentSectionClass>
template<typename... ParentCtorParams>
TSubSectionMixin<ParentSectionClass>::TSubSectionMixin(TSharedPtr<ISequencer> InSequencer, UMovieSceneSubSection& InSection, ParentCtorParams&&... Params)
    : ParentSectionClass(std::forward<ParentCtorParams>(Params)...)
    , SequencerPtr(InSequencer)
    , SubSectionObject(InSection)
    , EditorUtil(InSection)
{}

template<typename ParentSectionClass>
UMovieSceneSection* TSubSectionMixin<ParentSectionClass>::GetSectionObject()
{
    return &SubSectionObject;
}

#define LOCTEXT_NAMESPACE "FSubTrackEditorBase"
template<typename ParentSectionClass>
FText TSubSectionMixin<ParentSectionClass>::GetSectionTitle() const
{
    if (SubSectionObject.GetSequence())
    {
        return FText::FromString(SubSectionObject.GetSequence()->GetName());
    }
    else
    {
        return LOCTEXT("NoSequenceSelected", "No Sequence Selected");
    }
}

template<typename ParentSectionClass>
FText TSubSectionMixin<ParentSectionClass>::GetSectionToolTip() const
{
	const UMovieScene* MovieScene = SubSectionObject.GetTypedOuter<UMovieScene>();
	const UMovieSceneSequence* InnerSequence = SubSectionObject.GetSequence();
	const UMovieScene* InnerMovieScene = InnerSequence ? InnerSequence->GetMovieScene() : nullptr;

	if (!MovieScene || !InnerMovieScene || !SubSectionObject.HasStartFrame() || !SubSectionObject.HasEndFrame())
	{
		return FText::GetEmpty();
	}

	FFrameRate InnerTickResolution = InnerMovieScene->GetTickResolution();

	// Calculate the length of this section and convert it to the timescale of the sequence's internal sequence
	FFrameTime SectionLength = ConvertFrameTime(SubSectionObject.GetExclusiveEndFrame() - SubSectionObject.GetInclusiveStartFrame(), MovieScene->GetTickResolution(), InnerTickResolution);

	// Calculate the inner start time of the sequence in both full tick resolution and frame number
	FFrameTime StartOffset = SubSectionObject.GetOffsetTime().Get(0);
	FFrameTime InnerStartTime = InnerMovieScene->GetPlaybackRange().GetLowerBoundValue() + StartOffset;
	int32 InnerStartFrame = ConvertFrameTime(InnerStartTime, InnerTickResolution, InnerMovieScene->GetDisplayRate()).RoundToFrame().Value;

	// Calculate the length, which is limited by both the outer section length and internal sequence length, in terms of internal frames
	int32 InnerFrameLength = ConvertFrameTime(FMath::Min(SectionLength, InnerMovieScene->GetPlaybackRange().GetUpperBoundValue() - InnerStartTime), InnerTickResolution, InnerMovieScene->GetDisplayRate()).RoundToFrame().Value;

	// Calculate the inner frame number of the end frame
	int32 InnerEndFrame = InnerStartFrame + InnerFrameLength;
	
	FText MetaDataText = FSubTrackEditorUtil::GetMetaDataText(InnerSequence);

	if (MetaDataText.IsEmpty())
	{
		return FText::Format(LOCTEXT("ToolTipContentFormat", "{0} - {1} ({2} frames @ {3})"),
			InnerStartFrame,
			InnerEndFrame,
			InnerFrameLength,
			InnerMovieScene->GetDisplayRate().ToPrettyText()
		);
	}
	else
	{
		return FText::Format(LOCTEXT("ToolTipContentFormatWithMetaData", "{0} - {1} ({2} frames @ {3})\n\n{4}"),
			InnerStartFrame,
			InnerEndFrame,
			InnerFrameLength,
			InnerMovieScene->GetDisplayRate().ToPrettyText(),
			MetaDataText
		);
	}
}

template<typename ParentSectionClass>
TOptional<FFrameTime> TSubSectionMixin<ParentSectionClass>::GetSectionTime(FSequencerSectionPainter& InPainter) const
{
	if (!InPainter.bIsSelected)
	{
		return TOptional<FFrameTime>();
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return TOptional<FFrameTime>();
	}

	FFrameTime CurrentTime = Sequencer->GetLocalTime().Time;
	if (!SubSectionObject.GetRange().Contains(CurrentTime.FrameNumber))
	{
		return TOptional<FFrameTime>();
	}

	if (!SubSectionObject.GetSequence() || !SubSectionObject.GetSequence()->GetMovieScene())
	{
		return TOptional<FFrameTime>();
	}

	const UMovieScene* SubSequenceMovieScene = SubSectionObject.GetSequence()->GetMovieScene();
	const FFrameTime HintFrameTime = CurrentTime * SubSectionObject.OuterToInnerTransform();

	return HintFrameTime;
}

#undef LOCTEXT_NAMESPACE

template<typename ParentSectionClass>
float TSubSectionMixin<ParentSectionClass>::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	if (UMovieSceneSubTrack* Track = SubSectionObject.GetTypedOuter<UMovieSceneSubTrack>())
	{
		return Track->GetRowHeight();
	}
	return TSubSectionMixin<ParentSectionClass>::TrackHeight;
}

template<typename ParentSectionClass>
bool TSubSectionMixin<ParentSectionClass>::IsReadOnly() const
{
	return SubSectionObject.IsReadOnly();
}

template<typename ParentSectionClass>
int32 TSubSectionMixin<ParentSectionClass>::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
    InPainter.LayerId = InPainter.PaintSectionBackground();

    FSubSectionPainterUtil::PaintSection(this->GetSequencer(), SubSectionObject, InPainter, FSubSectionPainterParams(this->GetContentPadding()));

    return InPainter.LayerId;
}

template<typename ParentSectionClass>
FReply TSubSectionMixin<ParentSectionClass>::OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent)
{
    if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
    {
		UMovieSceneSequence* Sequence = SubSectionObject.GetSequence();
        if (Sequence)
        {
			if (MouseEvent.IsControlDown())
			{
				GEditor->GetTimerManager()->SetTimerForNextTick([Sequence]()
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Sequence);
				});
			}
			else
			{
				this->GetSequencer()->FocusSequenceInstance(SubSectionObject);
			}
        }
    }

    return FReply::Handled();
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::BeginResizeSection()
{
    EditorUtil.BeginResizeSection();
    ParentSectionClass::BeginResizeSection();
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
    ResizeTime = EditorUtil.ResizeSection(ResizeMode, ResizeTime);
    ParentSectionClass::ResizeSection(ResizeMode, ResizeTime);
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::BeginSlipSection()
{
    EditorUtil.BeginSlipSection();
    ParentSectionClass::BeginSlipSection();
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::SlipSection(FFrameNumber SlipTime)
{
    SlipTime = EditorUtil.SlipSection(SlipTime);
    ParentSectionClass::SlipSection(SlipTime);
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::BeginDilateSection()
{
    EditorUtil.BeginDilateSection();
    ParentSectionClass::BeginDilateSection();
}

template<typename ParentSectionClass>
void TSubSectionMixin<ParentSectionClass>::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
    EditorUtil.DilateSection(NewRange, DilationFactor);
    ParentSectionClass::DilateSection(NewRange, DilationFactor);
}

template<typename ParentSectionClass>
bool TSubSectionMixin<ParentSectionClass>::RequestDeleteKeyArea( const TArray<FName>& KeyAreaNamePath )
{
	SubSectionObject.Modify();
	SubSectionObject.DeleteChannels(KeyAreaNamePath);
	return true;
}

#undef UE_API
