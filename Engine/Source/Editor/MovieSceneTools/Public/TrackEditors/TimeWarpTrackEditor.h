// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/Extensions/ITimeDomainExtension.h"

namespace UE::Sequencer { struct FAddKeyResult; }

namespace UE::Sequencer
{


	struct FTimeWarpTrackModel : FTrackModel, ITimeDomainExtension
	{
		UE_SEQUENCER_DECLARE_CASTABLE_API(MOVIESCENETOOLS_API, FTimeWarpTrackModel, FTrackModel, ITimeDomainExtension)

		FTimeWarpTrackModel(UMovieSceneTimeWarpTrack* TimeWarpTrack)
			: FTrackModel(TimeWarpTrack)
		{}

		bool ShouldAnchorToTop() const override
		{
			return true;
		}

		MOVIESCENETOOLS_API bool IsActiveTimeWarp() const;

		ETimeDomain GetDomain() const override
		{
			return ETimeDomain::Unwarped;
		}

		void OnConstruct() override;
	};


	class FTimeWarpTrackExtension : public IDynamicExtension
	{
	public:

		UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(MOVIESCENETOOLS_API, FTimeWarpTrackExtension)

		MOVIESCENETOOLS_API const FTimeWarpTrackModel* GetActiveTimeWarpTrack() const;

		TArray<TWeakViewModelPtr<FTimeWarpTrackModel>> WeakTimeWarpModels;
	};


} // namespace UE::Sequencer

class UMovieSceneTimeWarpGetter;

template <typename T> class TSubclassOf;

class FTimeWarpTrackEditor
	: public FKeyframeTrackEditor<UMovieSceneTimeWarpTrack>
{
public:

	FTimeWarpTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FKeyframeTrackEditor<UMovieSceneTimeWarpTrack>(InSequencer)
	{}

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
	{
		return MakeShared<FTimeWarpTrackEditor>(InSequencer);
	}

	static TSharedPtr<UE::Sequencer::FTrackModel> CreateTrackModel(UMovieSceneTrack* Track);

	MOVIESCENETOOLS_API void HandleAddTimeWarpTrack(TSubclassOf<UMovieSceneTimeWarpGetter> ClassType);

private:
	virtual FText GetDisplayName() const override;
	virtual void BuildPinnedAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
};
